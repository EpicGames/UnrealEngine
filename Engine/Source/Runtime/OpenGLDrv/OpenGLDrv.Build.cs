// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Linux", "Android", "LinuxArm64")]
[SupportedPlatformGroups("Windows")]
public class OpenGLDrv : ModuleRules
{
	public OpenGLDrv(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] {
			"Core",
			"RHI",
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"CoreUObject",
			"ApplicationCore",
			"Engine",
			"RHICore",
			"RenderCore",
			"PreLoadScreen",
			"TraceLog",
		});

		PrivateIncludePathModuleNames.Add("ImageWrapper");
		DynamicallyLoadedModuleNames.Add("ImageWrapper");

		PublicIncludePathModuleNames.Add("OpenGL");
		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenGL");

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");
		}

		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateDependencyModuleNames.Add("detex");
			PrivateDependencyModuleNames.Add("ArmlibGPUInfo");
		}

		if (Target.Platform == UnrealTargetPlatform.Android)
        {
			// for Swappy
			PrivateDefinitions.Add("USE_ANDROID_OPENGL_SWAPPY=1");

            PrivateDependencyModuleNames.Add("GoogleGameSDK");
			PrivateIncludePathModuleNames.Add("Launch");
		}

        if (!Target.IsInPlatformGroup(UnrealPlatformGroup.Windows)
			&& !Target.IsInPlatformGroup(UnrealPlatformGroup.IOS)
			&& Target.Platform != UnrealTargetPlatform.Android
			&& !Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrecompileForTargets = PrecompileTargetsType.None;
		}

		PrivateDefinitions.Add(Target.Platform == UnrealTargetPlatform.Android ? "USE_ANDROID_OPENGL=1" : "USE_ANDROID_OPENGL=0");
	}
}
