// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidEGL.h"

#if USE_ANDROID_OPENGL

#include "OpenGLDrvPrivate.h"
#include "AndroidEGL.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidWindow.h"
#include <android/native_window.h>
#if USE_ANDROID_JNI
#include <android/native_window_jni.h>
#endif
#include "Misc/ScopeLock.h"
#include "UObject/GarbageCollection.h"
#include "Android/AndroidPlatformFramePacer.h"
#include <dlfcn.h>
#include "UnrealEngine.h"


AndroidEGL* AndroidEGL::Singleton = NULL;
DEFINE_LOG_CATEGORY(LogEGL);

#if USE_ANDROID_EGL_NO_ERROR_CONTEXT
#ifndef EGL_KHR_create_context_no_error
#define EGL_KHR_create_context_no_error 1
#define EGL_CONTEXT_OPENGL_NO_ERROR_KHR   0x31B3
#endif // EGL_KHR_create_context_no_error
#endif // USE_ANDROID_EGL_NO_ERROR_CONTEXT

typedef int32(*PFN_ANativeWindow_setBuffersTransform)(struct ANativeWindow* window, int32 transform);
static PFN_ANativeWindow_setBuffersTransform ANativeWindow_setBuffersTransform_API = nullptr;

// Use blit by default as setBuffersTransform is broken on random devices
static TAutoConsoleVariable<int32> CVarAndroidGLESFlipYMethod(
	TEXT("r.Android.GLESFlipYMethod"),
	2,
	TEXT(" 0: Flip Y method detected automatically by GPU vendor.\n"
		 " 1: Force flip Y by native window setBuffersTransform.\n"
		 " 2: Force flip Y by BlitFrameBuffer."),
	ECVF_RenderThreadSafe);

const int EGLMinRedBits       = 5;
const int EGLMinGreenBits     = 6;
const int EGLMinBlueBits      = 5;
const int EGLMinAlphaBits     = 0;
const int EGLMinDepthBits     = 16;
const int EGLMinStencilBits	  = 8; // This is required for UMG clipping
const int EGLMinSampleBuffers = 0;
const int EGLMinSampleSamples = 0;


FPlatformOpenGLSurfaceContext::FPlatformOpenGLSurfaceContext(void* InWindowHandle) : FPlatformOpenGLSurfaceContext()
{
	// the context should be locked at the beginning of the update process, this overload is used for GT initiated events.
	check(IsInGameThread());

	AndroidWindow = (FAndroidWindow*)InWindowHandle;
	if (FAndroidMisc::UseNewWindowBehavior())
	{		
		if (AndroidWindow) // AndroidWindow can be null in the cases where offscreen rendering is expected.
		{
			FAndroidWindow::FNativeAccessor WindowContainer = AndroidWindow->GetANativeAccessor(false);
			// Note: the window handle is only locked within this scope.
			Window = WindowContainer.GetANativeWindow();
			// We continue to use the native handle as it will only be changed via the android event thread and enqueued to the RHIT
			// Only the android event thread and rhit will use the handle from this point.
		}
	}
}

struct AndroidESPImpl
{
	bool Initalized = false;

	// A single GL context is used for all surfaces.
	FPlatformOpenGLContext RenderingContext;
	FAndroidEGLDisplayInfo EGLDisplay;

	// SurfacesMutex protects GLSurfaces, but not the contained instances.
	UE::FMutex SurfacesMutex;
	TArray<TUniquePtr<FPlatformOpenGLSurfaceContext>> GLSurfaces;

	FPlatformOpenGLSurfaceContext* GetSurfaceFromAndroidWindow(const FAndroidWindow* FindMe)
	{
		UE::TUniqueLock ScopeLock(SurfacesMutex);
		check(!GLSurfaces.IsEmpty());
		TUniquePtr<FPlatformOpenGLSurfaceContext>* Found = GLSurfaces.FindByPredicate([FindMe](const TUniquePtr<FPlatformOpenGLSurfaceContext>& Item) -> bool
			{
				return Item->GetAndroidWindow() == FindMe;
			});
		return Found ? Found->Get() : nullptr;
	}

	FPlatformOpenGLSurfaceContext* GetGLSurfaceFromWindowContainer(const TOptional<FAndroidWindow::FNativeAccessor>& WindowContainer)
	{
		VERIFY_GL_SCOPE();
		FPlatformOpenGLSurfaceContext* Surface = nullptr;
		if (!FAndroidMisc::UseNewWindowBehavior())
		{
			Surface = GetSurfaceFromAndroidWindow(nullptr);
		}
		else if (WindowContainer.IsSet())
		{
			Surface = GetSurfaceFromAndroidWindow(WindowContainer->Get());
		}

		if (!Surface)
		{
			UE_LOGF(LogRHI, Warning, "GetGLSurfaceFromWindowContainer: surface context not found for the given window");
		}
		return Surface;
	}
};

FPlatformOpenGLSurfaceContext* AndroidEGL::FindSurfaceContextFromWindow(FAndroidWindow* InWindowHandle)
{
	UE::TUniqueLock ScopeLock(PImplData->SurfacesMutex);

	FAndroidWindow* FindMeAndroidWindow = (FAndroidWindow*)InWindowHandle;
	for (TUniquePtr<FPlatformOpenGLSurfaceContext>& Surface : PImplData->GLSurfaces)
	{
		if (Surface->GetAndroidWindow() == InWindowHandle)
		{
			return Surface.Get();
		}
	}
	return nullptr;
}

FPlatformOpenGLSurfaceContext* AndroidEGL::CreateSurfaceContextFromWindow(FAndroidWindow* InWindowHandle)
{
	check(FindSurfaceContextFromWindow(InWindowHandle) == nullptr);
	{
		UE::TUniqueLock ScopeLock(PImplData->SurfacesMutex);
		PImplData->GLSurfaces.Add(MakeUnique<FPlatformOpenGLSurfaceContext>(InWindowHandle));
	}
	return FindSurfaceContextFromWindow(InWindowHandle);
}

void AndroidEGL::DestroySurfaceContext(FPlatformOpenGLSurfaceContext* SurfaceContext)
{
	UE::TUniqueLock ScopeLock(PImplData->SurfacesMutex);
	DestroyBackBuffer(SurfaceContext);
	DestroyRenderSurface(SurfaceContext);

	PImplData->GLSurfaces.RemoveAll([SurfaceContext](const TUniquePtr<FPlatformOpenGLSurfaceContext>& FindSurface)
		{
			return FindSurface.Get() == SurfaceContext;
		});
}

const EGLint Attributes[] = {
	EGL_RED_SIZE,       EGLMinRedBits,
	EGL_GREEN_SIZE,     EGLMinGreenBits,
	EGL_BLUE_SIZE,      EGLMinBlueBits,
	EGL_ALPHA_SIZE,     EGLMinAlphaBits,
	EGL_DEPTH_SIZE,     EGLMinDepthBits,
	EGL_STENCIL_SIZE,   EGLMinStencilBits,
	EGL_SAMPLE_BUFFERS, EGLMinSampleBuffers,
	EGL_SAMPLES,        EGLMinSampleSamples,
	EGL_RENDERABLE_TYPE,  EGL_OPENGL_ES2_BIT,
	EGL_SURFACE_TYPE, EGL_WINDOW_BIT | EGL_PBUFFER_BIT,
	EGL_CONFIG_CAVEAT,  EGL_NONE,
	EGL_NONE
};

AndroidEGL::AndroidEGL()
{
	PImplData = new AndroidESPImpl();

	void* const LibNativeWindow = dlopen("libnativewindow.so", RTLD_NOW | RTLD_LOCAL);
	if (LibNativeWindow != nullptr)
	{
		ANativeWindow_setBuffersTransform_API = reinterpret_cast<PFN_ANativeWindow_setBuffersTransform>(dlsym(LibNativeWindow, "ANativeWindow_setBuffersTransform"));
	}
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("ANativeWindow_setBuffersTransform is %s on this device"), ANativeWindow_setBuffersTransform_API == nullptr ? TEXT("not supported") : TEXT("supported"));

	// Create a special case null surface context.
	// this is used for all windows when !FAndroidMisc::UseNewWindowBehavior()
	// or off screen rendering (e.g. when FAndroidMisc::UseNewWindowBehavior() but no window exists).
	AndroidEGL::CreateSurfaceContextFromWindow(nullptr);
}

void AndroidEGL::ResetDisplay()
{
	VERIFY_EGL_SCOPE();
	if(PImplData->EGLDisplay.eglDisplay != EGL_NO_DISPLAY)
	{
		FPlatformMisc::LowLevelOutputDebugStringf( TEXT("AndroidEGL::ResetDisplay(%p) " ), PImplData->EGLDisplay.eglDisplay);

#if USE_ANDROID_STANDALONE
		if (FAndroidMisc::UseNewWindowBehavior())
		{
			// This prevents flickering when attaching/detaching extra views in multi-viewport scenarios
			VERIFY_GL_SCOPE();
			glBindFramebuffer(GL_FRAMEBUFFER, 0);
			glBindFramebuffer(GL_READ_FRAMEBUFFER, 0);
			glBindFramebuffer(GL_DRAW_FRAMEBUFFER, 0);
		}
#endif

		if (eglMakeCurrent(PImplData->EGLDisplay.eglDisplay, EGL_NO_SURFACE, EGL_NO_SURFACE, EGL_NO_CONTEXT) == EGL_FALSE)
		{
			UE_LOGF(LogRHI, Warning, "eglMakeCurrent failed in ResetDisplay: 0x%x", eglGetError());
		}
	}
}

void AndroidEGL::DestroyRenderSurface(FPlatformOpenGLSurfaceContext* Surface)
{
	VERIFY_EGL_SCOPE();
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::DestroyRenderSurface(%p)"), Surface);
	if (Surface->eglSurface != EGL_NO_SURFACE)
	{
		eglDestroySurface(PImplData->EGLDisplay.eglDisplay, Surface->eglSurface);
		Surface->eglSurface = EGL_NO_SURFACE;
	}
}

void AndroidEGL::TerminateEGL()
{
	VERIFY_EGL_SCOPE();

	if (PImplData->EGLDisplay.auxSurface != EGL_NO_SURFACE)
	{
		eglDestroySurface(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.auxSurface);
		PImplData->EGLDisplay.auxSurface = EGL_NO_SURFACE;
	}

	if (PImplData->EGLDisplay.eglDisplay != EGL_NO_DISPLAY)
	{
		eglTerminate(PImplData->EGLDisplay.eglDisplay);
		PImplData->EGLDisplay.eglDisplay = EGL_NO_DISPLAY;
	}

	PImplData->Initalized = false;
}

/* Can be called from any thread */
EGLBoolean AndroidEGL::SetCurrentContext(EGLContext InContext, FPlatformOpenGLSurfaceContext* InSurfaceInfo)
{
	VERIFY_EGL_SCOPE();
	//context can be null.so can surface from PlatformNULLContextSetup
	EGLBoolean Result = EGL_TRUE;
	EGLContext CurrentContext = GetCurrentContext();
	EGLSurface CurrentSurface = GetCurrentSurface(EGL_DRAW);
	check(GetCurrentSurface(EGL_DRAW) == GetCurrentSurface(EGL_READ));

	EGLSurface WantedSurface = InSurfaceInfo ? InSurfaceInfo->eglSurface : EGL_NO_SURFACE;
	// activate the context
	if (CurrentContext != InContext || CurrentSurface != WantedSurface)
	{
		if (CurrentContext !=EGL_NO_CONTEXT)
		{
			glFlush();
		}

		if (InContext == EGL_NO_CONTEXT && InSurfaceInfo == nullptr)
		{
			// disconnect context from current thread
			Result = eglMakeCurrent(PImplData->EGLDisplay.eglDisplay, EGL_NO_SURFACE, EGL_NO_SURFACE, EGL_NO_CONTEXT);
			checkf(Result == EGL_TRUE, TEXT("ERROR: SetCurrentContext eglMakeCurrent EGL_NO_CONTEXT failed : 0x%x"), eglGetError());
		}
		else
		{
			//if we have a valid context, and no surface then create a tiny pbuffer and use that temporarily
			EGLSurface Surface = WantedSurface;
			if (!bSupportsKHRSurfacelessContext && InContext != EGL_NO_CONTEXT && Surface == EGL_NO_SURFACE)
			{
				// Reuse existing auxSurface if available, only create if needed
				if (PImplData->EGLDisplay.auxSurface == EGL_NO_SURFACE)
				{
					EGLint PBufferAttribs[] =
					{
						EGL_WIDTH, 1,
						EGL_HEIGHT, 1,
						EGL_TEXTURE_TARGET, EGL_NO_TEXTURE,
						EGL_TEXTURE_FORMAT, EGL_NO_TEXTURE,
						EGL_NONE
					};

					PImplData->EGLDisplay.auxSurface = eglCreatePbufferSurface(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.eglConfigParam, PBufferAttribs);

					checkf(PImplData->EGLDisplay.auxSurface != EGL_NO_SURFACE, TEXT("eglCreatePbufferSurface error : 0x%x"), eglGetError());
				}
				Surface = PImplData->EGLDisplay.auxSurface;
			}

		//	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::SetCurrentContext() %p, %p, %p, %p"), PImplData->EGLDisplay.eglDisplay, Surface, Surface, InContext);
			Result = eglMakeCurrent(PImplData->EGLDisplay.eglDisplay, Surface, Surface, InContext);
			checkf(Result == EGL_TRUE, TEXT("ERROR: SetCurrentContext eglMakeCurrent failed : 0x%x"), eglGetError());
		}
	}
	return Result;
}

void AndroidEGL::ResetInternal()
{
	Terminate();
}

void AndroidEGL::CreateEGLRenderSurface(FPlatformOpenGLSurfaceContext* Surface, bool bCreateWndSurface)
{
	VERIFY_EGL_SCOPE();
	check(Surface);

	// due to possible early initialization, don't redo this
	if (Surface->eglSurface != EGL_NO_SURFACE)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::CreateEGLRenderSurface() Already initialized: window=%p, %p"), Surface->Window, Surface->eglSurface);
		return;
	}

	Surface->bIsWndSurface = bCreateWndSurface;

	if (bCreateWndSurface)
	{
		check(Surface->Window);
		//need ANativeWindow
		Surface->eglSurface = eglCreateWindowSurface(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.eglConfigParam, Surface->Window, NULL);
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::CreateEGLRenderSurface() eglCreateWindowSurface: window=%p, %p"), Surface->Window, Surface->eglSurface);

		if (FAndroidPlatformRHIFramePacer::CVarAllowFrameTimestamps.GetValueOnAnyThread())
		{
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidEGL::CreateEGLRenderSurface(native window = %p) using a.allowFrameTimestamps enable EGL_TIMESTAMPS_ANDROID on %p"), Surface->Window, Surface->eglSurface);
			eglSurfaceAttrib(PImplData->EGLDisplay.eglDisplay, Surface->eglSurface, EGL_TIMESTAMPS_ANDROID, EGL_TRUE);
		}
		else
		{
			// HAD to add the false condition so that android attributes reflect current state of CVar.
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidEGL::CreateEGLRenderSurface(native Window = %p) using a.allowFrameTimestamps disable EGL_TIMESTAMPS_ANDROID on %p"), Surface->Window, Surface->eglSurface);
			eglSurfaceAttrib(PImplData->EGLDisplay.eglDisplay, Surface->eglSurface, EGL_TIMESTAMPS_ANDROID, EGL_FALSE);
		}

		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidEGL::CreateEGLRenderSurface() window=%p %p" ), Surface->Window, Surface->eglSurface);

		if (Surface->eglSurface == EGL_NO_SURFACE)
		{
			checkf(Surface->eglSurface != EGL_NO_SURFACE, TEXT("eglCreateWindowSurface error : 0x%x"), eglGetError());
			ResetInternal();
		}

		// On some Android devices, eglChooseConfigs will lie about valid configurations (specifically 32-bit color)
		/*	if (eglGetError() == EGL10.EGL_BAD_MATCH)
		{
		Logger.LogOut("eglCreateWindowSurface FAILED, retrying with more restricted context");

		// Dump what's already been initialized
		cleanupEGL();

		// Reduce target color down to 565
		eglAttemptedParams.redSize = 5;
		eglAttemptedParams.greenSize = 6;
		eglAttemptedParams.blueSize = 5;
		eglAttemptedParams.alphaSize = 0;
		initEGL(eglAttemptedParams);

		// try again
		eglSurface = eglCreateWindowSurface(PImplData->EGLDisplay.eglDisplay, eglConfig, surface, null);
		}

		*/
		EGLBoolean result = EGL_FALSE;
		if (!( result =  ( eglQuerySurface(PImplData->EGLDisplay.eglDisplay, Surface->eglSurface, EGL_WIDTH, &Surface->eglWidth) && eglQuerySurface(PImplData->EGLDisplay.eglDisplay, Surface->eglSurface, EGL_HEIGHT, &Surface->eglHeight) ) ) )
		{
			ResetInternal();
		}

		checkf(result == EGL_TRUE, TEXT("eglQuerySurface error : 0x%x"), eglGetError());
	}
	else
	{
		// create a fake surface instead
		EGLint pbufferAttribs[] =
		{
			EGL_WIDTH, 1,
			EGL_HEIGHT, 1,
			EGL_TEXTURE_TARGET, EGL_NO_TEXTURE,
			EGL_TEXTURE_FORMAT, EGL_NO_TEXTURE,
			EGL_NONE
		};

		checkf(Surface->eglWidth != 0, TEXT("eglWidth is ZERO; could be a problem!"));
		checkf(Surface->eglHeight != 0, TEXT("eglHeight is ZERO; could be a problem!"));
		pbufferAttribs[1] = Surface->eglWidth;
		pbufferAttribs[3] = Surface->eglHeight;

		FPlatformMisc::LowLevelOutputDebugStringf( TEXT("AndroidEGL::CreateEGLRenderSurface(%d), eglSurface = eglCreatePbufferSurface(), %dx%d" ),
			int(bCreateWndSurface), pbufferAttribs[1], pbufferAttribs[3]);
		Surface->eglSurface = eglCreatePbufferSurface(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.eglConfigParam, pbufferAttribs);
		if(Surface->eglSurface== EGL_NO_SURFACE )
		{
			checkf(Surface->eglSurface != EGL_NO_SURFACE, TEXT("eglCreatePbufferSurface error : 0x%x"), eglGetError());
			ResetInternal();
		}
	}
}

void AndroidEGL::InitEGL(APIVariant API)
{
	VERIFY_EGL_SCOPE();
	// make sure we only do this once (it's optionally done early for cooker communication)
//	static bool bAlreadyInitialized = false;
	if (PImplData->Initalized)
	{
		return;
	}
//	bAlreadyInitialized = true;

	check(PImplData->EGLDisplay.eglDisplay == EGL_NO_DISPLAY)
	PImplData->EGLDisplay.eglDisplay = eglGetDisplay(EGL_DEFAULT_DISPLAY);
	checkf(PImplData->EGLDisplay.eglDisplay, TEXT(" eglGetDisplay error : 0x%x "), eglGetError());
	
	EGLBoolean  result = 	eglInitialize(PImplData->EGLDisplay.eglDisplay, 0 , 0);
	checkf( result == EGL_TRUE, TEXT("elgInitialize error: 0x%x "), eglGetError());

	// Get the EGL Extension list to determine what is supported
	FString Extensions = ANSI_TO_TCHAR( eglQueryString( PImplData->EGLDisplay.eglDisplay, EGL_EXTENSIONS));

	UE_LOGF(LogAndroid, Log, "EGL Extensions: \n%ls", *Extensions);

	bSupportsKHRCreateContext = Extensions.Contains(TEXT("EGL_KHR_create_context"));
	bSupportsKHRSurfacelessContext = Extensions.Contains(TEXT("EGL_KHR_surfaceless_context"));
	bSupportsKHRNoErrorContext = Extensions.Contains(TEXT("EGL_KHR_create_context_no_error"));
	bSupportsEXTRobustContext = Extensions.Contains(TEXT("EGL_EXT_create_context_robustness"));

	if (API == AV_OpenGLES)
	{
		result = eglBindAPI(EGL_OPENGL_ES_API);
	}
	else if (API == AV_OpenGLCore)
	{
		result = eglBindAPI(EGL_OPENGL_API);
	}
	else
	{
		checkf( 0, TEXT("Attempt to initialize EGL with unexpected API type"));
	}

	checkf( result == EGL_TRUE, TEXT("eglBindAPI error: 0x%x "), eglGetError());

#if ENABLE_CONFIG_FILTER

	EGLConfig* EGLConfigList = NULL;
	result = eglChooseConfig(PImplData->EGLDisplay.eglDisplay, Attributes, NULL, 0, &PImplData->EGLDisplay.eglNumConfigs);
	if (result)
	{
		int NumConfigs = PImplData->EGLDisplay.eglNumConfigs;
		EGLConfigList = new EGLConfig[NumConfigs];
		result = eglChooseConfig(PImplData->EGLDisplay.eglDisplay, Attributes, EGLConfigList, NumConfigs, &PImplData->EGLDisplay.eglNumConfigs);
	}
	if (!result)
	{
		ResetInternal();
	}

	checkf(result == EGL_TRUE , TEXT(" eglChooseConfig error: 0x%x"), eglGetError());

	checkf(PImplData->EGLDisplay.eglNumConfigs != 0  ,TEXT(" eglChooseConfig num EGLConfigLists is 0 . error: 0x%x"), eglGetError());

	int ResultValue = 0 ;
	bool haveConfig = false;
	int64 score = LONG_MAX;
	for (uint32_t i = 0; i < PImplData->EGLDisplay.eglNumConfigs; i++)
	{
		int64 currScore = 0;
		int r, g, b, a, d, s, sb, sc, nvi;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_RED_SIZE, &ResultValue); r = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_GREEN_SIZE, &ResultValue); g = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_BLUE_SIZE, &ResultValue); b = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_ALPHA_SIZE, &ResultValue); a = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_DEPTH_SIZE, &ResultValue); d = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_STENCIL_SIZE, &ResultValue); s = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_SAMPLE_BUFFERS, &ResultValue); sb = ResultValue;
		eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_SAMPLES, &ResultValue); sc = ResultValue;

		// Optional, Tegra-specific non-linear depth buffer, which allows for much better
		// effective depth range in relatively limited bit-depths (e.g. 16-bit)
		int bNonLinearDepth = 0;
		if (eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_DEPTH_ENCODING_NV, &ResultValue))
		{
			bNonLinearDepth = (ResultValue == EGL_DEPTH_ENCODING_NONLINEAR_NV) ? 1 : 0;
		}
		else
		{
			// explicitly consume the egl error if EGL_DEPTH_ENCODING_NV does not exist.
			GetError();
		}

		// Favor EGLConfigLists by RGB, then Depth, then Non-linear Depth, then Stencil, then Alpha
		currScore = 0;
		currScore |= ((int64)FPlatformMath::Min(FPlatformMath::Abs(sb - PImplData->EGLDisplay.Parms.sampleBuffers), 15)) << 29;
		currScore |= ((int64)FPlatformMath::Min(FPlatformMath::Abs(sc - PImplData->EGLDisplay.Parms.sampleSamples), 31)) << 24;
		currScore |= FPlatformMath::Min(
						FPlatformMath::Abs(r - PImplData->EGLDisplay.Parms.redSize) +
						FPlatformMath::Abs(g - PImplData->EGLDisplay.Parms.greenSize) +
						FPlatformMath::Abs(b - PImplData->EGLDisplay.Parms.blueSize), 127) << 17;
		currScore |= FPlatformMath::Min(FPlatformMath::Abs(d - PImplData->EGLDisplay.Parms.depthSize), 63) << 11;
		currScore |= FPlatformMath::Min(FPlatformMath::Abs(1 - bNonLinearDepth), 1) << 10;
		currScore |= FPlatformMath::Min(FPlatformMath::Abs(s - PImplData->EGLDisplay.Parms.stencilSize), 31) << 6;
		currScore |= FPlatformMath::Min(FPlatformMath::Abs(a - PImplData->EGLDisplay.Parms.alphaSize), 31) << 0;

#if ENABLE_EGL_DEBUG
		LogConfigInfo(EGLConfigList[i]);
#endif

		if (currScore < score || !haveConfig)
		{
			PImplData->EGLDisplay.eglConfigParam	= EGLConfigList[i];
			PImplData->EGLDisplay.DepthSize = d;		// store depth/stencil sizes
			haveConfig	= true;
			score		= currScore;
			eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[i], EGL_NATIVE_VISUAL_ID, &ResultValue);
			PImplData->EGLDisplay.NativeVisualID = ResultValue;
		}
	}
	check(haveConfig);
	delete[] EGLConfigList;
#else

	EGLConfig EGLConfigList[1];
	if(!( result = eglChooseConfig(PImplData->EGLDisplay.eglDisplay, Attributes, EGLConfigList, 1,  &PImplData->EGLDisplay.eglNumConfigs)))
	{
		ResetInternal();
	}

	checkf(result == EGL_TRUE , TEXT(" eglChooseConfig error: 0x%x"), eglGetError());

	checkf(PImplData->EGLDisplay.eglNumConfigs != 0  ,TEXT(" eglChooseConfig num EGLConfigLists is 0 . error: 0x%x"), eglGetError());
	PImplData->EGLDisplay.eglConfigParam = EGLConfigList[0];
	int ResultValue = 0 ;
	eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[0], EGL_DEPTH_SIZE, &ResultValue); PImplData->EGLDisplay.DepthSize = ResultValue;
	eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, EGLConfigList[0], EGL_NATIVE_VISUAL_ID, &ResultValue); PImplData->EGLDisplay.NativeVisualID = ResultValue;
#endif
}

// call out to JNI to see if the application was packaged for XRImmersiveHMD
extern bool AndroidThunkCpp_IsXRImmersiveHMDApplication();

void AndroidEGL::SetRenderContextWindowSurface(const TOptional<FAndroidWindow::FNativeAccessor>& WindowContainer)
{
	VERIFY_GL_SCOPE();

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::SetRenderContextWindowSurface  recreating context! tid: %d"),  FPlatformTLS::GetCurrentThreadId());

	FPlatformOpenGLSurfaceContext* Surface = PImplData->GetGLSurfaceFromWindowContainer(WindowContainer);
	if (!Surface)
	{
		UE_LOGF(LogRHI, Warning, "AndroidEGL::SetRenderContextWindowSurface: no surface context available, aborting");
		return;
	}

	UnBindRender(Surface);

	SetCurrentContext(EGL_NO_CONTEXT, nullptr);
	bool bCreateSurface = !AndroidThunkCpp_IsXRImmersiveHMDApplication();

	if (!FAndroidMisc::UseNewWindowBehavior())
	{
		// SetRenderContextWindowSurface is called only when the window lock is successful.
		Surface->Window = (ANativeWindow*)FAndroidWindow::GetHardwareWindow_EventThread();
	}
	else
	{
		check(WindowContainer.IsSet());
		Surface->Window = WindowContainer->GetANativeWindow();
	}
	check(Surface->Window);

	InitRenderSurface(false, bCreateSurface, Surface);
	SetCurrentRenderingContext(Surface);

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::SetRenderContextWindowSurface  DONE! tid: %d"),  FPlatformTLS::GetCurrentThreadId());
}

void AndroidEGL::ResizeRenderContextSurface(FPlatformOpenGLSurfaceContext* Surface)
{
	VERIFY_GL_SCOPE();

	// Resize render originates from the gamethread, we cant use Window_Event here.
	if (Surface->eglWidth != (Surface->CachedWindowRect.Right - Surface->CachedWindowRect.Left) ||
		Surface->eglHeight != (Surface->CachedWindowRect.Bottom - Surface->CachedWindowRect.Top))
	{
		UE_LOGF(LogAndroid, Log,     "AndroidEGL::ResizeRenderContextSurface, Surface->Window=%p, Surface->eglWidth=%d, Surface->eglHeight=%d!, CachedWidth=%d, CachedHeight=%d, tid: %d", 
			Surface->Window,
			Surface->eglWidth,
			Surface->eglHeight,
			(Surface->CachedWindowRect.Right - Surface->CachedWindowRect.Left),
			(Surface->CachedWindowRect.Bottom - Surface->CachedWindowRect.Top),
			FPlatformTLS::GetCurrentThreadId()
		);

		UnBindRender(Surface);

		{
			bool bCreateSurface = !AndroidThunkCpp_IsXRImmersiveHMDApplication();
			InitRenderSurface(false, bCreateSurface, Surface);
			SetCurrentRenderingContext(Surface);
		}
	}
}

AndroidEGL* AndroidEGL::GetInstance()
{
	if(!Singleton)
	{
		Singleton = new AndroidEGL();
	}

	return Singleton;
}

void AndroidEGL::DestroyBackBuffer(FPlatformOpenGLSurfaceContext* Surface)
{
	if (Surface->ViewportFramebuffer)
	{
		VERIFY_GL_SCOPE();
		glDeleteFramebuffers(1, &Surface->ViewportFramebuffer);

		Surface->ViewportFramebuffer = 0 ;
	}
}

// this is expected to happen once, during RHIInit
void AndroidEGL::InitBackBuffer(struct FPlatformOpenGLSurfaceContext* Surface)
{
	if (Surface == nullptr)
	{
		if (!FAndroidMisc::UseNewWindowBehavior())
		{
			Surface = FindSurfaceContextFromWindow(nullptr);
		}
		else
		{
			return; //to satisfy lint warning
		}
	} 

	check(Surface);
	check(Surface->ViewportFramebuffer == 0);
	VERIFY_GL_SCOPE();
	glGenFramebuffers(1, &Surface->ViewportFramebuffer);
}

extern void AndroidThunkCpp_SetDesiredViewSize(int32 Width, int32 Height);

void AndroidEGL::InitRenderSurface(bool bUseSmallSurface, bool bCreateWndSurface, FPlatformOpenGLSurfaceContext* Surface)
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::InitRenderSurface %d, %d"), int(bUseSmallSurface), int(bCreateWndSurface));
	if (Surface == nullptr)
	{
		// todo: validate, only needed 
		Surface = FindSurfaceContextFromWindow(nullptr);
	}

	if (FAndroidMisc::UseNewWindowBehavior())
	{
		bUseSmallSurface = true;
		bCreateWndSurface = false;

		{
			if(Surface->Window)
			{
				bUseSmallSurface = false;
				bCreateWndSurface = true;
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::InitRenderSurface window %p was supplied %d, %d"), Surface->Window, int(bUseSmallSurface), int(bCreateWndSurface));
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::InitRenderSurface null native window was supplied %d, %d"), int(bUseSmallSurface), int(bCreateWndSurface));
			}
		}
	}
	else
	{
		// the previous window behaviour should not see a null native window..
		check(Surface->Window);
	}

	int32 Width = 8, Height = 8;
	if (!bUseSmallSurface)
	{
		FPlatformRect WindowSize = FAndroidWindow::GetScreenRect();

		if (Surface->CachedWindowRect.Right > 0 && Surface->CachedWindowRect.Bottom > 0)
		{
			// If we resumed from a lost window reuse the window size, the game thread will update the window dimensions.
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::InitRenderSurface, Using CachedWindowRect, left: %d, top: %d, right: %d, bottom: %d "), Surface->CachedWindowRect.Left, Surface->CachedWindowRect.Top, Surface->CachedWindowRect.Right, Surface->CachedWindowRect.Bottom);
			WindowSize = Surface->CachedWindowRect;
		}
#if USE_ANDROID_STANDALONE
		if (WindowSize.Left != 0 || WindowSize.Top != 0)
		{
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("AndroidEGL::InitRenderSurface, WARNING!!! WindowSize is offset, left: %d, top: %d, right: %d, bottom: %d "), WindowSize.Left, WindowSize.Top, WindowSize.Right, WindowSize.Bottom);
		}
		Width = WindowSize.Right - WindowSize.Left;
		Height = WindowSize.Bottom - WindowSize.Top;

		if (Surface->Window)
		{
			Width = ANativeWindow_getWidth(Surface->Window);
			Height = ANativeWindow_getHeight(Surface->Window);
		}
#else

		Width = WindowSize.Right;
		Height = WindowSize.Bottom;
		
		AndroidThunkCpp_SetDesiredViewSize(Width, Height);
#endif

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::InitRenderSurface, Using width: %d, height %d "), Width, Height);
	}

	if(Surface->Window)
	{
		FIntVector2 OriginalWindowSize(ANativeWindow_getWidth(Surface->Window), ANativeWindow_getHeight(Surface->Window));

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::InitRenderSurface, setting wnd: %p, width: %d->%d, height %d->%d "), Surface->Window, OriginalWindowSize.X, Width, OriginalWindowSize.Y, Height);
		ANativeWindow_setBuffersGeometry(Surface->Window, Width, Height, PImplData->EGLDisplay.NativeVisualID);
	}

	CreateEGLRenderSurface(Surface, bCreateWndSurface);
}

// EGL_CONTEXT_OPENGL_ROBUST_ACCESS_EXT is enabled if configrules asks for it or the command line specifies it.
// If -OpenGLRobustContext=[0/1] is specified on the command line it takes precedence.
void AndroidEGL::Init(APIVariant API, uint32 MajorVersion, uint32 MinorVersion)
{
	check(IsInGameThread());
	const bool bDebug = IsOGLDebugOutputEnabled();
	const FString* ConfigRulesForceRobustGLContext = FAndroidMisc::GetConfigRulesVariable(TEXT("ForceRobustGLContext"));
	bool bWantsRobustGLContext = ConfigRulesForceRobustGLContext && ConfigRulesForceRobustGLContext->Equals("true", ESearchCase::IgnoreCase);
	
	FString RobustArg;
	if (FParse::Value(FCommandLine::Get(), TEXT("-OpenGLRobustContext="), RobustArg))
	{
		bWantsRobustGLContext = RobustArg.Contains(TEXT("1"));
	}
	
	if (PImplData->Initalized)
	{
		ensure(bDebug == PImplData->RenderingContext.bIsDebug); // if this fires you would need to tear down the previous context and recreate to honour the debug change.
		return;
	}

	InitEGL(API);
	PImplData->RenderingContext.bIsDebug = bDebug;
	if (bSupportsKHRCreateContext)
	{
		const uint32 MaxElements = 16;
		uint32 Flags = 0;

		Flags |= PImplData->RenderingContext.bIsDebug ? EGL_CONTEXT_OPENGL_DEBUG_BIT_KHR : 0;

		ContextAttributes = new int[MaxElements];
		uint32 Element = 0;

		ContextAttributes[Element++] = EGL_CONTEXT_MAJOR_VERSION_KHR;
		ContextAttributes[Element++] = MajorVersion;
		ContextAttributes[Element++] = EGL_CONTEXT_MINOR_VERSION_KHR;
		ContextAttributes[Element++] = MinorVersion;
#if USE_ANDROID_EGL_NO_ERROR_CONTEXT
		if (bSupportsKHRNoErrorContext && AndroidThunkCpp_IsXRImmersiveHMDApplication())
		{
			ContextAttributes[Element++] = EGL_CONTEXT_OPENGL_NO_ERROR_KHR;
			ContextAttributes[Element++] = EGL_TRUE;
		}
#endif // USE_ANDROID_EGL_NO_ERROR_CONTEXT

		bIsEXTRobustContextActive = bSupportsEXTRobustContext && bWantsRobustGLContext;
		if (bIsEXTRobustContextActive)
		{
			UE_LOGF(LogAndroid, Log, "Enabling: EGL_CONTEXT_OPENGL_ROBUST_ACCESS_EXT");
			ContextAttributes[Element++] = EGL_CONTEXT_OPENGL_ROBUST_ACCESS_EXT;
			ContextAttributes[Element++] = EGL_TRUE;
		}

		if (API == AV_OpenGLCore)
		{
			ContextAttributes[Element++] = EGL_CONTEXT_OPENGL_PROFILE_MASK_KHR;
			ContextAttributes[Element++] = EGL_CONTEXT_OPENGL_CORE_PROFILE_BIT_KHR;
		}
		ContextAttributes[Element++] = EGL_CONTEXT_FLAGS_KHR;
		ContextAttributes[Element++] = Flags;
		ContextAttributes[Element++] = EGL_NONE;

		checkf( Element <= MaxElements, TEXT("Too many elements in config list"));
	}
	else
	{
		// Fall back to the least common denominator
		ContextAttributes = new int[3];
		ContextAttributes[0] = EGL_CONTEXT_CLIENT_VERSION;
		ContextAttributes[1] = 2;
		ContextAttributes[2] = EGL_NONE;
	}

	bool bSuccess = InitContexts();

	// Try to create the context again for ES3.1 if it is failed to create for ES3.2
	if (!bSuccess && ContextAttributes[3] > 1)
	{
		ContextAttributes[3] -= 1;

		bSuccess = InitContexts();

		if (!bSuccess)
		{
			// Try to create an ES2 context if ES3.1 also failed, which can happen in the Android emulator.
			// This is enough for FAndroidGPUInfo detection to enable Vulkan.
			ContextAttributes[0] = EGL_CONTEXT_CLIENT_VERSION;
			ContextAttributes[1] = 2;
			ContextAttributes[2] = EGL_NONE;

			bSuccess = InitContexts();
		}
	}

	PImplData->Initalized   = true;
}

AndroidEGL::~AndroidEGL()
{
	delete PImplData;
	delete []ContextAttributes;
}

void AndroidEGL::DestroyContext(EGLContext InContext)
{
	VERIFY_EGL_SCOPE();
	if(InContext != EGL_NO_CONTEXT) //soft fail
	{
		eglDestroyContext(PImplData->EGLDisplay.eglDisplay, InContext);
	}
}

EGLContext AndroidEGL::CreateContext(EGLContext InParentContext)
{
	VERIFY_EGL_SCOPE();
	return eglCreateContext(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.eglConfigParam, InParentContext, ContextAttributes);
}

int32 AndroidEGL::GetError()
{
	return eglGetError();
}

bool AndroidEGL::IsInitialized()
{
	return PImplData->Initalized;
}

EGLContext AndroidEGL::GetCurrentContext()
{
	VERIFY_EGL_SCOPE();
	return eglGetCurrentContext();
}

EGLSurface AndroidEGL::GetCurrentSurface(EGLint readdraw)
{
	VERIFY_EGL_SCOPE();
	return eglGetCurrentSurface(readdraw);
}

EGLDisplay AndroidEGL::GetDisplay() const
{
	return PImplData->EGLDisplay.eglDisplay;
}

EGLSurface AndroidEGL::GetSurface() const
{
	FPlatformOpenGLSurfaceContext* Surface = PImplData->GetSurfaceFromAndroidWindow(nullptr);
	return Surface->eglSurface;
}

EGLConfig AndroidEGL::GetConfig() const
{
	return PImplData->EGLDisplay.eglConfigParam;
}

bool AndroidEGL::IsUsingWindowedSurface() const
{
	FPlatformOpenGLSurfaceContext* Surface = PImplData->GetSurfaceFromAndroidWindow(nullptr);
	return Surface->bIsWndSurface;
}

void AndroidEGL::GetSwapIntervalRange(EGLint& OutMinSwapInterval, EGLint& OutMaxSwapInterval) const
{
	eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.eglConfigParam, EGL_MIN_SWAP_INTERVAL, &OutMinSwapInterval);
	eglGetConfigAttrib(PImplData->EGLDisplay.eglDisplay, PImplData->EGLDisplay.eglConfigParam, EGL_MAX_SWAP_INTERVAL, &OutMaxSwapInterval);
}


ANativeWindow* AndroidEGL::GetNativeWindow() const
{
	FPlatformOpenGLSurfaceContext* Surface = PImplData->GetSurfaceFromAndroidWindow(nullptr);
	return Surface->Window;
}

bool AndroidEGL::InitContexts()
{
	PImplData->RenderingContext.eglContext = CreateContext(EGL_NO_CONTEXT);
	return PImplData->RenderingContext.eglContext != EGL_NO_CONTEXT;
}

void AndroidEGL::AcquireCurrentRenderingContext()
{
	// we can acquire a context without a windowed surface.
	SetCurrentRenderingContext(nullptr);
}

void AndroidEGL::SetCurrentRenderingContext(FPlatformOpenGLSurfaceContext* InSurface)
{
	SetCurrentContext(PImplData->RenderingContext.eglContext, InSurface);
}

void AndroidEGL::ReleaseContextOwnership()
{
	if (PlatformOpenGLThreadHasRenderingContext())
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::  ReleaseContextOwnership,  thread releasing rendering context tid: %d"), FPlatformTLS::GetCurrentThreadId());
		SetCurrentContext(EGL_NO_CONTEXT, nullptr);
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::  ReleaseContextOwnership,  rendering context was not current to this thread tid: %d"), FPlatformTLS::GetCurrentThreadId());
	}
}

void AndroidEGL::Terminate()
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL:: Terminate! tid: %d"), FPlatformTLS::GetCurrentThreadId());

	ResetDisplay();

	DestroyContext(PImplData->RenderingContext.eglContext);
	PImplData->RenderingContext.Reset();

	{
		UE::TUniqueLock ScopeLock(PImplData->SurfacesMutex);
		for (TUniquePtr<FPlatformOpenGLSurfaceContext>& Surface : PImplData->GLSurfaces)
		{
			DestroyBackBuffer(Surface.Get());
			DestroyRenderSurface(Surface.Get());
		}
		PImplData->GLSurfaces.Empty();
	}

	TerminateEGL();
}

bool AndroidEGL::ThreadHasRenderingContext()
{
	return GetCurrentContext() == PImplData->RenderingContext.eglContext;
}

FPlatformOpenGLContext* AndroidEGL::GetRenderingContext()
{
	return &PImplData->RenderingContext;
}

bool AndroidEGL::GetSupportsNoErrorContext()
{
	return bSupportsKHRNoErrorContext;
}

void AndroidEGL::UnBindRender(FPlatformOpenGLSurfaceContext* Surface)
{
	check(Surface);
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::UnBindRender(%p)"), Surface->Window);
	ResetDisplay();
	DestroyRenderSurface(Surface);
}

// Called by new window behaviour when the android event thread wants to destroy a window.
// The event thread waits for this to complete.
void AndroidEGL::OnLostNativeWindow(FPlatformOpenGLSurfaceContext* Surface)
{
	check(FAndroidMisc::UseNewWindowBehavior());
	// remove a surface from the context, free the surface and rebind the context with a null windows. (this creates a pbuffer surface and binds the context to it.)
	UnBindRender(Surface);
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidEGL::OnLostNativeWindow(native window = null (was %p) )"), Surface->Window);
	Surface->Window = nullptr;
	// recreate the surface and make current
	InitRenderSurface(true, false, Surface);
	SetCurrentRenderingContext(Surface);
}


void FAndroidAppEntry::ReInitWindow(const TOptional<FAndroidWindow::FNativeAccessor>& WindowContainer)
{
	check(IsInGameThread());

	// Window creation is now handled by BlockRendering, when it resumes after a new window is created.
	FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidEGL::ReInitWindow()"));

	GSystemResolution.bForceRefresh = true;

	// It isn't safe to call ShouldUseVulkan if AndroidEGL is not initialized.
	// However, since we don't need to ReInit the window in that case anyways we
	// can return early.
	if (!AndroidEGL::GetInstance()->IsInitialized())
	{
		return;
	}

	// @todo vulkan: Clean this up, and does vulkan need any code here?
	if (!FAndroidMisc::ShouldUseVulkan())
	{
		// the window size could have been adjusted by the GT by now, if so it must be updated.
		AndroidEGL::GetInstance()->RefreshWindowSize(WindowContainer);
	}
}

void AndroidEGL::RefreshWindowSize(const TOptional<FAndroidWindow::FNativeAccessor>& WindowContainer)
{
	check(IsInGameThread());
	check(!FAndroidMisc::ShouldUseVulkan());
	FPlatformRect WindowRect = FAndroidWindow::GetScreenRect();

	// Extract the FAndroidWindow* before lambda capture (FNativeAccessor is non-copyable)
	const FAndroidWindow* AndroidWindowPtr = nullptr;
	if (FAndroidMisc::UseNewWindowBehavior() && WindowContainer.IsSet())
	{
		AndroidWindowPtr = WindowContainer->Get();
	}

	ENQUEUE_RENDER_COMMAND(EGLResizeRenderContextSurface)(
		[this, WindowRect, AndroidWindowPtr](FRHICommandListImmediate& RHICmdList) mutable
	{
			RHICmdList.EnqueueLambda([this, WindowRect, AndroidWindowPtr](FRHICommandListImmediate&) mutable
				{
					FPlatformOpenGLSurfaceContext* Surface = PImplData->GetSurfaceFromAndroidWindow(AndroidWindowPtr);
					if (!Surface)
					{
						UE_LOGF(LogAndroid, Warning, "AndroidEGL::RefreshWindowSize surface context not found, skipping resize");
						return;
					}
					UE_LOGF(LogAndroid, Log, "AndroidEGL::RefreshWindowSize updating window size = %d, %d, cached size : %d, %d tid : %d", WindowRect.Right, WindowRect.Bottom, Surface->CachedWindowRect.Right, Surface->CachedWindowRect.Bottom, FPlatformTLS::GetCurrentThreadId());
					Surface->CachedWindowRect = WindowRect;
					AndroidEGL::GetInstance()->ResizeRenderContextSurface(Surface);
		});
	});

	FlushRenderingCommands();
}

void FAndroidAppEntry::OnPauseEvent()
{
	const auto& OnPauseCallback = FAndroidMisc::GetOnPauseCallback();
	if (OnPauseCallback)
	{
		OnPauseCallback();
	}
}

void AndroidEGL::LogConfigInfo(EGLConfig  EGLConfigInfo)
{
	VERIFY_EGL_SCOPE();
	EGLint ResultValue = 0 ;
	EGLDisplay eglDisplay = PImplData->EGLDisplay.eglDisplay;
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_RED_SIZE, &ResultValue); FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo : EGL_RED_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_GREEN_SIZE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_GREEN_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_BLUE_SIZE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_BLUE_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_ALPHA_SIZE, &ResultValue); FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_ALPHA_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_DEPTH_SIZE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_DEPTH_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_STENCIL_SIZE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_STENCIL_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_SAMPLE_BUFFERS, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_SAMPLE_BUFFERS :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_BIND_TO_TEXTURE_RGB, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_BIND_TO_TEXTURE_RGB :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_SAMPLES, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_SAMPLES :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_COLOR_BUFFER_TYPE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_COLOR_BUFFER_TYPE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_CONFIG_CAVEAT, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_CONFIG_CAVEAT :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_CONFIG_ID, &ResultValue); FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_CONFIG_ID :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_CONFORMANT, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_CONFORMANT :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_LEVEL, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_LEVEL :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_LUMINANCE_SIZE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_LUMINANCE_SIZE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_MAX_PBUFFER_WIDTH, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_MAX_PBUFFER_WIDTH :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_MAX_PBUFFER_HEIGHT, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_MAX_PBUFFER_HEIGHT :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_MAX_PBUFFER_PIXELS, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_MAX_PBUFFER_PIXELS :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_MAX_SWAP_INTERVAL, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_MAX_SWAP_INTERVAL :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_MIN_SWAP_INTERVAL, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_MIN_SWAP_INTERVAL :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_NATIVE_RENDERABLE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_NATIVE_RENDERABLE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_NATIVE_VISUAL_TYPE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_NATIVE_VISUAL_TYPE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_NATIVE_VISUAL_ID, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_NATIVE_VISUAL_ID :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_RENDERABLE_TYPE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_RENDERABLE_TYPE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_SURFACE_TYPE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_SURFACE_TYPE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_TRANSPARENT_TYPE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_TRANSPARENT_TYPE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_TRANSPARENT_RED_VALUE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_TRANSPARENT_RED_VALUE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_TRANSPARENT_GREEN_VALUE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_TRANSPARENT_GREEN_VALUE :	%u" ), ResultValue );
	eglGetConfigAttrib(eglDisplay, EGLConfigInfo, EGL_TRANSPARENT_BLUE_VALUE, &ResultValue);  FPlatformMisc::LowLevelOutputDebugStringf( TEXT("EGLConfigInfo :EGL_TRANSPARENT_BLUE_VALUE :	%u" ), ResultValue );
}

void AndroidEGL::UpdateBuffersTransform()
{
	if (ANativeWindow_setBuffersTransform_API != nullptr && !IsOfflineSurfaceRequired())
	{
		int32 BufferTransform = ANATIVEWINDOW_TRANSFORM_IDENTITY;

		EDeviceScreenOrientation ScreenOrientation = FPlatformMisc::GetDeviceOrientation();
		
		// Update the device orientation in case it hasn't been updated yet.
		if (ScreenOrientation == EDeviceScreenOrientation::Unknown)
		{
			FAndroidMisc::UpdateDeviceOrientation();
			ScreenOrientation = FPlatformMisc::GetDeviceOrientation();
		}

		switch (ScreenOrientation)
		{
		case EDeviceScreenOrientation::Portrait:
			BufferTransform = ANATIVEWINDOW_TRANSFORM_MIRROR_VERTICAL;
			break;

		case EDeviceScreenOrientation::PortraitUpsideDown:
			BufferTransform = ANATIVEWINDOW_TRANSFORM_MIRROR_HORIZONTAL;
			break;

		case EDeviceScreenOrientation::LandscapeLeft:
			BufferTransform = ANATIVEWINDOW_TRANSFORM_ROTATE_90 | ANATIVEWINDOW_TRANSFORM_MIRROR_VERTICAL;
			break;

		case EDeviceScreenOrientation::LandscapeRight:
			BufferTransform = ANATIVEWINDOW_TRANSFORM_ROTATE_90 | ANATIVEWINDOW_TRANSFORM_MIRROR_HORIZONTAL;
			break;

		default:
			ensureMsgf(false, TEXT("BufferTransform %d should be handled with no exception, otherwise wrong orientation could be displayed on device"), BufferTransform);
			break;
		}

		ANativeWindow_setBuffersTransform_API(GetNativeWindow(), BufferTransform);
	}
}

bool AndroidEGL::IsOfflineSurfaceRequired()
{
	return FAndroidMisc::SupportsBackbufferSampling()
		// force to use BlitFrameBuffer
		|| CVarAndroidGLESFlipYMethod.GetValueOnAnyThread() == 2
		// setBuffersTransform doesn't work on android 9 and below devices
		|| !(CVarAndroidGLESFlipYMethod.GetValueOnAnyThread() == 1 || FAndroidMisc::GetAndroidMajorVersion() >= 10)
		// setBuffersTransform doesn't work on arm and powerVR GPU devices
		|| (CVarAndroidGLESFlipYMethod.GetValueOnAnyThread() == 0 && (GRHIVendorId == 0x13B5 || GRHIVendorId == 0x1010));
}

///
extern FCriticalSection GAndroidWindowLock;

void BlockOnLostWindowRenderCommand(TSharedPtr<FEvent, ESPMode::ThreadSafe> RTBlockedTrigger)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlockOnLostWindowRenderCommand);
	check(IsInRenderingThread());

	// Hold GC scope guard, as GC will timeout if anything waits for RT fences.
	FGCScopeGuard GCGuard;
	
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	UE_LOGF(LogAndroid, Log, "Blocking renderer");
	if (FAndroidMisc::ShouldUseVulkan())
	{
		if (IsRunningRHIInSeparateThread() && !RHICmdList.Bypass()) 
		{
			UE_LOGF(LogAndroid, Log, "RendererBlock FlushRHIThread");
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			UE_LOGF(LogAndroid, Log, "RendererBlock DONE FlushRHIThread");
		}
		
		const auto& OnReleaseWindowCallback = FAndroidMisc::GetOnReleaseWindowCallback();
		if (OnReleaseWindowCallback)
		{
			UE_LOGF(LogAndroid, Log, "RendererBlock release window callback");
			OnReleaseWindowCallback();
		}

		RTBlockedTrigger->Trigger();

		GAndroidWindowLock.Lock();
		UE_LOGF(LogAndroid, Log, "RendererBlock acquired window lock");
		const auto& OnReinitWindowCallback = FAndroidMisc::GetOnReInitWindowCallback();
		if (OnReinitWindowCallback)
		{
			OnReinitWindowCallback(FAndroidWindow::GetHardwareWindow_EventThread());
			UE_LOGF(LogAndroid, Log, "RendererBlock updating window");
		}
		GAndroidWindowLock.Unlock();
	}
	else
	{
		RHICmdList.EnqueueLambda([RTBlockedTrigger](FRHICommandListImmediate&)
		{
			RTBlockedTrigger->Trigger();
			GAndroidWindowLock.Lock();
			UE_LOGF(LogAndroid, Log, "RendererBlock acquired window lock");
			AndroidEGL::GetInstance()->SetRenderContextWindowSurface(TOptional<FAndroidWindow::FNativeAccessor>()); // FNativeAccessor is ignored with previous window behavior.
			UE_LOGF(LogAndroid, Log, "RendererBlock updating window");
			GAndroidWindowLock.Unlock();
		});
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	}
	UE_LOGF(LogAndroid, Log, "RendererBlock released window lock");
}

void SetSharedContextGameCommand(TSharedPtr<FEvent, ESPMode::ThreadSafe> GTBlockedTrigger)
{
	check(IsInGameThread());
	AndroidEGL* EGL = AndroidEGL::GetInstance();
	EGL->SetCurrentContext(EGL_NO_CONTEXT, nullptr);

	GTBlockedTrigger->Trigger();
}

extern bool IsInAndroidEventThread();
void BlockRendering()
{
	check(IsInAndroidEventThread());
	check(GIsRHIInitialized);

	UE_LOGF(LogAndroid, Log, "Blocking renderer on suspended window.");
	
	TSharedPtr<FEvent, ESPMode::ThreadSafe> BlockedTrigger = MakeShareable(FPlatformProcess::GetSynchEventFromPool(), [](FEvent* EventToDelete)
	{
		FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
	});

#if !USE_ANDROID_ALTERNATIVE_SUSPEND
	// Flush GT first in case it has any dependency on RT work to complete
	FGraphEventRef GTBlockTask = FFunctionGraphTask::CreateAndDispatchWhenReady([BlockedTrigger]()
	{
		SetSharedContextGameCommand(BlockedTrigger);
	}, TStatId(), NULL, ENamedThreads::GameThread);

	UE_LOGF(LogAndroid, Log, "Waiting for game thread to release EGL context/surface.");
	BlockedTrigger->Wait();
#endif

	// Wait for GC to complete and prevent further GCs
	FGCScopeGuard GCGuard;

	FGraphEventRef RTBlockTask = FFunctionGraphTask::CreateAndDispatchWhenReady([BlockedTrigger]()
	{
		BlockOnLostWindowRenderCommand(BlockedTrigger);
	}, TStatId(), NULL, ENamedThreads::GetRenderThread());

	// wait for the render thread to process.
	UE_LOGF(LogAndroid, Log, "Waiting for renderer to encounter blocking command.");
	BlockedTrigger->Wait();
}

#endif
