// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidEGL.h: Private EGL definitions for Android-specific functionality
=============================================================================*/
#pragma once

#include "Android/AndroidPlatform.h"

#if USE_ANDROID_OPENGL

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include <EGL/egl.h>
#include <EGL/eglext.h>
#include <GLES3/gl31.h>
#include "Android/AndroidWindow.h"
#include <atomic>

struct AndroidESPImpl;
struct ANativeWindow;

#ifndef USE_ANDROID_EGL_NO_ERROR_CONTEXT
#if UE_BUILD_SHIPPING
#define USE_ANDROID_EGL_NO_ERROR_CONTEXT 1
#else
#define USE_ANDROID_EGL_NO_ERROR_CONTEXT 0
#endif
#endif // USE_ANDROID_EGL_NO_ERROR_CONTEXT

DECLARE_LOG_CATEGORY_EXTERN(LogEGL, Log, All);


struct EGLConfigParms
{
	/** Whether this is a valid configuration or not */
	int validConfig = 0;
	/** The number of bits requested for the red component */
	int redSize = 8;
	/** The number of bits requested for the green component */
	int greenSize = 8;
	/** The number of bits requested for the blue component */
	int blueSize = 8;
	/** The number of bits requested for the alpha component */
	int alphaSize = 0;
	/** The number of bits requested for the depth component */
	int depthSize = 24;
	/** The number of bits requested for the stencil component */
	int stencilSize = 0;
	/** The number of multisample buffers requested */
	int sampleBuffers = 0;
	/** The number of samples requested */
	int sampleSamples = 0;

	EGLConfigParms()
	{
		// If not default, set the preference
		int DepthBufferPreference = (int)FAndroidWindow::GetDepthBufferPreference();
		if (DepthBufferPreference > 0)
		{
			depthSize = DepthBufferPreference;
		}

		if (FAndroidMisc::GetMobilePropagateAlphaSetting() > 0)
		{
			alphaSize = 8;
		}
	}
};

struct FAndroidEGLDisplayInfo
{
	EGLDisplay  eglDisplay = EGL_NO_DISPLAY;
	EGLConfig	eglConfigParam = nullptr;

	EGLConfigParms Parms;
	EGLint         eglNumConfigs = 0;

	EGLint		NativeVisualID = 0;
	int			DepthSize = 0;

	// this is an offscreen temp surface used when no HW window is supplied.
	EGLSurface  auxSurface = EGL_NO_SURFACE;
};

struct FPlatformOpenGLContext
{
	FPlatformOpenGLContext()
	{
		Reset();
	}

	void Reset()
	{
		eglContext = EGL_NO_CONTEXT;
		DefaultVertexArrayObject = 0;
		bIsDebug = false;
	}

	GLuint					DefaultVertexArrayObject;
	bool					bIsDebug;
	EGLContext				eglContext;
};

struct FPlatformOpenGLSurfaceContext
{
public:
	FPlatformOpenGLSurfaceContext(void* InWindowHandle);
	FPlatformOpenGLSurfaceContext(TOptional<FAndroidWindow::FNativeAccessor> WindowContainerIn);

	GLuint		ViewportFramebuffer;

	GLuint		BackBufferResource;
	GLenum		BackBufferTarget;
	FPlatformRect  CachedWindowRect{};

	bool TargetDirty = false;

	// EGLProperties
	EGLint         eglFormat = -1;
	EGLSurface     eglSurface = EGL_NO_SURFACE;
	EGLint         eglWidth = 8; // required for Gear VR apps with internal win surf mgmt
	EGLint         eglHeight = 8; // required for Gear VR apps with internal win surf mgmt
	float          eglRatio = 0;
	ANativeWindow* Window = nullptr;
	bool           bIsWndSurface = false; // True when the surface is attached to a HW window.

	EGLSurface GetSurface() const { return eglSurface; }

private:
	// android window properties:
	FAndroidWindow* AndroidWindow = nullptr;

public:
	FAndroidWindow* GetAndroidWindow() { return AndroidWindow; }

	bool IsUsingWindowedSurface() const { return bIsWndSurface; }

	FPlatformOpenGLSurfaceContext()
	{
		Reset();
	}

	void Reset()
	{
		ViewportFramebuffer = 0;
		BackBufferResource = 0;
		BackBufferTarget = 0;
		CachedWindowRect = FPlatformRect();
		TargetDirty = false;
		eglFormat = -1;
		eglSurface = EGL_NO_SURFACE;
		eglWidth = 8; // required for Gear VR apps with internal win surf mgmt
		eglHeight = 8; // required for Gear VR apps with internal win surf mgmt
		eglRatio = 0;
		Window = nullptr;
		AndroidWindow = nullptr;
		bIsWndSurface = false; // True when the surface is attached to a HW window.
	}
};

class AndroidEGL
{
public:
	enum APIVariant
	{
		AV_OpenGLES,
		AV_OpenGLCore
	};

	static AndroidEGL* GetInstance();
	~AndroidEGL();	

	bool IsInitialized();

	void InitBackBuffer(struct FPlatformOpenGLSurfaceContext* Surface);
	void DestroyBackBuffer(struct FPlatformOpenGLSurfaceContext* Surface);

	void Init(APIVariant API, uint32 MajorVersion, uint32 MinorVersion);

	void UnBindRender(struct FPlatformOpenGLSurfaceContext* Surface);
	void OnLostNativeWindow(FPlatformOpenGLSurfaceContext* Surface);

	void Terminate();

	// Initialize the surface of the context.
	// when using the new window locking behavior, if WindowContainer is not set an off screen surface will be used.
	// TODO: remove bCreateWndSurface (and possibly bUseSmallSurface), these can be inferred from WindowContainer.
	void InitRenderSurface(bool bUseSmallSurface, bool bCreateWndSurface, FPlatformOpenGLSurfaceContext* Surface);
	void UpdateBuffersTransform();
	bool IsOfflineSurfaceRequired();

	bool IsUsingRobustContext() const { return bIsEXTRobustContextActive; }

	EGLDisplay GetDisplay() const;
	EGLSurface GetSurface() const;
	EGLConfig GetConfig() const;
	ANativeWindow* GetNativeWindow() const;
	void GetSwapIntervalRange(EGLint& OutMinSwapInterval, EGLint& OutMaxSwapInterval) const;

	EGLContext CreateContext(EGLContext InParentContext);
	int32 GetError();
	EGLBoolean SetCurrentContext(EGLContext InContext, FPlatformOpenGLSurfaceContext* InSurfaceInfo);

	void AcquireCurrentRenderingContext();
	void ReleaseContextOwnership();

	// Methods to control rendering blocking during window recreation
	void SetWaitingForWindowCreated(bool bWaiting) { bWaitingForWindowCreated.store(bWaiting); }
	bool IsWaitingForWindowCreated() const { return bWaitingForWindowCreated.load(); }

	EGLContext GetCurrentContext();
	EGLSurface GetCurrentSurface(EGLint readdraw);


	void SetCurrentRenderingContext(FPlatformOpenGLSurfaceContext* InSurface);
	bool ThreadHasRenderingContext();
	FPlatformOpenGLContext* GetRenderingContext();

	bool GetSupportsNoErrorContext();

	// recreate the EGL surface for the current hardware window.
	void SetRenderContextWindowSurface(const TOptional<FAndroidWindow::FNativeAccessor>& WindowHandle);

	// Called from game thread when a window is reinited.
	void RefreshWindowSize(const TOptional<FAndroidWindow::FNativeAccessor>& WindowContainer);

	// true if the current surface is associated with a window.
	bool IsUsingWindowedSurface() const;

	FPlatformOpenGLSurfaceContext* CreateSurfaceContextFromWindow(FAndroidWindow* InWindow);
	FPlatformOpenGLSurfaceContext* FindSurfaceContextFromWindow(FAndroidWindow* InWindow);
	void DestroySurfaceContext(FPlatformOpenGLSurfaceContext* SurfaceContext);

protected:
	AndroidEGL();
	static AndroidEGL* Singleton;

private:
	void InitEGL(APIVariant API);
	void TerminateEGL();

	void CreateEGLRenderSurface(FPlatformOpenGLSurfaceContext* Surface, bool bCreateWndSurface);
	void DestroyRenderSurface(FPlatformOpenGLSurfaceContext* Surface);

	bool InitContexts();
	void DestroyContext(EGLContext InContext);

	void ResetDisplay();

	AndroidESPImpl* PImplData;

	void ResetInternal();
	void LogConfigInfo(EGLConfig  EGLConfigInfo);

	// Actual Update to the egl surface to match the GT's requested size.
	void ResizeRenderContextSurface(FPlatformOpenGLSurfaceContext* Surface);

	bool bSupportsKHRCreateContext      = false;
	bool bSupportsKHRSurfacelessContext = false;
	bool bSupportsKHRNoErrorContext     = false;
	bool bSupportsEXTRobustContext      = false;
	bool bIsEXTRobustContextActive      = false;

	// Flag to block rendering while waiting for APP_EVENT_STATE_WINDOW_CREATED to be processed
	std::atomic<bool> bWaitingForWindowCreated{false};

	int* ContextAttributes = nullptr;
};

#define ENABLE_CONFIG_FILTER 1
#define ENABLE_EGL_DEBUG 0
#define ENABLE_VERIFY_EGL 0
#define ENABLE_VERIFY_EGL_TRACE 0

#if ENABLE_VERIFY_EGL

#define VERIFY_EGL(msg) { VerifyEGLResult(eglGetError(),TEXT(#msg),TEXT(""),TEXT(__FILE__),__LINE__); }

void VerifyEGLResult(EGLint ErrorCode, const TCHAR* Msg1, const TCHAR* Msg2, const TCHAR* Filename, uint32 Line)
{
	if (ErrorCode != EGL_SUCCESS)
	{
		static const TCHAR* EGLErrorStrings[] =
		{
			TEXT("EGL_NOT_INITIALIZED"),
			TEXT("EGL_BAD_ACCESS"),
			TEXT("EGL_BAD_ALLOC"),
			TEXT("EGL_BAD_ATTRIBUTE"),
			TEXT("EGL_BAD_CONFIG"),
			TEXT("EGL_BAD_CONTEXT"),
			TEXT("EGL_BAD_CURRENT_SURFACE"),
			TEXT("EGL_BAD_DISPLAY"),
			TEXT("EGL_BAD_MATCH"),
			TEXT("EGL_BAD_NATIVE_PIXMAP"),
			TEXT("EGL_BAD_NATIVE_WINDOW"),
			TEXT("EGL_BAD_PARAMETER"),
			TEXT("EGL_BAD_SURFACE"),
			TEXT("EGL_CONTEXT_LOST"),
			TEXT("UNKNOWN EGL ERROR")
		};

		uint32 ErrorIndex = FMath::Min<uint32>(ErrorCode - EGL_SUCCESS, UE_ARRAY_COUNT(EGLErrorStrings) - 1);
		UE_LOGF(LogRHI, Warning, "%ls(%u): %ls%ls failed with error %ls (0x%x)",
			Filename, Line, Msg1, Msg2, EGLErrorStrings[ErrorIndex], ErrorCode);
		check(0);
	}
}

class FEGLErrorScope
{
public:
	FEGLErrorScope(
		const TCHAR* InFunctionName,
		const TCHAR* InFilename,
		const uint32 InLine)
		: FunctionName(InFunctionName)
		, Filename(InFilename)
		, Line(InLine)
	{
#if ENABLE_VERIFY_EGL_TRACE
		UE_LOGF(LogRHI, Log, "EGL log before %ls(%d): %ls", InFilename, InLine, InFunctionName);
#endif
		CheckForErrors(TEXT("Before "));
	}

	~FEGLErrorScope()
	{
#if ENABLE_VERIFY_EGL_TRACE
		UE_LOGF(LogRHI, Log, "EGL log after  %ls(%d): %ls", Filename, Line, FunctionName);
#endif
		CheckForErrors(TEXT("After "));
	}

private:
	const TCHAR* FunctionName;
	const TCHAR* Filename;
	const uint32 Line;

	void CheckForErrors(const TCHAR* PrefixString)
	{
		VerifyEGLResult(eglGetError(), PrefixString, FunctionName, Filename, Line);
	}
};

#define MACRO_TOKENIZER(IdentifierName, Msg, FileName, LineNumber) FEGLErrorScope IdentifierName_ ## LineNumber (Msg, FileName, LineNumber)
#define MACRO_TOKENIZER2(IdentifierName, Msg, FileName, LineNumber) MACRO_TOKENIZER(IdentiferName, Msg, FileName, LineNumber)
#define VERIFY_EGL_SCOPE_WITH_MSG_STR(MsgStr) MACRO_TOKENIZER2(ErrorScope_, MsgStr, TEXT(__FILE__), __LINE__)
#define VERIFY_EGL_SCOPE() VERIFY_EGL_SCOPE_WITH_MSG_STR(ANSI_TO_TCHAR(__FUNCTION__))
#define VERIFY_EGL_FUNC(Func, ...) { VERIFY_EGL_SCOPE_WITH_MSG_STR(TEXT(#Func)); Func(__VA_ARGS__); }
#else
#define VERIFY_EGL(...)
#define VERIFY_EGL_SCOPE(...)
#endif



#endif
