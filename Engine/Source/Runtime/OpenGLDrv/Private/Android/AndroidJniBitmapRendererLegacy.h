// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni
{
	struct FBitmapRendererLegacy: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/BitmapRendererLegacy";

		static void JNICALL nativeClearCachedAttributeState(JNIEnv* jenv, jobject thiz, jint PositionAttrib, jint TexCoordsAttrib);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeClearCachedAttributeState)
		};
	};
	
	template struct TInitialize<FBitmapRendererLegacy>;
}
#endif
