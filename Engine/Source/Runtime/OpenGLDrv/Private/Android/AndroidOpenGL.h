// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidOpenGL.h: Public OpenGL ES definitions for Android-specific functionality
=============================================================================*/
#pragma once

#include "OpenGLThirdParty.h"
#include "AndroidOpenGLPlatform.h"

#include "AndroidEGL.h"
#include "OpenGLES.h"

struct FAndroidOpenGL : public FOpenGLES
{
	static FORCEINLINE bool HasHardwareHiddenSurfaceRemoval() { return bHasHardwareHiddenSurfaceRemoval; }

	static bool SupportsFramebufferSRGBEnable();

	static FORCEINLINE void DeleteSync(UGLsync Sync)
	{
		if (GUseThreadedRendering)
		{
			glDeleteSync( Sync );
		}
	}

	static FORCEINLINE UGLsync FenceSync(GLenum Condition, GLbitfield Flags)
	{
		return GUseThreadedRendering ? glFenceSync( Condition, Flags ) : 0;
	}

	static FORCEINLINE bool IsSync(UGLsync Sync)
	{
		if (GUseThreadedRendering)
		{
			return (glIsSync( Sync ) == GL_TRUE) ? true : false;
		}
		return true;
	}

	static FORCEINLINE EFenceResult ClientWaitSync(UGLsync Sync, GLbitfield Flags, GLuint64 Timeout)
	{
		if (GUseThreadedRendering)
		{
			GLenum Result = glClientWaitSync( Sync, Flags, Timeout );
			switch (Result)
			{
				case GL_ALREADY_SIGNALED:		return FR_AlreadySignaled;
				case GL_TIMEOUT_EXPIRED:		return FR_TimeoutExpired;
				case GL_CONDITION_SATISFIED:	return FR_ConditionSatisfied;
			}
			return FR_WaitFailed;
		}
		return FR_ConditionSatisfied;
	}
	
	// Disable all queries except occlusion
	// Query is a limited resource on Android and we better spent them all on occlusion
	static FORCEINLINE bool SupportsTimestampQueries()					{ return false; }
	static FORCEINLINE bool SupportsDisjointTimeQueries()				{ return false; }

	enum class EImageExternalType : uint8
	{
		None,
		ImageExternal100,
		ImageExternal300,
		ImageExternalESSL300
	};

	static FORCEINLINE bool SupportsImageExternal() { return bSupportsImageExternal; }

	static FORCEINLINE EImageExternalType GetImageExternalType() { return ImageExternalType; }

	static FORCEINLINE GLint GetMaxComputeUniformComponents() { check(MaxComputeUniformComponents != -1); return MaxComputeUniformComponents; }
	static FORCEINLINE GLint GetFirstComputeUAVUnit()			{ return 0; }
	static FORCEINLINE GLint GetMaxComputeUAVUnits()			{ check(MaxComputeUAVUnits != -1); return MaxComputeUAVUnits; }
	static FORCEINLINE GLint GetFirstVertexUAVUnit()			{ return 0; }
	static FORCEINLINE GLint GetFirstPixelUAVUnit()				{ return 0; }
	static FORCEINLINE GLint GetMaxPixelUAVUnits()				{ check(MaxPixelUAVUnits != -1); return MaxPixelUAVUnits; }
	static FORCEINLINE GLint GetMaxCombinedUAVUnits()			{ return MaxCombinedUAVUnits; }
		
	static FORCEINLINE void FrameBufferFetchBarrier()
	{
		if (glFramebufferFetchBarrierQCOM)
		{
			glFramebufferFetchBarrierQCOM();
		}
	}
		
	static void ProcessExtensions(const FString& ExtensionsString);
	static void SetupDefaultGLContextState(const FString& ExtensionsString);

	static bool RequiresAdrenoTilingModeHint();
	static void EnableAdrenoTilingModeHint(bool bEnable);
	static bool bRequiresAdrenoTilingHint;

	static bool ResetNonCoherentFramebufferFetch();
	static void DisableNonCoherentFramebufferFetch();
	static bool bDefaultStateNonCoherentFramebufferFetchEnabled;

	/** supported OpenGL ES version queried from the system */
	static int32 GLMajorVerion;
	static int32 GLMinorVersion;

	static FORCEINLINE GLuint GetMajorVersion()
	{
		return GLMajorVerion;
	}

	static FORCEINLINE GLuint GetMinorVersion()
	{
		return GLMinorVersion;
	}

	/** Whether device supports image external */
	static bool bSupportsImageExternal;

	/** Type of image external supported */
	static EImageExternalType ImageExternalType;

	/* interface to remote GLES program compiler */
	static TArray<uint8> DispatchAndWaitForRemoteGLProgramCompile(FGraphicsPipelineStateInitializer::EPSOPrecacheCompileType PSOCompileType,const TArrayView<uint8> ContextData, const TArray<ANSICHAR>& VertexGlslCode, const TArray<ANSICHAR>& PixelGlslCode, const TArray<ANSICHAR>& ComputeGlslCode, FString& FailureMessageOUT);
	static bool AreRemoteCompileServicesActive();
	static bool StartRemoteCompileServices(int NumServices);
	static void StopRemoteCompileServices();
};

using FOpenGL = FAndroidOpenGL;
