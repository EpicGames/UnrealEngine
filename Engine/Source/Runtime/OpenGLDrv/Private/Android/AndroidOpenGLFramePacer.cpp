// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidOpenGLFramePacer.h"

#if USE_ANDROID_OPENGL

/*******************************************************************
 * FAndroidOpenGLFramePacer implementation
 *******************************************************************/

#include "OpenGLDrvPrivate.h"
#include "Math/UnrealMathUtility.h"
#if USE_ANDROID_OPENGL_SWAPPY
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "swappy/swappyGL.h"
#include "swappy/swappyGL_extra.h"
#include "swappy/swappy_common.h"
#include "HAL/Thread.h"
#include "Misc/ScopeRWLock.h"
#include "EngineGlobals.h"

struct FSwappyThreadManager : public SwappyThreadFunctions
{
	static FRWLock SwappyThreadManagerMutex;
	static TMap<uint64, TUniquePtr<FThread>> Threads;
	FSwappyThreadManager()
	{
		start = [](SwappyThreadId* thread_id, void* (*thread_func)(void*),
			void* user_data)
		{
			FRWScopeLock Lock(SwappyThreadManagerMutex, SLT_Write);
			static int ThreadCount = 0;
			TUniquePtr<FThread> NewThread = MakeUnique<FThread>(*FString::Printf(TEXT("SwappyThread%d"), ThreadCount++), [thread_func, user_data]() {thread_func(user_data); });
			if (NewThread->GetThreadId())
			{
				*thread_id = NewThread->GetThreadId();
				Threads.Add(*thread_id, MoveTemp(NewThread));
				return 0;
			}			
			return -1;
		};

		join = [](SwappyThreadId thread_id)
		{
			FRWScopeLock Lock(SwappyThreadManagerMutex, SLT_Write);
			TUniquePtr<FThread> ThreadPtr;
			if(ensure(Threads.RemoveAndCopyValue(thread_id, ThreadPtr)))
			{
				ThreadPtr->Join();
			}
		};

		joinable = [](SwappyThreadId thread_id)
		{
			FRWScopeLock Lock(SwappyThreadManagerMutex, SLT_ReadOnly);
			return Threads[thread_id]->IsJoinable();
		};
	}	
}SwappyThreads;
TMap<uint64, TUniquePtr<FThread>> FSwappyThreadManager::Threads;
FRWLock FSwappyThreadManager::SwappyThreadManagerMutex;

#endif

#include "AndroidEGL.h"
#include <EGL/egl.h>

void FAndroidOpenGLFramePacer::Init()
{
	bSwappyInit = false;
#if USE_ANDROID_OPENGL_SWAPPY
	if (FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.GetValueOnAnyThread() == 1)
	{
		// initialize now if set on startup
		InitSwappy();
	}
	else
	{
		// initialize later if set by console
		FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Variable)
		{
			if (Variable->GetInt() == 1)
			{
				InitSwappy();
			}
		}));
	}
#endif
}

#if USE_ANDROID_OPENGL_SWAPPY

namespace AndroidGL
{
	void SwappyPostWaitCallback(void*, int64_t cpu_time_ns, int64_t gpu_time_ns)
	{
		const double GPUTimeInSeconds = (double)gpu_time_ns / 1000000000.0;

		uint64 Cycles = GPUTimeInSeconds / FPlatformTime::GetSecondsPerCycle64();

		GetDynamicRHI<FOpenGLDynamicRHI>()->RHISetExternalGPUTime(Cycles);
	}

	void SetSwappyPostWaitCallback()
	{
		SwappyTracer Tracer = { 0 };
		Tracer.postWait = AndroidGL::SwappyPostWaitCallback;
		SwappyGL_injectTracer(&Tracer);

		int32 FrameTimeFenceInMillis = FAndroidPlatformRHIFramePacer::CVarSwappyGPUFrameTimeFence.GetValueOnAnyThread();

		SwappyGL_setFenceTimeoutNS(FrameTimeFenceInMillis * 1000000); // millis to ns (ms * 1000000)
	}
};

void FAndroidOpenGLFramePacer::InitSwappy()
{

	if (!bSwappyInit)
	{
		extern void LoadSwappy();
		LoadSwappy();

		if( !FParse::Param(FCommandLine::Get(), TEXT("UseSwappyThreads")) )
		{
			Swappy_setThreadFunctions(&SwappyThreads);
		}
		// initialize Swappy
		JNIEnv* Env = FAndroidApplication::GetJavaEnv();
		if (ensure(Env))
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Init Swappy: version %d"), Swappy_version());
			SwappyGL_init(Env, FJavaWrapper::GameActivityThis);

			AndroidGL::SetSwappyPostWaitCallback();
		}
		bSwappyInit = true;
	}
}
#endif

FAndroidOpenGLFramePacer::~FAndroidOpenGLFramePacer()
{
#if USE_ANDROID_OPENGL_SWAPPY
	FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate());
	if (bSwappyInit)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Shutdown Swappy"));
		SwappyGL_destroy();
		bSwappyInit = false;
	}
#endif
}

static bool GGetTimeStampsSucceededThisFrame = true;
static uint32 GGetTimeStampsRetryCount = 0;

static bool CanUseGetFrameTimestamps()
{
	return FAndroidPlatformRHIFramePacer::CVarUseGetFrameTimestamps.GetValueOnAnyThread()
		&& eglGetFrameTimestampsANDROID_p
		&& eglGetNextFrameIdANDROID_p
		&& eglPresentationTimeANDROID_p
		&& (GGetTimeStampsRetryCount < FAndroidPlatformRHIFramePacer::CVarTimeStampErrorRetryCount.GetValueOnAnyThread());
}
static bool CanUseGetFrameTimestampsForThisFrame()
{
	return CanUseGetFrameTimestamps() && GGetTimeStampsSucceededThisFrame;
}

bool ShouldUseGPUFencesToLimitLatency()
{
	if (CanUseGetFrameTimestampsForThisFrame())
	{
		return true; // this method requires a GPU fence to give steady results
	}
	return FAndroidPlatformRHIFramePacer::CVarDisableOpenGLGPUSync.GetValueOnAnyThread() == 0; // otherwise just based on the FAndroidPlatformRHIFramePacer::CVar; thought to be bad to use GPU fences on PowerVR
}

static uint32 NextFrameIDSlot = 1;
#define NUM_FRAMES_TO_MONITOR (4)
static EGLuint64KHR FrameIDs[NUM_FRAMES_TO_MONITOR] = { 0 };

static int32 RecordedFrameInterval[100];
static int32 NumRecordedFrameInterval = 0;

extern float AndroidThunkCpp_GetMetaDataFloat(const FString& Key);

bool FAndroidOpenGLFramePacer::SupportsFramePaceInternal(int32 QueryFramePace, int32& OutRefreshRate, int32& OutSyncInterval)
{
#if USE_ANDROID_OPENGL_SWAPPY
	if (FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.GetValueOnAnyThread() == 1)
	{
		TArray<int32> RefreshRates = FAndroidMisc::GetSupportedNativeDisplayRefreshRates();
		RefreshRates.Sort();
		FString RefreshRatesString;
		for (int32 Rate : RefreshRates)
		{
			RefreshRatesString += FString::Printf(TEXT(" %d"), Rate);
		}
		UE_LOGF(LogRHI, Log, "FAndroidOpenGLFramePacer -> Supported Refresh Rates:%ls", *RefreshRatesString);

		for (int32 Rate : RefreshRates)
		{
			if ((Rate % QueryFramePace) == 0)
			{
				UE_LOGF(LogRHI, Log, "Supports %d using refresh rate %d and sync interval %d", QueryFramePace, Rate, Rate / QueryFramePace);
				OutRefreshRate = Rate;
				OutSyncInterval = Rate / QueryFramePace;
				return true;
			}
		}

		// check if we want to use naive frame pacing at less than a multiple of supported refresh rate
		if (FAndroidPlatformRHIFramePacer::CVarSupportNonVSyncMultipleFrameRates.GetValueOnAnyThread() == 1)
		{
			for (int32 Rate : RefreshRates)
			{
				if (Rate > QueryFramePace)
				{
					UE_LOGF(LogRHI, Log, "Supports %d using refresh rate %d with naive frame pacing", QueryFramePace, Rate);
					OutRefreshRate = Rate;
					OutSyncInterval = 0;
					return true;
				}
			}
		}
	}
#endif
	OutRefreshRate = QueryFramePace;
	OutSyncInterval = 0;
	return FGenericPlatformRHIFramePacer::SupportsFramePace(QueryFramePace);
}

bool FAndroidOpenGLFramePacer::SupportsFramePace(int32 QueryFramePace)
{
	int32 TempRefreshRate, TempSyncInterval;
	return SupportsFramePaceInternal(QueryFramePace, TempRefreshRate, TempSyncInterval);
}

extern bool AndroidThunkCpp_IsXRImmersiveHMDApplication();
extern bool AndroidThunkCpp_IsStandaloneApplication();

bool FAndroidOpenGLFramePacer::SwapBuffers(FPlatformOpenGLSurfaceContext* SurfaceContext, bool bLockToVsync)
{
	SCOPED_NAMED_EVENT(STAT_OpenGLSwapBuffersTime, FColor::Red)

#if !UE_BUILD_SHIPPING
	if (FAndroidPlatformRHIFramePacer::CVarStallSwap.GetValueOnAnyThread() > 0.0f)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Swap_Intentional_Stall);
		FPlatformProcess::Sleep(FAndroidPlatformRHIFramePacer::CVarStallSwap.GetValueOnRenderThread() / 1000.0f);
	}
#endif

	VERIFY_EGL_SCOPE();

	AndroidEGL* AnEGL = AndroidEGL::GetInstance();
	EGLDisplay eglDisplay = AnEGL->GetDisplay();
	EGLSurface eglSurface = SurfaceContext->eglSurface;
	int32 SyncInterval = FAndroidPlatformRHIFramePacer::GetLegacySyncInterval();

	if (SurfaceContext->IsUsingWindowedSurface() == false && !AndroidThunkCpp_IsXRImmersiveHMDApplication() && !AndroidThunkCpp_IsStandaloneApplication())
	{
		// offscreen surface, no swap to perform, limit update rate.
		FPlatformProcess::Sleep(0.1);
		return true;
	}

	bool bPrintMethod = false;

#if USE_ANDROID_OPENGL_SWAPPY
	int32 CurrentFramePace = 0;
	if (FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing.GetValueOnRenderThread() != 0 && (CurrentFramePace = FAndroidPlatformRHIFramePacer::GetFramePace()) != 0 && ensure(bSwappyInit))
	{
		ANativeWindow* CurrentNativeWindow = SurfaceContext->Window;
		if (CurrentNativeWindow != CachedNativeWindow)
		{
			CachedNativeWindow = CurrentNativeWindow;
			UE_LOGF(LogRHI, Verbose, "Swappy - setting native window %p", CachedNativeWindow);
			SwappyGL_setWindow(CurrentNativeWindow);
		}
		
		// cache refresh rate and sync interval
		if (CurrentFramePace != CachedFramePace)
		{
			CachedFramePace = CurrentFramePace;
			SupportsFramePaceInternal(CurrentFramePace, CachedRefreshRate, CachedSyncInterval);
			SwappyGL_resetFramePacing();
		}

		SwappyGL_setAutoSwapInterval(false);
		if (CachedSyncInterval != 0)
		{
			// Multiple of sync interval, use swappy directly
			UE_LOG(LogRHI, Verbose, TEXT("Setting swappy to interval of %" PRId64 " (%d fps)"), (int64)(1000000000L) / (int64)CurrentFramePace, CurrentFramePace);
			SwappyGL_setSwapIntervalNS((1000000000L) / (int64)CurrentFramePace);
		}
		else
		{
			// Unsupported frame rate. Set to higher refresh rate
			SwappyGL_setSwapIntervalNS((1000000000L) / (int64)CachedRefreshRate);

			// use naive frame pacing to limit the frame rate
			float MinTimeBetweenFrames = (1.f / CurrentFramePace);
			float ThisTime = FPlatformTime::Seconds() - LastTimeEmulatedSync;
			if (ThisTime > 0 && ThisTime < MinTimeBetweenFrames)
			{
				FPlatformProcess::Sleep(MinTimeBetweenFrames - ThisTime);
			}
		}

		SwappyGL_swap(eglDisplay, eglSurface);
		LastTimeEmulatedSync = FPlatformTime::Seconds();
	}
	else
#endif
	{
		if (DesiredSyncIntervalRelativeTo60Hz != SyncInterval)
		{
			GGetTimeStampsRetryCount = 0;

			bPrintMethod = true;
			DesiredSyncIntervalRelativeTo60Hz = SyncInterval;
			DriverRefreshRate = 60.0f;
			DriverRefreshNanos = 16666666;


			EGLnsecsANDROID EGL_COMPOSITE_DEADLINE_ANDROID_Value = -1;
			EGLnsecsANDROID EGL_COMPOSITE_INTERVAL_ANDROID_Value = -1;
			EGLnsecsANDROID EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID_Value = -1;

			if (eglGetCompositorTimingANDROID_p)
			{
				{
					EGLint Item = EGL_COMPOSITE_DEADLINE_ANDROID;
					if (!eglGetCompositorTimingANDROID_p(eglDisplay, eglSurface, 1, &Item, &EGL_COMPOSITE_DEADLINE_ANDROID_Value))
					{
						EGL_COMPOSITE_DEADLINE_ANDROID_Value = -1;
					}
				}
				{
					EGLint Item = EGL_COMPOSITE_INTERVAL_ANDROID;
					if (!eglGetCompositorTimingANDROID_p(eglDisplay, eglSurface, 1, &Item, &EGL_COMPOSITE_INTERVAL_ANDROID_Value))
					{
						EGL_COMPOSITE_INTERVAL_ANDROID_Value = -1;
					}
				}
				{
					EGLint Item = EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID;
					if (!eglGetCompositorTimingANDROID_p(eglDisplay, eglSurface, 1, &Item, &EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID_Value))
					{
						EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID_Value = -1;
					}
				}
				UE_LOGF(LogRHI, Log, "AndroidEGL:SwapBuffers eglGetCompositorTimingANDROID EGL_COMPOSITE_DEADLINE_ANDROID=%lld, EGL_COMPOSITE_INTERVAL_ANDROID=%lld, EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID=%lld",
					EGL_COMPOSITE_DEADLINE_ANDROID_Value,
					EGL_COMPOSITE_INTERVAL_ANDROID_Value,
					EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID_Value
				);
			}

			float RefreshRate = AndroidThunkCpp_GetMetaDataFloat(TEXT("unreal.display.getRefreshRate"));

			UE_LOGF(LogRHI, Log, "JNI Display getRefreshRate=%f",
				RefreshRate
			);

			if (EGL_COMPOSITE_INTERVAL_ANDROID_Value >= 4000000 && EGL_COMPOSITE_INTERVAL_ANDROID_Value <= 41666666)
			{
				DriverRefreshRate = float(1000000000.0 / double(EGL_COMPOSITE_INTERVAL_ANDROID_Value));
				DriverRefreshNanos = EGL_COMPOSITE_INTERVAL_ANDROID_Value;
			}
			else if (RefreshRate >= 24.0f && RefreshRate <= 250.0f)
			{
				DriverRefreshRate = RefreshRate;
				DriverRefreshNanos = int64(0.5 + 1000000000.0 / double(RefreshRate));
			}

			UE_LOGF(LogRHI, Log, "Final display timing metrics: DriverRefreshRate=%7.4f  DriverRefreshNanos=%lld",
				DriverRefreshRate,
				DriverRefreshNanos
			);

			// make sure requested interval is in supported range
			EGLint MinSwapInterval, MaxSwapInterval;
			AnEGL->GetSwapIntervalRange(MinSwapInterval, MaxSwapInterval);

			int64 SyncIntervalNanos = (30 + 1000000000l * int64(SyncInterval)) / 60;

			int32 UnderDriverInterval = int32(SyncIntervalNanos / DriverRefreshNanos);
			int32 OverDriverInterval = UnderDriverInterval + 1;

			int64 UnderNanos = int64(UnderDriverInterval) * DriverRefreshNanos;
			int64 OverNanos = int64(OverDriverInterval) * DriverRefreshNanos;

			DesiredSyncIntervalRelativeToDevice = (FMath::Abs(SyncIntervalNanos - UnderNanos) < FMath::Abs(SyncIntervalNanos - OverNanos)) ?
				UnderDriverInterval : OverDriverInterval;

			int32 DesiredDriverSyncInterval = FMath::Clamp<int32>(DesiredSyncIntervalRelativeToDevice, MinSwapInterval, MaxSwapInterval);

			UE_LOGF(LogRHI, Log, "AndroidEGL:SwapBuffers Min=%d, Max=%d, Request=%d, ClosestDriver=%d, SetDriver=%d", MinSwapInterval, MaxSwapInterval, DesiredSyncIntervalRelativeTo60Hz, DesiredSyncIntervalRelativeToDevice, DesiredDriverSyncInterval);

			if (DesiredDriverSyncInterval != DriverSyncIntervalRelativeToDevice)
			{
				DriverSyncIntervalRelativeToDevice = DesiredDriverSyncInterval;
				UE_LOGF(LogRHI, Log, "Called eglSwapInterval %d", DesiredDriverSyncInterval);
				eglSwapInterval(eglDisplay, DriverSyncIntervalRelativeToDevice);
			}
		}

		if (DesiredSyncIntervalRelativeToDevice > DriverSyncIntervalRelativeToDevice)
		{
			{
				UE_CLOGF(bPrintMethod, LogRHI, Display, "Using niave method for frame pacing (possible with timestamps method)");
				if (LastTimeEmulatedSync > 0.0)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_StallForEmulatedSyncInterval);
					float MinTimeBetweenFrames = (float(DesiredSyncIntervalRelativeToDevice) / DriverRefreshRate);

					for (;;)
					{
						float ThisTime = FPlatformTime::Seconds() - LastTimeEmulatedSync;
						// sleep only when there is substantial time left to a next sync interval
						// for a small duration rely on eglSwapBuffers
						if (ThisTime > 0.001f && ThisTime < MinTimeBetweenFrames)
						{
							// do not sleep for too long, poll occlussion queries from time to time as RT might be waiting for them
							float SleepDuration = FMath::Min(MinTimeBetweenFrames - ThisTime, 0.003f);
							FPlatformProcess::Sleep(SleepDuration);
						}
						else
						{
							break;
						}
					}
				}
			}
		}
		if (CanUseGetFrameTimestamps())
		{
			UE_CLOGF(bPrintMethod, LogRHI, Display, "Using eglGetFrameTimestampsANDROID method for frame pacing");

			//static bool bPrintOnce = true;
			if (FrameIDs[(int32(NextFrameIDSlot) - 1) % NUM_FRAMES_TO_MONITOR])
				// not supported   && eglGetFrameTimestampsSupportedANDROID_p && eglGetFrameTimestampsSupportedANDROID_p(eglDisplay, eglSurface, EGL_FIRST_COMPOSITION_START_TIME_ANDROID))
			{
				//UE_CLOGF(bPrintOnce, LogRHI, Log, "eglGetFrameTimestampsSupportedANDROID retured true for EGL_FIRST_COMPOSITION_START_TIME_ANDROID");
				EGLint TimestampList = EGL_FIRST_COMPOSITION_START_TIME_ANDROID;
				//EGLint TimestampList = EGL_COMPOSITION_LATCH_TIME_ANDROID;
				//EGLint TimestampList = EGL_LAST_COMPOSITION_START_TIME_ANDROID;
				//EGLint TimestampList = EGL_DISPLAY_PRESENT_TIME_ANDROID;
				EGLnsecsANDROID Result = 0;
				int32 DeltaFrameIndex = 1;
				for (int32 Index = int32(NextFrameIDSlot) - 1; Index >= int32(NextFrameIDSlot) - NUM_FRAMES_TO_MONITOR && Index >= 0; Index--)
				{
					Result = 0;
					if (FrameIDs[Index % NUM_FRAMES_TO_MONITOR])
					{
						eglGetFrameTimestampsANDROID_p(eglDisplay, eglSurface, FrameIDs[Index % NUM_FRAMES_TO_MONITOR], 1, &TimestampList, &Result);
					}
					if (Result > 0)
					{
						break;
					}
					DeltaFrameIndex++;
				}

				GGetTimeStampsSucceededThisFrame = Result > 0;
				if (GGetTimeStampsSucceededThisFrame)
				{
					EGLnsecsANDROID FudgeFactor = 0; //  8333 * 1000;
					EGLnsecsANDROID DeltaNanos = EGLnsecsANDROID(DesiredSyncIntervalRelativeToDevice) * EGLnsecsANDROID(DeltaFrameIndex) * DriverRefreshNanos;
					EGLnsecsANDROID PresentationTime = Result + DeltaNanos + FudgeFactor;
					eglPresentationTimeANDROID_p(eglDisplay, eglSurface, PresentationTime);
					GGetTimeStampsRetryCount = 0;
				}
				else
				{
					GGetTimeStampsRetryCount++;
					if (GGetTimeStampsRetryCount == FAndroidPlatformRHIFramePacer::CVarTimeStampErrorRetryCount.GetValueOnAnyThread())
					{
						UE_LOGF(LogRHI, Log, "eglGetFrameTimestampsANDROID_p failed for %d consecutive frames, reverting to naive frame pacer.", GGetTimeStampsRetryCount);
					}
				}
			}
			else
			{
				//UE_CLOGF(bPrintOnce, LogRHI, Log, "eglGetFrameTimestampsSupportedANDROID doesn't exist or retured false for EGL_FIRST_COMPOSITION_START_TIME_ANDROID, discarding eglGetNextFrameIdANDROID_p and eglGetFrameTimestampsANDROID_p");
			}
			//bPrintOnce = false;
		}

		LastTimeEmulatedSync = FPlatformTime::Seconds();

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_eglSwapBuffers);

			FrameIDs[(NextFrameIDSlot) % NUM_FRAMES_TO_MONITOR] = 0;
			if (eglGetNextFrameIdANDROID_p && (CanUseGetFrameTimestamps() || FAndroidPlatformRHIFramePacer::CVarSpewGetFrameTimestamps.GetValueOnAnyThread()))
			{
				eglGetNextFrameIdANDROID_p(eglDisplay, eglSurface, &FrameIDs[(NextFrameIDSlot) % NUM_FRAMES_TO_MONITOR]);
			}
			NextFrameIDSlot++;

			if (eglSurface == NULL || !eglSwapBuffers(eglDisplay, eglSurface))
			{
				// shutdown if swapbuffering goes down
				if (SwapBufferFailureCount > 10)
				{
					//Process.killProcess(Process.myPid());		//@todo android
				}
				SwapBufferFailureCount++;

				// basic reporting
				if (eglSurface == NULL)
				{
					return false;
				}
				else
				{
					if (eglGetError() == EGL_CONTEXT_LOST)
					{
						//Logger.LogOut("swapBuffers: EGL11.EGL_CONTEXT_LOST err: " + eglGetError());					
						//Process.killProcess(Process.myPid());		//@todo android
					}
				}

				return false;
			}
		}

		if (DesiredSyncIntervalRelativeToDevice > 0 && eglGetFrameTimestampsANDROID_p && FAndroidPlatformRHIFramePacer::CVarSpewGetFrameTimestamps.GetValueOnAnyThread())
		{
			static EGLint TimestampList[9] =
			{
				EGL_REQUESTED_PRESENT_TIME_ANDROID,
				EGL_RENDERING_COMPLETE_TIME_ANDROID,
				EGL_COMPOSITION_LATCH_TIME_ANDROID,
				EGL_FIRST_COMPOSITION_START_TIME_ANDROID,
				EGL_LAST_COMPOSITION_START_TIME_ANDROID,
				EGL_FIRST_COMPOSITION_GPU_FINISHED_TIME_ANDROID,
				EGL_DISPLAY_PRESENT_TIME_ANDROID,
				EGL_DEQUEUE_READY_TIME_ANDROID,
				EGL_READS_DONE_TIME_ANDROID
			};

			static const TCHAR* TimestampStrings[9] =
			{
				TEXT("EGL_REQUESTED_PRESENT_TIME_ANDROID"),
				TEXT("EGL_RENDERING_COMPLETE_TIME_ANDROID"),
				TEXT("EGL_COMPOSITION_LATCH_TIME_ANDROID"),
				TEXT("EGL_FIRST_COMPOSITION_START_TIME_ANDROID"),
				TEXT("EGL_LAST_COMPOSITION_START_TIME_ANDROID"),
				TEXT("EGL_FIRST_COMPOSITION_GPU_FINISHED_TIME_ANDROID"),
				TEXT("EGL_DISPLAY_PRESENT_TIME_ANDROID"),
				TEXT("EGL_DEQUEUE_READY_TIME_ANDROID"),
				TEXT("EGL_READS_DONE_TIME_ANDROID")
			};


			EGLnsecsANDROID Results[NUM_FRAMES_TO_MONITOR][9] = { {0} };
			EGLnsecsANDROID FirstRealValue = 0;
			for (int32 Index = int32(NextFrameIDSlot) - NUM_FRAMES_TO_MONITOR; Index < int32(NextFrameIDSlot); Index++)
			{
				eglGetFrameTimestampsANDROID_p(eglDisplay, eglSurface, FrameIDs[Index % NUM_FRAMES_TO_MONITOR], 9, TimestampList, Results[Index % NUM_FRAMES_TO_MONITOR]);
				for (int32 IndexInner = 0; IndexInner < 9; IndexInner++)
				{
					if (!FirstRealValue || (Results[Index % NUM_FRAMES_TO_MONITOR][IndexInner] > 1 && Results[Index % NUM_FRAMES_TO_MONITOR][IndexInner] < FirstRealValue))
					{
						FirstRealValue = Results[Index % NUM_FRAMES_TO_MONITOR][IndexInner];
					}
				}
			}
			UE_CLOGF(FAndroidPlatformRHIFramePacer::CVarSpewGetFrameTimestamps.GetValueOnAnyThread() > 1, LogRHI, Log, "************************************  frame %d   base time is %lld", NextFrameIDSlot - 1, FirstRealValue);

			for (int32 Index = int32(NextFrameIDSlot) - NUM_FRAMES_TO_MONITOR; Index < int32(NextFrameIDSlot); Index++)
			{

				UE_CLOGF(FAndroidPlatformRHIFramePacer::CVarSpewGetFrameTimestamps.GetValueOnAnyThread() > 1, LogRHI, Log, "eglGetFrameTimestampsANDROID_p  frame %d", Index);
				for (int32 IndexInner = 0; IndexInner < 9; IndexInner++)
				{
					int32 MsVal = (Results[Index % NUM_FRAMES_TO_MONITOR][IndexInner] > 1) ? int32((Results[Index % NUM_FRAMES_TO_MONITOR][IndexInner] - FirstRealValue) / 1000000) : int32(Results[Index % NUM_FRAMES_TO_MONITOR][IndexInner]);

					UE_CLOGF(FAndroidPlatformRHIFramePacer::CVarSpewGetFrameTimestamps.GetValueOnAnyThread() > 1, LogRHI, Log, "     %8d    %ls", MsVal, TimestampStrings[IndexInner]);
				}
			}

			int32 IndexLast = int32(NextFrameIDSlot) - NUM_FRAMES_TO_MONITOR;
			int32 IndexLastNext = IndexLast + 1;

			if (Results[IndexLast % NUM_FRAMES_TO_MONITOR][3] > 1 && Results[IndexLastNext % NUM_FRAMES_TO_MONITOR][3] > 1)
			{
				int32 MsVal = int32((Results[IndexLastNext % NUM_FRAMES_TO_MONITOR][3] - Results[IndexLast % NUM_FRAMES_TO_MONITOR][3]) / 1000000);

				RecordedFrameInterval[NumRecordedFrameInterval++] = MsVal;
				if (NumRecordedFrameInterval == 100)
				{
					FString All;
					int32 NumOnTarget = 0;
					int32 NumBelowTarget = 0;
					int32 NumAboveTarget = 0;
					for (int32 Index = 0; Index < 100; Index++)
					{
						if (Index)
						{
							All += TCHAR(' ');
						}
						All += FString::Printf(TEXT("%d"), RecordedFrameInterval[Index]);

						if (RecordedFrameInterval[Index] > DesiredSyncIntervalRelativeTo60Hz * 16 - 8 && RecordedFrameInterval[Index] < DesiredSyncIntervalRelativeTo60Hz * 16 + 8)
						{
							NumOnTarget++;
						}
						else if (RecordedFrameInterval[Index] < DesiredSyncIntervalRelativeTo60Hz * 16)
						{
							NumBelowTarget++;
						}
						else
						{
							NumAboveTarget++;
						}
					}
					UE_LOGF(LogRHI, Log, "%3d fast  %3d ok  %3d slow   %ls", NumBelowTarget, NumOnTarget, NumAboveTarget, *All);
					NumRecordedFrameInterval = 0;
				}
			}
		}
	}

	return true;
}

#endif
