// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatform.h"
#include "Android/AndroidPlatformFramePacer.h"
