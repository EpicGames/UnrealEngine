// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidOpenGLFunctions.h"

PFNeglPresentationTimeANDROID eglPresentationTimeANDROID_p = NULL;
PFNeglGetNextFrameIdANDROID eglGetNextFrameIdANDROID_p = NULL;
PFNeglGetCompositorTimingANDROID eglGetCompositorTimingANDROID_p = NULL;
PFNeglGetFrameTimestampsANDROID eglGetFrameTimestampsANDROID_p = NULL;
PFNeglQueryTimestampSupportedANDROID eglQueryTimestampSupportedANDROID_p = NULL;
PFNeglQueryTimestampSupportedANDROID eglGetCompositorTimingSupportedANDROID_p = NULL;
PFNeglQueryTimestampSupportedANDROID eglGetFrameTimestampsSupportedANDROID_p = NULL;

PFNEGLGETSYSTEMTIMENVPROC eglGetSystemTimeNV_p = NULL;
PFNEGLCREATESYNCKHRPROC eglCreateSyncKHR_p = NULL;
PFNEGLDESTROYSYNCKHRPROC eglDestroySyncKHR_p = NULL;
PFNEGLCLIENTWAITSYNCKHRPROC eglClientWaitSyncKHR_p = NULL;
PFNEGLGETSYNCATTRIBKHRPROC eglGetSyncAttribKHR_p = NULL;

PFNEGLGETNATIVECLIENTBUFFERANDROIDPROC eglGetNativeClientBufferANDROID_p = nullptr;
PFNEGLCREATEIMAGEKHRPROC eglCreateImageKHR_p = nullptr;
PFNEGLDESTROYIMAGEKHRPROC eglDestroyImageKHR_p = nullptr;
PFNGLEGLIMAGETARGETTEXTURE2DOESPROC glEGLImageTargetTexture2DOES_p = nullptr;

namespace GLFuncPointers
{
	// Offscreen MSAA rendering
	PFNGLFRAMEBUFFERTEXTURE2DMULTISAMPLEEXTPROC	glFramebufferTexture2DMultisampleEXT = NULL;
	PFNGLRENDERBUFFERSTORAGEMULTISAMPLEEXTPROC	glRenderbufferStorageMultisampleEXT = NULL;

	PFNGLPUSHGROUPMARKEREXTPROC				glPushGroupMarkerEXT = NULL;
	PFNGLPOPGROUPMARKEREXTPROC				glPopGroupMarkerEXT = NULL;
	PFNGLLABELOBJECTEXTPROC					glLabelObjectEXT = NULL;
	PFNGLGETOBJECTLABELEXTPROC				glGetObjectLabelEXT = NULL;

	PFNGLBUFFERSTORAGEEXTPROC				glBufferStorageEXT = NULL;
	// KHR_debug
	PFNGLDEBUGMESSAGECONTROLKHRPROC			glDebugMessageControlKHR = NULL;
	PFNGLDEBUGMESSAGEINSERTKHRPROC			glDebugMessageInsertKHR = NULL;
	PFNGLDEBUGMESSAGECALLBACKKHRPROC		glDebugMessageCallbackKHR = NULL;
	PFNGLGETDEBUGMESSAGELOGKHRPROC			glDebugMessageLogKHR = NULL;
	PFNGLGETPOINTERVKHRPROC					glGetPointervKHR = NULL;
	PFNGLPUSHDEBUGGROUPKHRPROC				glPushDebugGroupKHR = NULL;
	PFNGLPOPDEBUGGROUPKHRPROC				glPopDebugGroupKHR = NULL;
	PFNGLOBJECTLABELKHRPROC					glObjectLabelKHR = NULL;
	PFNGLGETOBJECTLABELKHRPROC				glGetObjectLabelKHR = NULL;
	PFNGLOBJECTPTRLABELKHRPROC				glObjectPtrLabelKHR = NULL;
	PFNGLGETOBJECTPTRLABELKHRPROC			glGetObjectPtrLabelKHR = NULL;

	// GL_EXT_disjoint_timer_query
	PFNGLQUERYCOUNTEREXTPROC                glQueryCounterEXT = nullptr;
	PFNGLGETQUERYOBJECTUI64VEXTPROC         glGetQueryObjectui64vEXT = nullptr;

	// ES 3.2
	PFNGLTEXBUFFEREXTPROC					glTexBufferEXT = nullptr;
	PFNGLTEXBUFFERRANGEEXTPROC				glTexBufferRangeEXT = nullptr;
	PFNGLCOPYIMAGESUBDATAEXTPROC			glCopyImageSubData = nullptr;
	PFNGLENABLEIEXTPROC						glEnableiEXT = nullptr;
	PFNGLDISABLEIEXTPROC					glDisableiEXT = nullptr;
	PFNGLBLENDEQUATIONIEXTPROC				glBlendEquationiEXT = nullptr;
	PFNGLBLENDEQUATIONSEPARATEIEXTPROC		glBlendEquationSeparateiEXT = nullptr;
	PFNGLBLENDFUNCIEXTPROC					glBlendFunciEXT = nullptr;
	PFNGLBLENDFUNCSEPARATEIEXTPROC			glBlendFuncSeparateiEXT = nullptr;
	PFNGLCOLORMASKIEXTPROC					glColorMaskiEXT = nullptr;
	PFNGLFRAMEBUFFERTEXTUREPROC				glFramebufferTexture = nullptr;

	PFNGLFRAMEBUFFERTEXTUREMULTIVIEWOVRPROC				glFramebufferTextureMultiviewOVR = NULL;
	PFNGLFRAMEBUFFERTEXTUREMULTISAMPLEMULTIVIEWOVRPROC	glFramebufferTextureMultisampleMultiviewOVR = NULL;

	PFNGLFRAMEBUFFERFETCHBARRIERQCOMPROC glFramebufferFetchBarrierQCOM = NULL;
};
