// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenGLPlatform.h"

typedef khronos_stime_nanoseconds_t EGLnsecsANDROID;

typedef GLboolean(GL_APIENTRYP PFNeglPresentationTimeANDROID) (EGLDisplay dpy, EGLSurface surface, EGLnsecsANDROID time);
typedef GLboolean(GL_APIENTRYP PFNeglGetNextFrameIdANDROID) (EGLDisplay dpy, EGLSurface surface, EGLuint64KHR* frameId);
typedef GLboolean(GL_APIENTRYP PFNeglGetCompositorTimingANDROID) (EGLDisplay dpy, EGLSurface surface, EGLint numTimestamps, const EGLint* names, EGLnsecsANDROID* values);
typedef GLboolean(GL_APIENTRYP PFNeglGetFrameTimestampsANDROID) (EGLDisplay dpy, EGLSurface surface, EGLuint64KHR frameId, EGLint numTimestamps, const EGLint* timestamps, EGLnsecsANDROID* values);
typedef GLboolean(GL_APIENTRYP PFNeglQueryTimestampSupportedANDROID) (EGLDisplay dpy, EGLSurface surface, EGLint timestamp);

typedef EGLClientBuffer(GL_APIENTRYP PFNEGLGETNATIVECLIENTBUFFERANDROIDPROC) (const struct AHardwareBuffer* buffer);

extern "C"
{
	extern PFNEGLGETSYSTEMTIMENVPROC eglGetSystemTimeNV_p;
	extern PFNEGLCREATESYNCKHRPROC eglCreateSyncKHR_p;
	extern PFNEGLDESTROYSYNCKHRPROC eglDestroySyncKHR_p;
	extern PFNEGLCLIENTWAITSYNCKHRPROC eglClientWaitSyncKHR_p;
	extern PFNEGLGETSYNCATTRIBKHRPROC eglGetSyncAttribKHR_p;

	extern PFNeglPresentationTimeANDROID eglPresentationTimeANDROID_p;
	extern PFNeglGetNextFrameIdANDROID eglGetNextFrameIdANDROID_p;
	extern PFNeglGetCompositorTimingANDROID eglGetCompositorTimingANDROID_p;
	extern PFNeglGetFrameTimestampsANDROID eglGetFrameTimestampsANDROID_p;
	extern PFNeglQueryTimestampSupportedANDROID eglQueryTimestampSupportedANDROID_p;
	extern PFNeglQueryTimestampSupportedANDROID eglGetCompositorTimingSupportedANDROID_p;
	extern PFNeglQueryTimestampSupportedANDROID eglGetFrameTimestampsSupportedANDROID_p;

	extern PFNEGLGETNATIVECLIENTBUFFERANDROIDPROC eglGetNativeClientBufferANDROID_p;
	extern PFNEGLCREATEIMAGEKHRPROC eglCreateImageKHR_p;
	extern PFNEGLDESTROYIMAGEKHRPROC eglDestroyImageKHR_p;
	extern PFNGLEGLIMAGETARGETTEXTURE2DOESPROC glEGLImageTargetTexture2DOES_p;
}

namespace GLFuncPointers
{
	// GL_QCOM_shader_framebuffer_fetch_noncoherent
	extern PFNGLFRAMEBUFFERFETCHBARRIERQCOMPROC	glFramebufferFetchBarrierQCOM;
}

// FIXME: include gl32.h
typedef void (GL_APIENTRYP PFNGLFRAMEBUFFERTEXTUREPROC) (GLenum target, GLenum attachment, GLuint texture, GLint level);

namespace GLFuncPointers
{
	// GL_EXT_multisampled_render_to_texture
	extern PFNGLFRAMEBUFFERTEXTURE2DMULTISAMPLEEXTPROC	glFramebufferTexture2DMultisampleEXT;
	extern PFNGLRENDERBUFFERSTORAGEMULTISAMPLEEXTPROC	glRenderbufferStorageMultisampleEXT;

	// GL_EXT_debug_marker
	extern PFNGLPUSHGROUPMARKEREXTPROC		glPushGroupMarkerEXT;
	extern PFNGLPOPGROUPMARKEREXTPROC 		glPopGroupMarkerEXT;

	// GL_EXT_debug_label
	extern PFNGLLABELOBJECTEXTPROC			glLabelObjectEXT;
	extern PFNGLGETOBJECTLABELEXTPROC		glGetObjectLabelEXT;
	//GL_EXT_buffer_storage
	extern PFNGLBUFFERSTORAGEEXTPROC		glBufferStorageEXT;

	extern PFNGLDEBUGMESSAGECONTROLKHRPROC	glDebugMessageControlKHR;
	extern PFNGLDEBUGMESSAGEINSERTKHRPROC	glDebugMessageInsertKHR;
	extern PFNGLDEBUGMESSAGECALLBACKKHRPROC	glDebugMessageCallbackKHR;
	extern PFNGLGETDEBUGMESSAGELOGKHRPROC	glDebugMessageLogKHR;
	extern PFNGLGETPOINTERVKHRPROC			glGetPointervKHR;
	extern PFNGLPUSHDEBUGGROUPKHRPROC		glPushDebugGroupKHR;
	extern PFNGLPOPDEBUGGROUPKHRPROC		glPopDebugGroupKHR;
	extern PFNGLOBJECTLABELKHRPROC			glObjectLabelKHR;
	extern PFNGLGETOBJECTLABELKHRPROC		glGetObjectLabelKHR;
	extern PFNGLOBJECTPTRLABELKHRPROC		glObjectPtrLabelKHR;
	extern PFNGLGETOBJECTPTRLABELKHRPROC	glGetObjectPtrLabelKHR;

	// GL_EXT_disjoint_timer_query
	extern PFNGLQUERYCOUNTEREXTPROC         glQueryCounterEXT;
	extern PFNGLGETQUERYOBJECTUI64VEXTPROC  glGetQueryObjectui64vEXT;

	// ES 3.2
	extern PFNGLTEXBUFFEREXTPROC				glTexBufferEXT;
	extern PFNGLTEXBUFFERRANGEEXTPROC			glTexBufferRangeEXT;
	extern PFNGLCOPYIMAGESUBDATAEXTPROC			glCopyImageSubData;
	extern PFNGLENABLEIEXTPROC					glEnableiEXT;
	extern PFNGLDISABLEIEXTPROC					glDisableiEXT;
	extern PFNGLBLENDEQUATIONIEXTPROC			glBlendEquationiEXT;
	extern PFNGLBLENDEQUATIONSEPARATEIEXTPROC	glBlendEquationSeparateiEXT;
	extern PFNGLBLENDFUNCIEXTPROC				glBlendFunciEXT;
	extern PFNGLBLENDFUNCSEPARATEIEXTPROC		glBlendFuncSeparateiEXT;
	extern PFNGLCOLORMASKIEXTPROC				glColorMaskiEXT;
	extern PFNGLFRAMEBUFFERTEXTUREPROC			glFramebufferTexture;

	// Mobile multi-view
	extern PFNGLFRAMEBUFFERTEXTUREMULTIVIEWOVRPROC glFramebufferTextureMultiviewOVR;
	extern PFNGLFRAMEBUFFERTEXTUREMULTISAMPLEMULTIVIEWOVRPROC glFramebufferTextureMultisampleMultiviewOVR;
};

using namespace GLFuncPointers;