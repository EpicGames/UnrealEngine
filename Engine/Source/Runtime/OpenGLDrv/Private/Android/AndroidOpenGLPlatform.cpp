// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidOpenGLPlatform.h"

#include "Android/AndroidApplication.h"
#include "Android/AndroidEGL.h"
#include "Android/AndroidOpenGLPrivate.h"
#include "Android/AndroidPlatformMisc.h"

FString FAndroidMisc::GetGPUFamily()
{
	return FAndroidGPUInfo::Get().GetGPUFamily();
}

FString FAndroidMisc::GetGLVersion()
{
	return FAndroidGPUInfo::Get().GLVersion;
}

bool FAndroidMisc::SupportsFloatingPointRenderTargets()
{
	return FAndroidGPUInfo::Get().bSupportsFloatingPointRenderTargets;
}

bool FAndroidMisc::SupportsShaderFramebufferFetch()
{
	return FAndroidGPUInfo::Get().bSupportsFrameBufferFetch;
}

bool FAndroidMisc::SupportsES30()
{
	return true;
}

void FAndroidMisc::GetValidTargetPlatforms(TArray<FString>& TargetPlatformNames)
{
	TargetPlatformNames = FAndroidGPUInfo::Get().TargetPlatformNames;
}

void FAndroidAppEntry::PlatformInit()
{
	// Try to create an ES3.2 EGL here for gpu queries and don't have to recreate the GL context.
	AndroidEGL::GetInstance()->Init(AndroidEGL::AV_OpenGLES, 3, 2);
}

void FAndroidAppEntry::ReleaseEGL()
{
	AndroidEGL* EGL = AndroidEGL::GetInstance();
	if (EGL->IsInitialized())
	{
		EGL->Terminate();
	}
}
