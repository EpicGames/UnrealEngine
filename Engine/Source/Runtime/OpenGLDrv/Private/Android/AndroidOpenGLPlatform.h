// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenGLThirdParty.h"

#define UGL_PLATFORM_SUPPORTS_GLES 1
#define UGL_PLATFORM_SUPPORTS_GL3  0
#define UGL_PLATFORM_SUPPORTS_GL4  0

typedef GLsync			UGLsync;
#define GLdouble		GLfloat
typedef khronos_int64_t GLint64;
typedef khronos_uint64_t GLuint64;
#define GL_CLAMP		GL_CLAMP_TO_EDGE

#define EGL_TIMESTAMPS_ANDROID 0x3430
#define EGL_COMPOSITE_DEADLINE_ANDROID 0x3431
#define EGL_COMPOSITE_INTERVAL_ANDROID 0x3432
#define EGL_COMPOSITE_TO_PRESENT_LATENCY_ANDROID 0x3433
#define EGL_REQUESTED_PRESENT_TIME_ANDROID 0x3434
#define EGL_RENDERING_COMPLETE_TIME_ANDROID 0x3435
#define EGL_COMPOSITION_LATCH_TIME_ANDROID 0x3436
#define EGL_FIRST_COMPOSITION_START_TIME_ANDROID 0x3437
#define EGL_LAST_COMPOSITION_START_TIME_ANDROID 0x3438
#define EGL_FIRST_COMPOSITION_GPU_FINISHED_TIME_ANDROID 0x3439
#define EGL_DISPLAY_PRESENT_TIME_ANDROID 0x343A
#define EGL_DEQUEUE_READY_TIME_ANDROID 0x343B
#define EGL_READS_DONE_TIME_ANDROID 0x343C

#ifdef GL_AMD_debug_output
	#undef GL_AMD_debug_output
#endif

/** Unreal tokens that maps to different OpenGL tokens by platform. */
#define UGL_DRAW_FRAMEBUFFER	GL_DRAW_FRAMEBUFFER_NV
#define UGL_READ_FRAMEBUFFER	GL_READ_FRAMEBUFFER_NV
#define UGL_ABGR8				GL_UNSIGNED_BYTE
#define UGL_ANY_SAMPLES_PASSED	GL_ANY_SAMPLES_PASSED
#define UGL_SAMPLES_PASSED		GL_SAMPLES_PASSED
#define UGL_TIME_ELAPSED		GL_TIME_ELAPSED_EXT
#define UGL_CLAMP_TO_BORDER		GL_CLAMP_TO_EDGE

/** Official OpenGL definitions */
#ifndef GL_FILL
#define GL_FILL 0x1B02
#endif
#ifndef GL_GEOMETRY_SHADER
#define GL_GEOMETRY_SHADER 0x8DD9
#endif
#ifndef GL_SAMPLER_1D
#define GL_SAMPLER_1D 0x8B5D
#endif
#ifndef GL_SAMPLER_1D_SHADOW
#define GL_SAMPLER_1D_SHADOW 0x8B61
#endif
#ifndef GL_DOUBLE
#define GL_DOUBLE 0x140A
#endif
#ifndef GL_BGRA
#define GL_BGRA	GL_BGRA_EXT 
#endif
#ifndef GL_TEXTURE_BUFFER
#define GL_TEXTURE_BUFFER 0x8C2A
#endif
#ifndef GL_RGBA16
/* GL_EXT_texture_norm16 */
#define GL_RGBA16 GL_RGBA16_EXT
#endif
#ifndef GL_R16
/* GL_EXT_texture_norm16 */
#define GL_R16 GL_R16_EXT
#endif
#ifndef GL_TEXTURE_BUFFER_OFFSET_ALIGNMENT
#define GL_TEXTURE_BUFFER_OFFSET_ALIGNMENT 0x919F
#endif
#ifndef GL_POLYGON_OFFSET_LINE
#define GL_POLYGON_OFFSET_LINE 0x2A02
#endif
#ifndef GL_POLYGON_OFFSET_POINT
#define GL_POLYGON_OFFSET_POINT 0x2A01
#endif
#ifndef GL_TEXTURE_LOD_BIAS
#define GL_TEXTURE_LOD_BIAS 0x8501
#endif
#ifndef GL_FRAMEBUFFER_SRGB
#define GL_FRAMEBUFFER_SRGB 0x8DB9
#endif
#ifndef GL_SAMPLES_PASSED
#define GL_SAMPLES_PASSED 0x8914
#endif
#ifndef GL_POINT
#define GL_POINT 0x1B00
#endif
#ifndef GL_LINE
#define GL_LINE 0x1B01
#endif
#ifndef GL_TEXTURE_1D
#define GL_TEXTURE_1D			0x0DE0
#endif
#ifndef GL_TEXTURE_1D_ARRAY
#define GL_TEXTURE_1D_ARRAY		0x8C18
#endif
#ifndef GL_TEXTURE_RECTANGLE
#define GL_TEXTURE_RECTANGLE	0x84F5
#endif
#ifndef GL_MAX_TEXTURE_BUFFER_SIZE
#define GL_MAX_TEXTURE_BUFFER_SIZE 0x8C2B
#endif
#ifndef GL_DEPTH_CLAMP
#define GL_DEPTH_CLAMP 0x864F
#endif

/** For the shader stage bits that don't exist just use 0 */
#define GL_GEOMETRY_SHADER_BIT				0x00000000
#define GL_TESS_CONTROL_SHADER_BIT			0x00000000
#define GL_TESS_EVALUATION_SHADER_BIT		0x00000000

// Normalize debug macros due to naming differences across GL versions
#if defined(GL_KHR_debug) && GL_KHR_debug
#define GL_DEBUG_SOURCE_OTHER_ARB GL_DEBUG_SOURCE_OTHER_KHR
#define GL_DEBUG_SOURCE_API_ARB GL_DEBUG_SOURCE_API_KHR
#define GL_DEBUG_TYPE_ERROR_ARB GL_DEBUG_TYPE_ERROR_KHR
#define GL_DEBUG_TYPE_OTHER_ARB GL_DEBUG_TYPE_OTHER_KHR
#define GL_DEBUG_TYPE_MARKER GL_DEBUG_TYPE_MARKER_KHR
#define GL_DEBUG_TYPE_PUSH_GROUP GL_DEBUG_TYPE_PUSH_GROUP_KHR
#define GL_DEBUG_TYPE_POP_GROUP GL_DEBUG_TYPE_POP_GROUP_KHR
#define GL_DEBUG_SEVERITY_HIGH_ARB GL_DEBUG_SEVERITY_HIGH_KHR
#define GL_DEBUG_SEVERITY_LOW_ARB GL_DEBUG_SEVERITY_LOW_KHR
#define GL_DEBUG_SEVERITY_NOTIFICATION GL_DEBUG_SEVERITY_NOTIFICATION_KHR
#endif
