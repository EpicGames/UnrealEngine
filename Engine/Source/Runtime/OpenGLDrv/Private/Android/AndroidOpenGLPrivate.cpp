// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidOpenGLPrivate.h"

bool GAndroidGPUInfoReady = false;

FAndroidGPUInfo& FAndroidGPUInfo::Get()
{
	static FAndroidGPUInfo This;
	return This;
}

FAndroidGPUInfo::FAndroidGPUInfo()
{
	// this is only valid in the game thread, make sure we are initialized there before being called on other threads!
	check(IsInGameThread())

	// make sure GL is started so we can get the supported formats
	AndroidEGL* EGL = AndroidEGL::GetInstance();

	if (!EGL->IsInitialized())
	{
		FAndroidAppEntry::PlatformInit();
	}

	FPlatformMisc::LowLevelOutputDebugString(TEXT("FAndroidGPUInfo"));

	EGL->SetCurrentRenderingContext(nullptr); // a windowed surface is not required here.

	// get extensions
	// Process the extension caps directly here, as FOpenGL might not yet be setup
	// Do not process extensions here, because extension pointers may not be setup
	const ANSICHAR* GlGetStringOutput = (const ANSICHAR*)glGetString(GL_EXTENSIONS);
	FString ExtensionsString = GlGetStringOutput;

	GLVersion = (const ANSICHAR*)glGetString(GL_VERSION);

	// highest priority is the per-texture version
	if (ExtensionsString.Contains(TEXT("GL_KHR_texture_compression_astc_ldr")))
	{
		TargetPlatformNames.Add(TEXT("Android_ASTC"));
	}
	if (ExtensionsString.Contains(TEXT("GL_NV_texture_compression_s3tc")) || ExtensionsString.Contains(TEXT("GL_EXT_texture_compression_s3tc")))
	{
		TargetPlatformNames.Add(TEXT("Android_DXT"));
	}

	TargetPlatformNames.Add(TEXT("Android_ETC2"));

	// finally, generic Android
	TargetPlatformNames.Add(TEXT("Android"));

	bSupportsFloatingPointRenderTargets =
		ExtensionsString.Contains(TEXT("GL_EXT_color_buffer_half_float"))
		// According to https://www.khronos.org/registry/gles/extensions/EXT/EXT_color_buffer_float.txt
		|| (ExtensionsString.Contains(TEXT("GL_EXT_color_buffer_float")));

	bSupportsFrameBufferFetch = ExtensionsString.Contains(TEXT("GL_EXT_shader_framebuffer_fetch")) || ExtensionsString.Contains(TEXT("GL_NV_shader_framebuffer_fetch"))
		|| ExtensionsString.Contains(TEXT("GL_ARM_shader_framebuffer_fetch ")); // has space at the end to exclude GL_ARM_shader_framebuffer_fetch_depth_stencil match

	GAndroidGPUInfoReady = true;

	VendorName = FString(ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VENDOR)));
}

void FAndroidGPUInfo::ReadGPUFamily()
{
	GPUFamily = (const ANSICHAR*)glGetString(GL_RENDERER);
	check(!GPUFamily.IsEmpty());

	// If we're ANGLE, get the raw GPU string from Vulkan.
	if (GPUFamily.StartsWith(TEXT("ANGLE")) && FAndroidMisc::HasVulkanDriverSupport())
	{
		UE_LOGF(LogAndroid, Log, "FAndroidGPUInfo ANGLE detected with GL_RENDERER string %ls", *GPUFamily);
		GPUFamily = FAndroidMisc::GetVulkanDeviceNameString();
		UE_LOGF(LogAndroid, Log, "FAndroidGPUInfo Using Vulkan deviceName string %ls", *GPUFamily);
	}

	// thirdparty api requires std::unique_ptr
	std::unique_ptr<libgpuinfo::instance> ArmGPUInfoInstance = libgpuinfo::instance::create();

	if (ArmGPUInfoInstance)
	{
		const libgpuinfo::gpuinfo& ArmGPUInfo = ArmGPUInfoInstance->get_info();
		// Note:
		// if libgpuinfo is not upto date then the gpu may not appear in gpuinfo's internal list,
		// To avoid this we ignore the name and use the lib to extract only the core count. (which does not use the list)
		const FRegexPattern RegexPattern(TEXT("^Mali(?:.+[MC|MP]([0-9]+))?")); // find anything that starts with Mali and capture the number after the last M[CP]
		FRegexMatcher RegexMatcher(RegexPattern, *GPUFamily);
		if (RegexMatcher.FindNext() && ArmGPUInfo.num_shader_cores > 0)
		{
			FString Capture = RegexMatcher.GetCaptureGroup(1);
			if (Capture.IsEmpty())
			{
				FString ARMLibName = FString::Format(TEXT("{0} MP{1}"), { *GPUFamily, ArmGPUInfo.num_shader_cores });
				UE_LOGF(LogAndroid, Log, "FAndroidGPUInfo renaming GPUFamily: %ls -> %ls", *GPUFamily, *ARMLibName);
				GPUFamily = ARMLibName;
			}
			else if ((uint32)FCString::Atoi64(*Capture) != ArmGPUInfo.num_shader_cores)
			{
				UE_LOGF(LogAndroid, Warning, "FAndroidGPUInfo GPUFamily core count mismatch: %ls, expected MP%d", *GPUFamily, ArmGPUInfo.num_shader_cores);
			}
		}
	}
}
