// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidOpenGLPrivate.h: Code shared betweeen AndroidOpenGL and AndroidESDeferredOpenGL (Removed)
=============================================================================*/
#pragma once

#include "AndroidEGL.h"
#include "Android/AndroidApplication.h"
#include "libgpuinfo.hpp"
#include "Internationalization/Regex.h"
#include "Misc/CString.h"

extern bool GAndroidGPUInfoReady;

// call out to JNI to see if the application was packaged for XRImmersiveHMD
extern bool AndroidThunkCpp_IsXRImmersiveHMDApplication();
extern bool ShouldUseGPUFencesToLimitLatency();

class FAndroidGPUInfo
{
public:
	static FAndroidGPUInfo& Get();

	FString GLVersion;
	FString VendorName;
	bool bSupportsFloatingPointRenderTargets;
	bool bSupportsFrameBufferFetch;
	TArray<FString> TargetPlatformNames;

	void RemoveTargetPlatform(FString PlatformName)
	{
		TargetPlatformNames.Remove(PlatformName);
	}

	// computing GPU family needs regex access, which might not be available early in init
	FString& GetGPUFamily()
	{
		if (GPUFamily.IsEmpty())
			ReadGPUFamily();
		return GPUFamily;
	}

private:
	FString GPUFamily;

	FAndroidGPUInfo();
	void ReadGPUFamily();
};
