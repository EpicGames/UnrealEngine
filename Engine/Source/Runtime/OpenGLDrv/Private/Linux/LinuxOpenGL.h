// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenGLThirdParty.h"
#include "OpenGLPlatform.h"

#include "OpenGL4.h"

struct FLinuxOpenGL : public FOpenGL4
{
	static FORCEINLINE void InitDebugContext()
	{
		bDebugContext = glIsEnabled(GL_DEBUG_OUTPUT) != GL_FALSE;
	}

	static FORCEINLINE void LabelObject(GLenum Type, GLuint Object, const ANSICHAR* Name)
	{
		if (glObjectLabel && bDebugContext)
		{
			glObjectLabel(Type, Object, FCStringAnsi::Strlen(Name), Name);
		}
	}

	static FORCEINLINE void PushGroupMarker(const ANSICHAR* Name)
	{
		if (glPushDebugGroup && bDebugContext)
		{
			glPushDebugGroup(GL_DEBUG_SOURCE_APPLICATION, 1, FCStringAnsi::Strlen(Name), Name);
		}
	}

	static FORCEINLINE void PopGroupMarker()
	{
		if (glPopDebugGroup && bDebugContext)
		{
			glPopDebugGroup();
		}
	}

	static FORCEINLINE bool TexStorage2D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLenum Format, GLenum Type, ETextureCreateFlags Flags)
	{
		if (glTexStorage2D != NULL)
		{
			glTexStorage2D(Target, Levels, InternalFormat, Width, Height);
			return true;
		}

		return false;
	}

	static FORCEINLINE void TexStorage3D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type)
	{
		if (glTexStorage3D)
		{
			glTexStorage3D(Target, Levels, InternalFormat, Width, Height, Depth);
		}
		else
		{
			const bool bArrayTexture = Target == GL_TEXTURE_2D_ARRAY || Target == GL_TEXTURE_CUBE_MAP_ARRAY;

			for (uint32 MipIndex = 0; MipIndex < uint32(Levels); MipIndex++)
			{
				glTexImage3D(
					Target,
					MipIndex,
					InternalFormat,
					FMath::Max<uint32>(1, (Width >> MipIndex)),
					FMath::Max<uint32>(1, (Height >> MipIndex)),
					(bArrayTexture) ? Depth : FMath::Max<uint32>(1, (Depth >> MipIndex)),
					0,
					Format,
					Type,
					NULL
				);
			}
		}
	}

	static FORCEINLINE void CopyImageSubData(GLuint SrcName, GLenum SrcTarget, GLint SrcLevel, GLint SrcX, GLint SrcY, GLint SrcZ, GLuint DstName, GLenum DstTarget, GLint DstLevel, GLint DstX, GLint DstY, GLint DstZ, GLsizei Width, GLsizei Height, GLsizei Depth)
	{
		glCopyImageSubData(SrcName, SrcTarget, SrcLevel, SrcX, SrcY, SrcZ, DstName, DstTarget, DstLevel, DstX, DstY, DstZ, Width, Height, Depth);
	}

	static void ProcessExtensions(const FString& ExtensionsString)
	{
		FOpenGL4::ProcessExtensions(ExtensionsString);

		FString VendorName(ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VENDOR)));

		if (VendorName.Contains(TEXT("ATI ")))
		{
			// Workaround for AMD driver not handling GL_SRGB8_ALPHA8 in glTexStorage2D() properly (gets treated as non-sRGB)
			// FIXME: obsolete ? this was the case in <= 2014
			glTexStorage1D = nullptr;
			glTexStorage2D = nullptr;
			glTexStorage3D = nullptr;
		}
	}
};

using FOpenGL = FLinuxOpenGL;

