// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxOpenGLFunctions.h"

/*------------------------------------------------------------------------------
	OpenGL function pointers.
------------------------------------------------------------------------------*/
#define DEFINE_GL_ENTRYPOINTS(Type,Func) Type Func = NULL;
namespace GLFuncPointers	// see explanation in LinuxOpenGLFunctions.h why we need the namespace
{
	ENUM_GL_ENTRYPOINTS_ALL(DEFINE_GL_ENTRYPOINTS);
};
#undef DEFINE_GL_ENTRYPOINTS
