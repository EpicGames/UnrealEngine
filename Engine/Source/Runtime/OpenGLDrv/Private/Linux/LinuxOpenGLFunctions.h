// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenGLPlatform.h"

/** List all OpenGL entry points used by Unreal. */
#define ENUM_GL_ENTRYPOINTS(EnumMacro) \
	EnumMacro(PFNGLBINDTEXTUREPROC,glBindTexture) \
	EnumMacro(PFNGLBLENDFUNCPROC,glBlendFunc) \
	EnumMacro(PFNGLCOLORMASKPROC,glColorMask) \
	EnumMacro(PFNGLCOPYTEXIMAGE1DPROC,glCopyTexImage1D) \
	EnumMacro(PFNGLCOPYTEXIMAGE2DPROC,glCopyTexImage2D) \
	EnumMacro(PFNGLCOPYTEXSUBIMAGE1DPROC,glCopyTexSubImage1D) \
	EnumMacro(PFNGLCOPYTEXSUBIMAGE2DPROC,glCopyTexSubImage2D) \
	EnumMacro(PFNGLCULLFACEPROC,glCullFace) \
	EnumMacro(PFNGLDELETETEXTURESPROC,glDeleteTextures) \
	EnumMacro(PFNGLDEPTHFUNCPROC,glDepthFunc) \
	EnumMacro(PFNGLDEPTHMASKPROC,glDepthMask) \
	EnumMacro(PFNGLDEPTHRANGEPROC,glDepthRange) \
	EnumMacro(PFNGLDISABLEPROC,glDisable) \
	EnumMacro(PFNGLDRAWARRAYSPROC,glDrawArrays) \
	EnumMacro(PFNGLDRAWBUFFERPROC,glDrawBuffer) \
	EnumMacro(PFNGLDRAWELEMENTSPROC,glDrawElements) \
	EnumMacro(PFNGLENABLEPROC,glEnable) \
	EnumMacro(PFNGLFINISHPROC,glFinish) \
	EnumMacro(PFNGLFLUSHPROC,glFlush) \
	EnumMacro(PFNGLFRONTFACEPROC,glFrontFace) \
	EnumMacro(PFNGLGENTEXTURESPROC,glGenTextures) \
	EnumMacro(PFNGLGETBOOLEANVPROC,glGetBooleanv) \
	EnumMacro(PFNGLGETDOUBLEVPROC,glGetDoublev) \
	EnumMacro(PFNGLGETERRORPROC,glGetError) \
	EnumMacro(PFNGLGETFLOATVPROC,glGetFloatv) \
	EnumMacro(PFNGLGETINTEGERVPROC,glGetIntegerv) \
	EnumMacro(PFNGLGETPOINTERVPROC,glGetPointerv) \
	EnumMacro(PFNGLGETSTRINGPROC,glGetString) \
	EnumMacro(PFNGLGETTEXIMAGEPROC,glGetTexImage) \
	EnumMacro(PFNGLGETTEXLEVELPARAMETERFVPROC,glGetTexLevelParameterfv) \
	EnumMacro(PFNGLGETTEXLEVELPARAMETERIVPROC,glGetTexLevelParameteriv) \
	EnumMacro(PFNGLGETTEXPARAMETERFVPROC,glGetTexParameterfv) \
	EnumMacro(PFNGLGETTEXPARAMETERIVPROC,glGetTexParameteriv) \
	EnumMacro(PFNGLHINTPROC,glHint) \
	EnumMacro(PFNGLISENABLEDPROC,glIsEnabled) \
	EnumMacro(PFNGLISTEXTUREPROC,glIsTexture) \
	EnumMacro(PFNGLLINEWIDTHPROC,glLineWidth) \
	EnumMacro(PFNGLLOGICOPPROC,glLogicOp) \
	EnumMacro(PFNGLPIXELSTOREFPROC,glPixelStoref) \
	EnumMacro(PFNGLPIXELSTOREIPROC,glPixelStorei) \
	EnumMacro(PFNGLPOINTSIZEPROC,glPointSize) \
	EnumMacro(PFNGLPOLYGONMODEPROC,glPolygonMode) \
	EnumMacro(PFNGLPOLYGONOFFSETPROC,glPolygonOffset) \
	EnumMacro(PFNGLREADBUFFERPROC,glReadBuffer) \
	EnumMacro(PFNGLREADPIXELSPROC,glReadPixels) \
	EnumMacro(PFNGLSCISSORPROC,glScissor) \
	EnumMacro(PFNGLCLEARDEPTHPROC,glClearDepth) \
	EnumMacro(PFNGLCLEARCOLORPROC,glClearColor) \
	EnumMacro(PFNGLCLEARSTENCILPROC,glClearStencil) \
	EnumMacro(PFNGLCLEARPROC,glClear) \
	EnumMacro(PFNGLSTENCILFUNCPROC,glStencilFunc) \
	EnumMacro(PFNGLSTENCILMASKPROC,glStencilMask) \
	EnumMacro(PFNGLSTENCILOPPROC,glStencilOp) \
	EnumMacro(PFNGLTEXIMAGE1DPROC,glTexImage1D) \
	EnumMacro(PFNGLTEXIMAGE2DPROC,glTexImage2D) \
	EnumMacro(PFNGLTEXPARAMETERFPROC,glTexParameterf) \
	EnumMacro(PFNGLTEXPARAMETERFVPROC,glTexParameterfv) \
	EnumMacro(PFNGLTEXPARAMETERIPROC,glTexParameteri) \
	EnumMacro(PFNGLTEXPARAMETERIVPROC,glTexParameteriv) \
	EnumMacro(PFNGLTEXSUBIMAGE1DPROC,glTexSubImage1D) \
	EnumMacro(PFNGLTEXSUBIMAGE2DPROC,glTexSubImage2D) \
	EnumMacro(PFNGLVIEWPORTPROC,glViewport) \
	EnumMacro(PFNGLBINDSAMPLERPROC,glBindSampler) \
	EnumMacro(PFNGLDELETESAMPLERSPROC,glDeleteSamplers) \
	EnumMacro(PFNGLGENSAMPLERSPROC,glGenSamplers) \
	EnumMacro(PFNGLSAMPLERPARAMETERIPROC,glSamplerParameteri) \
	EnumMacro(PFNGLCLEARBUFFERFVPROC,glClearBufferfv) \
	EnumMacro(PFNGLCLEARBUFFERIVPROC,glClearBufferiv) \
	EnumMacro(PFNGLCLEARBUFFERUIVPROC,glClearBufferuiv) \
	EnumMacro(PFNGLCLEARBUFFERFIPROC,glClearBufferfi) \
	EnumMacro(PFNGLCOLORMASKIPROC,glColorMaski) \
	EnumMacro(PFNGLDISABLEIPROC,glDisablei) \
	EnumMacro(PFNGLENABLEIPROC,glEnablei) \
	EnumMacro(PFNGLGETBOOLEANI_VPROC,glGetBooleani_v) \
	EnumMacro(PFNGLGETINTEGERI_VPROC,glGetIntegeri_v) \
	EnumMacro(PFNGLISENABLEDIPROC,glIsEnabledi) \
	EnumMacro(PFNGLBLENDEQUATIONSEPARATEIARBPROC,glBlendEquationSeparateiARB) \
	EnumMacro(PFNGLBLENDEQUATIONIARBPROC,glBlendEquationiARB) \
	EnumMacro(PFNGLBLENDFUNCSEPARATEIARBPROC,glBlendFuncSeparateiARB) \
	EnumMacro(PFNGLBLENDFUNCIARBPROC,glBlendFunciARB) \
	EnumMacro(PFNGLBLENDCOLORPROC,glBlendColor) \
	EnumMacro(PFNGLBLENDEQUATIONPROC,glBlendEquation) \
	EnumMacro(PFNGLDRAWRANGEELEMENTSPROC,glDrawRangeElements) \
	EnumMacro(PFNGLTEXIMAGE3DPROC,glTexImage3D) \
	EnumMacro(PFNGLTEXSUBIMAGE3DPROC,glTexSubImage3D) \
	EnumMacro(PFNGLCOPYTEXSUBIMAGE3DPROC,glCopyTexSubImage3D) \
	EnumMacro(PFNGLACTIVETEXTUREPROC,glActiveTexture) \
	EnumMacro(PFNGLSAMPLECOVERAGEPROC,glSampleCoverage) \
	EnumMacro(PFNGLCOMPRESSEDTEXIMAGE3DPROC,glCompressedTexImage3D) \
	EnumMacro(PFNGLCOMPRESSEDTEXIMAGE2DPROC,glCompressedTexImage2D) \
	EnumMacro(PFNGLCOMPRESSEDTEXIMAGE1DPROC,glCompressedTexImage1D) \
	EnumMacro(PFNGLCOMPRESSEDTEXSUBIMAGE3DPROC,glCompressedTexSubImage3D) \
	EnumMacro(PFNGLCOMPRESSEDTEXSUBIMAGE2DPROC,glCompressedTexSubImage2D) \
	EnumMacro(PFNGLCOMPRESSEDTEXSUBIMAGE1DPROC,glCompressedTexSubImage1D) \
	EnumMacro(PFNGLGETCOMPRESSEDTEXIMAGEPROC,glGetCompressedTexImage) \
	EnumMacro(PFNGLBLENDFUNCSEPARATEPROC,glBlendFuncSeparate) \
	EnumMacro(PFNGLMULTIDRAWARRAYSPROC,glMultiDrawArrays) \
	EnumMacro(PFNGLMULTIDRAWELEMENTSPROC,glMultiDrawElements) \
	EnumMacro(PFNGLPOINTPARAMETERFPROC,glPointParameterf) \
	EnumMacro(PFNGLPOINTPARAMETERFVPROC,glPointParameterfv) \
	EnumMacro(PFNGLPOINTPARAMETERIPROC,glPointParameteri) \
	EnumMacro(PFNGLPOINTPARAMETERIVPROC,glPointParameteriv) \
	EnumMacro(PFNGLGENQUERIESPROC,glGenQueries) \
	EnumMacro(PFNGLDELETEQUERIESPROC,glDeleteQueries) \
	EnumMacro(PFNGLISQUERYPROC,glIsQuery) \
	EnumMacro(PFNGLBEGINQUERYPROC,glBeginQuery) \
	EnumMacro(PFNGLENDQUERYPROC,glEndQuery) \
	EnumMacro(PFNGLGETQUERYIVPROC,glGetQueryiv) \
	EnumMacro(PFNGLGETQUERYOBJECTIVPROC,glGetQueryObjectiv) \
	EnumMacro(PFNGLGETQUERYOBJECTUIVPROC,glGetQueryObjectuiv) \
	EnumMacro(PFNGLBINDBUFFERPROC,glBindBuffer) \
	EnumMacro(PFNGLBINDBUFFERBASEPROC,glBindBufferBase) \
	EnumMacro(PFNGLDELETEBUFFERSPROC,glDeleteBuffers) \
	EnumMacro(PFNGLGENBUFFERSPROC,glGenBuffers) \
	EnumMacro(PFNGLISBUFFERPROC,glIsBuffer) \
	EnumMacro(PFNGLBUFFERDATAPROC,glBufferData) \
	EnumMacro(PFNGLBUFFERSUBDATAPROC,glBufferSubData) \
	EnumMacro(PFNGLGETBUFFERSUBDATAPROC,glGetBufferSubData) \
	EnumMacro(PFNGLMAPBUFFERPROC,glMapBuffer) \
	EnumMacro(PFNGLUNMAPBUFFERPROC,glUnmapBuffer) \
	EnumMacro(PFNGLGETBUFFERPARAMETERIVPROC,glGetBufferParameteriv) \
	EnumMacro(PFNGLGETBUFFERPOINTERVPROC,glGetBufferPointerv) \
	EnumMacro(PFNGLBLENDEQUATIONSEPARATEPROC,glBlendEquationSeparate) \
	EnumMacro(PFNGLDRAWBUFFERSPROC,glDrawBuffers) \
	EnumMacro(PFNGLSTENCILOPSEPARATEPROC,glStencilOpSeparate) \
	EnumMacro(PFNGLSTENCILFUNCSEPARATEPROC,glStencilFuncSeparate) \
	EnumMacro(PFNGLSTENCILMASKSEPARATEPROC,glStencilMaskSeparate) \
	EnumMacro(PFNGLATTACHSHADERPROC,glAttachShader) \
	EnumMacro(PFNGLBINDATTRIBLOCATIONPROC,glBindAttribLocation) \
	EnumMacro(PFNGLBINDFRAGDATALOCATIONPROC,glBindFragDataLocation) \
	EnumMacro(PFNGLCOMPILESHADERPROC,glCompileShader) \
	EnumMacro(PFNGLCREATEPROGRAMPROC,glCreateProgram) \
	EnumMacro(PFNGLCREATESHADERPROC,glCreateShader) \
	EnumMacro(PFNGLDELETEPROGRAMPROC,glDeleteProgram) \
	EnumMacro(PFNGLDELETESHADERPROC,glDeleteShader) \
	EnumMacro(PFNGLDETACHSHADERPROC,glDetachShader) \
	EnumMacro(PFNGLDISABLEVERTEXATTRIBARRAYPROC,glDisableVertexAttribArray) \
	EnumMacro(PFNGLENABLEVERTEXATTRIBARRAYPROC,glEnableVertexAttribArray) \
	EnumMacro(PFNGLGETACTIVEATTRIBPROC,glGetActiveAttrib) \
	EnumMacro(PFNGLGETACTIVEUNIFORMPROC,glGetActiveUniform) \
	EnumMacro(PFNGLGETATTACHEDSHADERSPROC,glGetAttachedShaders) \
	EnumMacro(PFNGLGETATTRIBLOCATIONPROC,glGetAttribLocation) \
	EnumMacro(PFNGLGETPROGRAMIVPROC,glGetProgramiv) \
	EnumMacro(PFNGLGETPROGRAMINFOLOGPROC,glGetProgramInfoLog) \
	EnumMacro(PFNGLGETSHADERIVPROC,glGetShaderiv) \
	EnumMacro(PFNGLGETSHADERINFOLOGPROC,glGetShaderInfoLog) \
	EnumMacro(PFNGLGETSHADERSOURCEPROC,glGetShaderSource) \
	EnumMacro(PFNGLGETUNIFORMLOCATIONPROC,glGetUniformLocation) \
	EnumMacro(PFNGLGETUNIFORMBLOCKINDEXPROC,glGetUniformBlockIndex) \
	EnumMacro(PFNGLGETUNIFORMFVPROC,glGetUniformfv) \
	EnumMacro(PFNGLGETUNIFORMIVPROC,glGetUniformiv) \
	EnumMacro(PFNGLGETVERTEXATTRIBDVPROC,glGetVertexAttribdv) \
	EnumMacro(PFNGLGETVERTEXATTRIBFVPROC,glGetVertexAttribfv) \
	EnumMacro(PFNGLGETVERTEXATTRIBIVPROC,glGetVertexAttribiv) \
	EnumMacro(PFNGLGETVERTEXATTRIBPOINTERVPROC,glGetVertexAttribPointerv) \
	EnumMacro(PFNGLISPROGRAMPROC,glIsProgram) \
	EnumMacro(PFNGLISSHADERPROC,glIsShader) \
	EnumMacro(PFNGLLINKPROGRAMPROC,glLinkProgram) \
	EnumMacro(PFNGLSHADERSOURCEPROC,glShaderSource) \
	EnumMacro(PFNGLUSEPROGRAMPROC,glUseProgram) \
	EnumMacro(PFNGLUNIFORM1FPROC,glUniform1f) \
	EnumMacro(PFNGLUNIFORM2FPROC,glUniform2f) \
	EnumMacro(PFNGLUNIFORM3FPROC,glUniform3f) \
	EnumMacro(PFNGLUNIFORM4FPROC,glUniform4f) \
	EnumMacro(PFNGLUNIFORM1IPROC,glUniform1i) \
	EnumMacro(PFNGLUNIFORM2IPROC,glUniform2i) \
	EnumMacro(PFNGLUNIFORM3IPROC,glUniform3i) \
	EnumMacro(PFNGLUNIFORM4IPROC,glUniform4i) \
	EnumMacro(PFNGLUNIFORM1FVPROC,glUniform1fv) \
	EnumMacro(PFNGLUNIFORM2FVPROC,glUniform2fv) \
	EnumMacro(PFNGLUNIFORM3FVPROC,glUniform3fv) \
	EnumMacro(PFNGLUNIFORM4FVPROC,glUniform4fv) \
	EnumMacro(PFNGLUNIFORM1IVPROC,glUniform1iv) \
	EnumMacro(PFNGLUNIFORM2IVPROC,glUniform2iv) \
	EnumMacro(PFNGLUNIFORM3IVPROC,glUniform3iv) \
	EnumMacro(PFNGLUNIFORM4IVPROC,glUniform4iv) \
	EnumMacro(PFNGLUNIFORM1UIVPROC,glUniform1uiv) \
	EnumMacro(PFNGLUNIFORM2UIVPROC,glUniform2uiv) \
	EnumMacro(PFNGLUNIFORM3UIVPROC,glUniform3uiv) \
	EnumMacro(PFNGLUNIFORM4UIVPROC,glUniform4uiv) \
	EnumMacro(PFNGLUNIFORMBLOCKBINDINGPROC,glUniformBlockBinding) \
	EnumMacro(PFNGLUNIFORMMATRIX2FVPROC,glUniformMatrix2fv) \
	EnumMacro(PFNGLUNIFORMMATRIX3FVPROC,glUniformMatrix3fv) \
	EnumMacro(PFNGLUNIFORMMATRIX4FVPROC,glUniformMatrix4fv) \
	EnumMacro(PFNGLVALIDATEPROGRAMPROC,glValidateProgram) \
	EnumMacro(PFNGLVERTEXATTRIB1DPROC,glVertexAttrib1d) \
	EnumMacro(PFNGLVERTEXATTRIB1DVPROC,glVertexAttrib1dv) \
	EnumMacro(PFNGLVERTEXATTRIB1FPROC,glVertexAttrib1f) \
	EnumMacro(PFNGLVERTEXATTRIB1FVPROC,glVertexAttrib1fv) \
	EnumMacro(PFNGLVERTEXATTRIB1SPROC,glVertexAttrib1s) \
	EnumMacro(PFNGLVERTEXATTRIB1SVPROC,glVertexAttrib1sv) \
	EnumMacro(PFNGLVERTEXATTRIB2DPROC,glVertexAttrib2d) \
	EnumMacro(PFNGLVERTEXATTRIB2DVPROC,glVertexAttrib2dv) \
	EnumMacro(PFNGLVERTEXATTRIB2FPROC,glVertexAttrib2f) \
	EnumMacro(PFNGLVERTEXATTRIB2FVPROC,glVertexAttrib2fv) \
	EnumMacro(PFNGLVERTEXATTRIB2SPROC,glVertexAttrib2s) \
	EnumMacro(PFNGLVERTEXATTRIB2SVPROC,glVertexAttrib2sv) \
	EnumMacro(PFNGLVERTEXATTRIB3DPROC,glVertexAttrib3d) \
	EnumMacro(PFNGLVERTEXATTRIB3DVPROC,glVertexAttrib3dv) \
	EnumMacro(PFNGLVERTEXATTRIB3FPROC,glVertexAttrib3f) \
	EnumMacro(PFNGLVERTEXATTRIB3FVPROC,glVertexAttrib3fv) \
	EnumMacro(PFNGLVERTEXATTRIB3SPROC,glVertexAttrib3s) \
	EnumMacro(PFNGLVERTEXATTRIB3SVPROC,glVertexAttrib3sv) \
	EnumMacro(PFNGLVERTEXATTRIB4NBVPROC,glVertexAttrib4Nbv) \
	EnumMacro(PFNGLVERTEXATTRIB4NIVPROC,glVertexAttrib4Niv) \
	EnumMacro(PFNGLVERTEXATTRIB4NSVPROC,glVertexAttrib4Nsv) \
	EnumMacro(PFNGLVERTEXATTRIB4NUBPROC,glVertexAttrib4Nub) \
	EnumMacro(PFNGLVERTEXATTRIB4NUBVPROC,glVertexAttrib4Nubv) \
	EnumMacro(PFNGLVERTEXATTRIB4NUIVPROC,glVertexAttrib4Nuiv) \
	EnumMacro(PFNGLVERTEXATTRIB4NUSVPROC,glVertexAttrib4Nusv) \
	EnumMacro(PFNGLVERTEXATTRIB4BVPROC,glVertexAttrib4bv) \
	EnumMacro(PFNGLVERTEXATTRIB4DPROC,glVertexAttrib4d) \
	EnumMacro(PFNGLVERTEXATTRIB4DVPROC,glVertexAttrib4dv) \
	EnumMacro(PFNGLVERTEXATTRIB4FPROC,glVertexAttrib4f) \
	EnumMacro(PFNGLVERTEXATTRIB4FVPROC,glVertexAttrib4fv) \
	EnumMacro(PFNGLVERTEXATTRIB4IVPROC,glVertexAttrib4iv) \
	EnumMacro(PFNGLVERTEXATTRIB4SPROC,glVertexAttrib4s) \
	EnumMacro(PFNGLVERTEXATTRIB4SVPROC,glVertexAttrib4sv) \
	EnumMacro(PFNGLVERTEXATTRIB4UBVPROC,glVertexAttrib4ubv) \
	EnumMacro(PFNGLVERTEXATTRIB4UIVPROC,glVertexAttrib4uiv) \
	EnumMacro(PFNGLVERTEXATTRIB4USVPROC,glVertexAttrib4usv) \
	EnumMacro(PFNGLVERTEXATTRIBI4IVPROC,glVertexAttribI4iv) \
	EnumMacro(PFNGLVERTEXATTRIBI4UIVPROC,glVertexAttribI4uiv) \
	EnumMacro(PFNGLVERTEXATTRIBI4SVPROC,glVertexAttribI4sv) \
	EnumMacro(PFNGLVERTEXATTRIBI4USVPROC,glVertexAttribI4usv) \
	EnumMacro(PFNGLVERTEXATTRIBI4BVPROC,glVertexAttribI4bv) \
	EnumMacro(PFNGLVERTEXATTRIBI4UBVPROC,glVertexAttribI4ubv) \
	EnumMacro(PFNGLVERTEXATTRIBPOINTERPROC,glVertexAttribPointer) \
	EnumMacro(PFNGLVERTEXATTRIBIPOINTERPROC,glVertexAttribIPointer) \
	EnumMacro(PFNGLUNIFORMMATRIX2X3FVPROC,glUniformMatrix2x3fv) \
	EnumMacro(PFNGLUNIFORMMATRIX3X2FVPROC,glUniformMatrix3x2fv) \
	EnumMacro(PFNGLUNIFORMMATRIX2X4FVPROC,glUniformMatrix2x4fv) \
	EnumMacro(PFNGLUNIFORMMATRIX4X2FVPROC,glUniformMatrix4x2fv) \
	EnumMacro(PFNGLUNIFORMMATRIX3X4FVPROC,glUniformMatrix3x4fv) \
	EnumMacro(PFNGLUNIFORMMATRIX4X3FVPROC,glUniformMatrix4x3fv) \
	EnumMacro(PFNGLISRENDERBUFFERPROC,glIsRenderbuffer) \
	EnumMacro(PFNGLBINDRENDERBUFFERPROC,glBindRenderbuffer) \
	EnumMacro(PFNGLDELETERENDERBUFFERSPROC,glDeleteRenderbuffers) \
	EnumMacro(PFNGLGENRENDERBUFFERSPROC,glGenRenderbuffers) \
	EnumMacro(PFNGLRENDERBUFFERSTORAGEPROC,glRenderbufferStorage) \
	EnumMacro(PFNGLGETRENDERBUFFERPARAMETERIVPROC,glGetRenderbufferParameteriv) \
	EnumMacro(PFNGLISFRAMEBUFFERPROC,glIsFramebuffer) \
	EnumMacro(PFNGLBINDFRAMEBUFFERPROC,glBindFramebuffer) \
	EnumMacro(PFNGLDELETEFRAMEBUFFERSPROC,glDeleteFramebuffers) \
	EnumMacro(PFNGLGENFRAMEBUFFERSPROC,glGenFramebuffers) \
	EnumMacro(PFNGLCHECKFRAMEBUFFERSTATUSPROC,glCheckFramebufferStatus) \
	EnumMacro(PFNGLFRAMEBUFFERTEXTURE1DPROC,glFramebufferTexture1D) \
	EnumMacro(PFNGLFRAMEBUFFERTEXTURE2DPROC,glFramebufferTexture2D) \
	EnumMacro(PFNGLFRAMEBUFFERTEXTURE3DPROC,glFramebufferTexture3D) \
	EnumMacro(PFNGLFRAMEBUFFERTEXTUREPROC,glFramebufferTexture) \
	EnumMacro(PFNGLFRAMEBUFFERRENDERBUFFERPROC,glFramebufferRenderbuffer) \
	EnumMacro(PFNGLGETFRAMEBUFFERATTACHMENTPARAMETERIVPROC,glGetFramebufferAttachmentParameteriv) \
	EnumMacro(PFNGLGENERATEMIPMAPPROC,glGenerateMipmap) \
	EnumMacro(PFNGLBLITFRAMEBUFFERPROC,glBlitFramebuffer) \
	EnumMacro(PFNGLRENDERBUFFERSTORAGEMULTISAMPLEPROC,glRenderbufferStorageMultisample) \
	EnumMacro(PFNGLFRAMEBUFFERTEXTURELAYERPROC,glFramebufferTextureLayer) \
	EnumMacro(PFNGLMAPBUFFERRANGEPROC,glMapBufferRange) \
	EnumMacro(PFNGLFLUSHMAPPEDBUFFERRANGEPROC,glFlushMappedBufferRange) \
	EnumMacro(PFNGLVERTEXATTRIBDIVISORPROC,glVertexAttribDivisor) \
	EnumMacro(PFNGLDRAWARRAYSINSTANCEDPROC,glDrawArraysInstanced) \
	EnumMacro(PFNGLDRAWELEMENTSINSTANCEDPROC,glDrawElementsInstanced) \
	EnumMacro(PFNGLGETSTRINGIPROC,glGetStringi) \
	EnumMacro(PFNGLGENVERTEXARRAYSPROC,glGenVertexArrays) \
	EnumMacro(PFNGLDELETEVERTEXARRAYSPROC,glDeleteVertexArrays) \
	EnumMacro(PFNGLBINDVERTEXARRAYPROC,glBindVertexArray) \
	EnumMacro(PFNGLCOPYBUFFERSUBDATAPROC,glCopyBufferSubData) \
	EnumMacro(PFNGLTEXBUFFERPROC,glTexBuffer) \
	EnumMacro(PFNGLTEXBUFFERRANGEPROC,glTexBufferRange) \
	EnumMacro(PFNGLTEXIMAGE2DMULTISAMPLEPROC,glTexImage2DMultisample) \
	EnumMacro(PFNGLQUERYCOUNTERPROC, glQueryCounter)\
	EnumMacro(PFNGLISSYNCPROC, glIsSync)\
	EnumMacro(PFNGLDELETESYNCPROC, glDeleteSync)\
	EnumMacro(PFNGLGETQUERYOBJECTUI64VPROC, glGetQueryObjectui64v)\
	EnumMacro(PFNGLFENCESYNCPROC, glFenceSync)\
	EnumMacro(PFNGLGETSYNCIVPROC, glGetSynciv)\
	EnumMacro(PFNGLCLIENTWAITSYNCPROC, glClientWaitSync)\
	EnumMacro(PFNGLBINDBUFFERRANGEPROC, glBindBufferRange) \
	EnumMacro(PFNGLGETPROGRAMRESOURCEINDEXPROC, glGetProgramResourceIndex)

#define ENUM_GL_ENTRYPOINTS_OPTIONAL(EnumMacro) \
	EnumMacro(PFNGLCLIPCONTROLPROC,glClipControl) \
	EnumMacro(PFNGLDEBUGMESSAGECALLBACKARBPROC,glDebugMessageCallbackARB) \
	EnumMacro(PFNGLDEBUGMESSAGECONTROLARBPROC,glDebugMessageControlARB) \
	EnumMacro(PFNGLDEBUGMESSAGECALLBACKAMDPROC,glDebugMessageCallbackAMD) \
	EnumMacro(PFNGLDEBUGMESSAGEENABLEAMDPROC,glDebugMessageEnableAMD) \
	EnumMacro(PFNGLGETACTIVEUNIFORMSIVPROC,glGetActiveUniformsiv) \
	EnumMacro(PFNGLGETVERTEXATTRIBIUIVPROC,glGetVertexAttribIuiv) \
	EnumMacro(PFNGLGETACTIVEUNIFORMNAMEPROC,glGetActiveUniformName) \
	EnumMacro(PFNGLGETUNIFORMUIVPROC,glGetUniformuiv) \
	EnumMacro(PFNGLGETACTIVEUNIFORMBLOCKIVPROC,glGetActiveUniformBlockiv) \
	EnumMacro(PFNGLGETBUFFERPARAMETERI64VPROC,glGetBufferParameteri64v) \
	EnumMacro(PFNGLGETACTIVEUNIFORMBLOCKNAMEPROC,glGetActiveUniformBlockName) \
	EnumMacro(PFNGLGETSAMPLERPARAMETERFVPROC,glGetSamplerParameterfv) \
	EnumMacro(PFNGLGETSAMPLERPARAMETERIVPROC,glGetSamplerParameteriv) \
	EnumMacro(PFNGLDISPATCHCOMPUTEPROC, glDispatchCompute) \
	EnumMacro(PFNGLDISPATCHCOMPUTEINDIRECTPROC, glDispatchComputeIndirect) \
	EnumMacro(PFNGLBINDIMAGETEXTUREPROC, glBindImageTexture) \
	EnumMacro(PFNGLMEMORYBARRIERPROC, glMemoryBarrier) \
	EnumMacro(PFNGLBLENDEQUATIONIPROC, glBlendEquationi) \
	EnumMacro(PFNGLBLENDEQUATIONSEPARATEIPROC, glBlendEquationSeparatei) \
	EnumMacro(PFNGLBLENDFUNCIPROC, glBlendFunci) \
	EnumMacro(PFNGLBLENDFUNCSEPARATEIPROC, glBlendFuncSeparatei)\
	EnumMacro(PFNGLCLEARBUFFERDATAPROC, glClearBufferData)\
	EnumMacro(PFNGLCLEARBUFFERSUBDATAPROC, glClearBufferSubData)\
	EnumMacro(PFNGLPUSHDEBUGGROUPPROC, glPushDebugGroup)\
	EnumMacro(PFNGLPOPDEBUGGROUPPROC, glPopDebugGroup)\
	EnumMacro(PFNGLOBJECTLABELPROC, glObjectLabel)\
	EnumMacro(PFNGLOBJECTPTRLABELPROC, glObjectPtrLabel)\
	EnumMacro(PFNGLPATCHPARAMETERIPROC, glPatchParameteri)\
	EnumMacro(PFNGLBINDVERTEXBUFFERPROC, glBindVertexBuffer)\
	EnumMacro(PFNGLVERTEXATTRIBFORMATPROC, glVertexAttribFormat)\
	EnumMacro(PFNGLVERTEXATTRIBIFORMATPROC, glVertexAttribIFormat)\
	EnumMacro(PFNGLVERTEXATTRIBBINDINGPROC, glVertexAttribBinding)\
	EnumMacro(PFNGLVERTEXBINDINGDIVISORPROC, glVertexBindingDivisor)\
	EnumMacro(PFNGLCOPYIMAGESUBDATAPROC, glCopyImageSubData)\
	EnumMacro(PFNGLTEXSTORAGE1DPROC, glTexStorage1D)\
	EnumMacro(PFNGLTEXSTORAGE2DPROC, glTexStorage2D)\
	EnumMacro(PFNGLTEXSTORAGE3DPROC, glTexStorage3D)\
	EnumMacro(PFNGLBUFFERSTORAGEPROC, glBufferStorage)\
	EnumMacro(PFNGLTEXTUREVIEWPROC, glTextureView)\
	EnumMacro(PFNGLTEXSTORAGE2DMULTISAMPLEPROC, glTexStorage2DMultisample)\
	EnumMacro(PFNGLDRAWELEMENTSINDIRECTPROC, glDrawElementsIndirect)\
	EnumMacro(PFNGLDRAWARRAYSINDIRECTPROC, glDrawArraysIndirect)\
	EnumMacro(PFNGLDEPTHBOUNDSEXTPROC, glDepthBoundsEXT)\
	EnumMacro(PFNGLPROGRAMPARAMETERIPROC, glProgramParameteri)\
	EnumMacro(PFNGLUSEPROGRAMSTAGESPROC, glUseProgramStages)\
	EnumMacro(PFNGLBINDPROGRAMPIPELINEPROC, glBindProgramPipeline)\
	EnumMacro(PFNGLDELETEPROGRAMPIPELINESPROC, glDeleteProgramPipelines)\
	EnumMacro(PFNGLGENPROGRAMPIPELINESPROC, glGenProgramPipelines)\
	EnumMacro(PFNGLPROGRAMUNIFORM1IPROC, glProgramUniform1i)\
	EnumMacro(PFNGLPROGRAMUNIFORM4IVPROC, glProgramUniform4iv)\
	EnumMacro(PFNGLPROGRAMUNIFORM4FVPROC, glProgramUniform4fv)\
	EnumMacro(PFNGLPROGRAMUNIFORM4UIVPROC, glProgramUniform4uiv)\
	EnumMacro(PFNGLGETPROGRAMPIPELINEIVPROC, glGetProgramPipelineiv)\
	EnumMacro(PFNGLVALIDATEPROGRAMPIPELINEPROC, glValidateProgramPipeline)\
	EnumMacro(PFNGLGETPROGRAMPIPELINEINFOLOGPROC, glGetProgramPipelineInfoLog)\
	EnumMacro(PFNGLISPROGRAMPIPELINEPROC, glIsProgramPipeline)\
	EnumMacro(PFNGLGETINTEGER64VPROC, glGetInteger64v)

/** List of all OpenGL entry points. */
#define ENUM_GL_ENTRYPOINTS_ALL(EnumMacro) \
	ENUM_GL_ENTRYPOINTS(EnumMacro) \
	ENUM_GL_ENTRYPOINTS_OPTIONAL(EnumMacro)

/** Declare all GL functions. */
#define DECLARE_GL_ENTRYPOINTS(Type,Func) extern Type OPENGLDRV_API Func;

// We need to make pointer names different from GL functions otherwise we may end up getting
// addresses of those symbols when looking for extensions.
namespace GLFuncPointers
{
	ENUM_GL_ENTRYPOINTS_ALL(DECLARE_GL_ENTRYPOINTS);
};

#undef DECLARE_GL_ENTRYPOINTS

// this using is needed since the rest of code uses plain GL names
using namespace GLFuncPointers;
