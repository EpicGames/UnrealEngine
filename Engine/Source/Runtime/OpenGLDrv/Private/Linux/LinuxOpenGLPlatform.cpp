// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxOpenGLPlatform.h"

#include "Misc/ScopeLock.h"
#include "OpenGLDrv.h"
#include <SDL3/SDL.h>
#include "OpenGLDrvPrivate.h"
#include "ComponentReregisterContext.h"
#include "Linux/LinuxPlatformApplicationMisc.h"
#include "GenericPlatform/GenericPlatformFramePacer.h"
#include "RHIUtilities.h"

typedef SDL_Window* SDL_HWindow;
typedef SDL_GLContext	SDL_HGLContext;

/*------------------------------------------------------------------------------
	OpenGL context management.
------------------------------------------------------------------------------*/
void Linux_ContextMakeCurrent(SDL_HWindow hWnd, SDL_HGLContext hGLDC)
{
	GLint Result = SDL_GL_MakeCurrent(hWnd, hGLDC);
	if (Result != 0)
	{
		// this is a warning and not error, since Slate sometimes destroys windows before
		// releasing RHI resources associated with them. This code can result in leaks - proper resolution is tracked as UE-7388
		FString SdlError(UTF8_TO_TCHAR(SDL_GetError()));
		UE_LOGF(LogLinux, Warning, "SDL_GL_MakeCurrent() failed, SDL error: '%ls'", *SdlError);
	}
}

SDL_HGLContext Linux_GetCurrentContext()
{
	return SDL_GL_GetCurrentContext();
}

/** Platform specific OpenGL context. */
struct FPlatformOpenGLContext
{
	SDL_HWindow		hWnd;
	SDL_HGLContext	hGLContext;		//	this is a (void*) pointer

	bool bReleaseWindowOnDestroy;
	int32 SyncInterval;
	GLuint	ViewportFramebuffer;
	GLuint	VertexArrayObject;	// one has to be generated and set for each context (OpenGL 3.2 Core requirements)
};

struct FPlatformOpenGLSurfaceContext
{
};

class FScopeContext
{
public:
	FScopeContext(FPlatformOpenGLContext* Context)
	{
		check(Context);
		hPreWnd = SDL_GL_GetCurrentWindow();
		hPreGLContext = SDL_GL_GetCurrentContext();

		bSameDCAndContext = (hPreGLContext == Context->hGLContext);

		if (!bSameDCAndContext)
		{
			if (hPreGLContext)
			{
				glFlush();
			}
			// no need to glFlush() on Windows, it does flush by itself before switching contexts

			Linux_ContextMakeCurrent(Context->hWnd, Context->hGLContext);
		}
	}

	~FScopeContext(void)
	{
		if (!bSameDCAndContext)
		{
			glFlush();	// not needed on Windows, it does flush by itself before switching contexts
			if (hPreGLContext)
			{
				Linux_ContextMakeCurrent(hPreWnd, hPreGLContext);
			}
			else
			{
				Linux_ContextMakeCurrent(NULL, NULL);
			}
		}

	}

private:
	SDL_HWindow			hPreWnd;
	SDL_HGLContext		hPreGLContext;		//	this is a pointer, (void*)
	bool				bSameDCAndContext;
};

/**
 * Create a dummy window used to construct OpenGL contexts.
 */
void Linux_PlatformCreateDummyGLWindow(FPlatformOpenGLContext* OutContext)
{
	static bool bInitializedWindowClass = false;

	// Create a dummy window.
	SDL_HWindow DummyWindow = SDL_CreateWindow(NULL,
		1, 1,
		SDL_WINDOW_OPENGL | SDL_WINDOW_BORDERLESS | SDL_WINDOW_HIDDEN | SDL_WINDOW_UTILITY );
	if (DummyWindow == nullptr)
	{
		FString SdlError(UTF8_TO_TCHAR(SDL_GetError()));

		FText ErrorMessage = FText::Format(NSLOCTEXT("Renderer", "LinuxCannotCreatePlatformWindow", "Cannot create OpenGL-enabled SDL window. SDL error: '{0}'."), FText::FromString(SdlError));
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
			*ErrorMessage.ToString(),
			*(NSLOCTEXT("Renderer", "LinuxCannotCreatePlatformWindowTitle", "Cannot create SDL window.").ToString()));
		FPlatformMisc::RequestExit(true, TEXT("Linux_PlatformCreateDummyGLWindow"));
		// unreachable
		return;
	}
	else
	{
		SDL_SetWindowTitle(DummyWindow, "UnrealEditor Dummy GL window");
	}

	OutContext->hWnd = DummyWindow;
	OutContext->bReleaseWindowOnDestroy = true;
}

/**
 * Enable/Disable debug context from the commandline
 */
bool Linux_PlatformOpenGLDebugCtx()
{
	return IsOGLDebugOutputEnabled();
}

/**
 * Create a core profile OpenGL context.
 */
void Linux_PlatformCreateOpenGLContextCore(FPlatformOpenGLContext* OutContext)
{
	check(OutContext);

	SDL_HWindow prevWindow = SDL_GL_GetCurrentWindow();
	SDL_HGLContext prevContext = SDL_GL_GetCurrentContext();

	OutContext->SyncInterval = -1;	// invalid value to enforce setup on first buffer swap
	OutContext->ViewportFramebuffer = 0;

	OutContext->hGLContext = SDL_GL_CreateContext(OutContext->hWnd);
	if (OutContext->hGLContext == nullptr)
	{
		FString SdlError(UTF8_TO_TCHAR(SDL_GetError()));

		// ignore errors getting version, it will be clear from the logs
		int OpenGLMajorVersion = -1;
		int OpenGLMinorVersion = -1;
		SDL_GL_GetAttribute(SDL_GL_CONTEXT_MAJOR_VERSION, &OpenGLMajorVersion);
		SDL_GL_GetAttribute(SDL_GL_CONTEXT_MINOR_VERSION, &OpenGLMinorVersion);

		UE_LOGF(LogInit, Error, "Linux_PlatformCreateOpenGLContextCore - Could not create OpenGL %d.%d context, SDL error: '%ls'",
			OpenGLMajorVersion, OpenGLMinorVersion,
			*SdlError
		);
		// unreachable
		return;
	}

	SDL_GL_MakeCurrent(prevWindow, prevContext);
}

void PlatformReleaseOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context);

/** Platform specific OpenGL device. */
struct FPlatformOpenGLDevice
{
	static FPlatformOpenGLDevice* Singleton;

	FPlatformOpenGLContext	RenderingContext;

	/** Guards against operating on viewport contexts from more than one thread at the same time. */
	FCriticalSection* ContextUsageGuard;

	FPlatformOpenGLDevice()
	{
		check(!Singleton);
		Singleton = this;

		extern void InitDebugContext();

		ContextUsageGuard = new FCriticalSection;

		verifyf(SDL_GL_SetAttribute(SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 0) == 0, TEXT("SDL_GL_SetAttribute(SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 0) failed: %s\n."), UTF8_TO_TCHAR(SDL_GetError()));

		Linux_PlatformCreateDummyGLWindow(&RenderingContext);
		Linux_PlatformCreateOpenGLContextCore(&RenderingContext);

		if (RenderingContext.hGLContext == nullptr)
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
				*(NSLOCTEXT("Renderer", "LinuxInsufficientDriversText", "Cannot create OpenGL context. Check that the drivers and hardware support at least OpenGL 4.3 (or re-run with -opengl3)").ToString()),
				*(NSLOCTEXT("Renderer", "LinuxInsufficientDriversTitle", "Insufficient drivers or hardware").ToString()));
			FPlatformMisc::RequestExit(true, TEXT("FPlatformOpenGLDevice()"));
			// unreachable
			return;
		}

		Linux_ContextMakeCurrent(RenderingContext.hWnd, RenderingContext.hGLContext);

		InitDebugContext();
		glGenVertexArrays(1, &RenderingContext.VertexArrayObject);
		glBindVertexArray(RenderingContext.VertexArrayObject);
		InitDefaultGLContextState();

		verifyf(SDL_GL_SetAttribute(SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 1) == 0, TEXT("SDL_GL_SetAttribute(SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 1) failed: %s\n."), UTF8_TO_TCHAR(SDL_GetError()));
	}

	~FPlatformOpenGLDevice()
	{
		Linux_ContextMakeCurrent(NULL, NULL);

		PlatformReleaseOpenGLContext(this, &RenderingContext);

		delete ContextUsageGuard;

		check(Singleton == this);
		Singleton = nullptr;
	}
};

FPlatformOpenGLDevice* FPlatformOpenGLDevice::Singleton = nullptr;

FPlatformOpenGLDevice* PlatformCreateOpenGLDevice()
{
	return new FPlatformOpenGLDevice;
}

bool PlatformCanEnableGPUCapture()
{
	return false;
}

void PlatformDestroyOpenGLDevice(FPlatformOpenGLDevice* Device)
{
	delete Device;
}

/**
 * Create an OpenGL context.
 */
FPlatformOpenGLContext* PlatformCreateOpenGLContext(FPlatformOpenGLDevice* Device, void* InWindowHandle)
{
	check(InWindowHandle);
	FPlatformOpenGLContext* Context = new FPlatformOpenGLContext;

	Context->hWnd = (SDL_HWindow)InWindowHandle;
	Context->bReleaseWindowOnDestroy = false;

	check(Device->RenderingContext.hGLContext);

	{
		FScopeContext Scope(&Device->RenderingContext);
		verifyf(SDL_GL_SetAttribute(SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 1) == 0, TEXT("SDL_GL_SetAttribute(SDL_GL_SHARE_WITH_CURRENT_CONTEXT, 1) failed: %s\n."), UTF8_TO_TCHAR(SDL_GetError()));
		Linux_PlatformCreateOpenGLContextCore(Context);
	}

	check(Context->hGLContext);
	{
		FScopeContext Scope(Context);
		InitDefaultGLContextState();
	}

	return Context;
}

struct FPlatformOpenGLSurfaceContext* PlatformCreateOpenGLSurfaceContext(FPlatformOpenGLContext* SourceContext, void* InWindowHandle)
{
	return nullptr;
}

void PlatformDestroyOpenGLSurfaceContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context, FPlatformOpenGLSurfaceContext* SurfaceContext)
{
}

/**
 * Release an OpenGL context.
 */
void PlatformReleaseOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context)
{
	check(Context && Context->hGLContext);

	{
		FScopeLock ScopeLock(Device->ContextUsageGuard);
		{
			FScopeContext ScopeContext(Context);

			glBindVertexArray(0);
			glDeleteVertexArrays(1, &Context->VertexArrayObject);

			if (Context->ViewportFramebuffer)
			{
				glDeleteFramebuffers(1, &Context->ViewportFramebuffer);	// this can be done from any context shared with ours, as long as it's not nil.
				Context->ViewportFramebuffer = 0;
			}
		}

		SDL_GL_DestroyContext(Context->hGLContext);
		Context->hGLContext = NULL;
	}

	check(Context->hWnd);

	if (Context->bReleaseWindowOnDestroy)
	{
		SDL_DestroyWindow(Context->hWnd);
	}

	Context->hWnd = NULL;
}

/**
 * Destroy an OpenGL context.
 */
void PlatformDestroyOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context)
{
	PlatformReleaseOpenGLContext(Device, Context);
	delete Context;
}

void* PlatformGetWindow(FPlatformOpenGLContext* Context, void** AddParam)
{
	check(Context && Context->hWnd);

	if (AddParam)
	{
		*AddParam = &Context->hGLContext;
	}

	return (void*)&Context->hWnd;
}

const TCHAR* PlatformDescribeSyncInterval(int32 SyncInterval)
{
	switch (SyncInterval)
	{
	case -1:
		return TEXT("Late swap");
	case 0:
		return TEXT("Immediate");
	case 1:
		return TEXT("Synchronized with retrace");
	default: break;
	}
	return TEXT("Unknown");
}

/**
 * Main function for transferring data to on-screen buffers.
 * On Windows it temporarily switches OpenGL context, on Mac only context's output view.
 */
bool PlatformBlitToViewport(IRHICommandContext& RHICmdContext, 
	FPlatformOpenGLDevice* Device,
	FOpenGLViewport& Viewport,
	uint32 BackbufferSizeX,
	uint32 BackbufferSizeY,
	bool bPresent,
	bool bLockToVsync)
{
	FPlatformOpenGLContext* const Context = Viewport.GetGLContext();

	check(Context && Context->hWnd);

	FScopeLock ScopeLock(Device->ContextUsageGuard);

	int32 SyncInterval = (int32)RHIGetSyncInterval();

	{
		FScopeContext ScopeContext(Context);

		if (Viewport.GetCustomPresent())
		{
			glDisable(GL_FRAMEBUFFER_SRGB);
			bool bShouldPresent = Viewport.GetCustomPresent()->Present(&Viewport, RHICmdContext, SyncInterval);
			glEnable(GL_FRAMEBUFFER_SRGB);
			if (!bShouldPresent)
			{
				return false;
			}
		}

		glBindFramebuffer(GL_DRAW_FRAMEBUFFER, 0);
		glDrawBuffer(GL_BACK);
		glBindFramebuffer(GL_READ_FRAMEBUFFER, Context->ViewportFramebuffer);
		glReadBuffer(GL_COLOR_ATTACHMENT0);
		glDisable(GL_FRAMEBUFFER_SRGB);

		int WinW = 0;
		int WinH = 0;
		SDL_GetWindowSizeInPixels(Context->hWnd, &WinW, &WinH);
		GLenum BlitFilter;
		GLint DestX0, DestY0, DestX1, DestY1;

		if (WinH == 0 || WinW == 0)
		{
			// Nothing to blit
			return false;
		}

		if ((WinW == BackbufferSizeX) && (WinH == BackbufferSizeY))
		{
			// We match up. We're probably in windowed mode, or an exact
			//  match for FULLSCREEN_DESKTOP mode. Use a NEAREST blit and
			//  don't clear the window system's framebuffer first.
			BlitFilter = GL_NEAREST;
			DestX0 = 0;
			DestY0 = BackbufferSizeY;  // flip vertically.
			DestX1 = BackbufferSizeX;
			DestY1 = 0;
		}
		else
		{
			// we need to scale to match the size of the window system's
			// framebuffer, so scale linearly, and adjust for letterboxing.
			BlitFilter = GL_LINEAR;

			const uint32 w = BackbufferSizeX;
			const uint32 h = BackbufferSizeY;
			const float WantedAspect = (w > h) ? (((float)w) / ((float)h)) : (((float)h) / ((float)w));
			const float PhysicalAspect = (((float)WinW) / ((float)WinH));

			bool bMustClear;  // have to clear the window framebuffer if letterboxing.
			if (PhysicalAspect == WantedAspect)  // Perfect aspect ratio; no letterboxing needed?
			{
				bMustClear = false;
				DestX0 = 0;
				DestY0 = WinH;  // flip vertically.
				DestX1 = WinW;
				DestY1 = 0;
			}
			else if (PhysicalAspect > WantedAspect)  // view is wider than wanted aspect?
			{
				bMustClear = true;
				const float ScaledW = WinH * WantedAspect;
				const float ScaledX = (((float)WinW) - ScaledW) / 2.0f;
				DestX0 = ScaledX;
				DestY0 = WinH;  // flip vertically.
				DestX1 = ScaledX + ScaledW;
				DestY1 = 0;
			}
			else  // view is taller than wanted aspect?
			{
				bMustClear = true;
				const float ScaledH = WinW / WantedAspect;
				const float ScaledY = (((float)WinH) - ScaledH) / 2.0f;
				DestX0 = 0;
				DestY0 = ScaledY + ScaledH;  // flip vertically.
				DestX1 = WinW;
				DestY1 = ScaledY;
			}

			// if the Steam Overlay is running, it might write garbage into our
			//  letterbox area, so if we have a letterbox, clear the framebuffer.
			if (bMustClear)
			{
				glClearColor(0.0f, 0.0f, 0.0f, 1.0f);
				glClear(GL_COLOR_BUFFER_BIT);
			}
		}

		// Get it to the window system's framebuffer.
		glBlitFramebuffer(0, 0, BackbufferSizeX, BackbufferSizeY,
			DestX0, DestY0, DestX1, DestY1,
			GL_COLOR_BUFFER_BIT,
			BlitFilter);

		if (bPresent)
		{
			int32 RealSyncInterval = bLockToVsync ? SyncInterval : 0;

			if (Context->SyncInterval != RealSyncInterval)
			{
				//	0 for immediate updates
				//	1 for updates synchronized with the vertical retrace
				//	-1 for late swap tearing; 

				UE_LOGF(LogLinux, Log, "Setting swap interval to '%ls'", PlatformDescribeSyncInterval(RealSyncInterval));
				int SetSwapResult = SDL_GL_SetSwapInterval(RealSyncInterval);
				// if late tearing is not supported, this needs to be retried with a valid value.
				if (SetSwapResult == -1)
				{
					if (RealSyncInterval == -1)
					{
						// fallback to synchronized
						int FallbackInterval = 1;
						UE_LOGF(LogLinux, Log, "Unable to set desired swap interval, falling back to '%ls'", PlatformDescribeSyncInterval(FallbackInterval));
						SetSwapResult = SDL_GL_SetSwapInterval(FallbackInterval);
					}
				}

				if (SetSwapResult == -1)
				{
					UE_LOGF(LogLinux, Warning, "Unable to set desired swap interval '%ls'", PlatformDescribeSyncInterval(RealSyncInterval));
				}

				// even if we failed to set it, update the context value to prevent further attempts
				Context->SyncInterval = RealSyncInterval;
			}

			SDL_GL_SwapWindow(Context->hWnd);

			glEnable(GL_FRAMEBUFFER_SRGB);
		}
	}
	return true;
}

void PlatformRenderingContextSetup(FPlatformOpenGLDevice* Device)
{
	check(Device && Device->RenderingContext.hWnd && Device->RenderingContext.hGLContext);

	if (Linux_GetCurrentContext())
	{
		glFlush();
	}

	Linux_ContextMakeCurrent(Device->RenderingContext.hWnd, Device->RenderingContext.hGLContext);
}

void PlatformNULLContextSetup()
{
	if (Linux_GetCurrentContext())
	{
		glFlush();
	}

	Linux_ContextMakeCurrent(NULL, NULL);
}

/**
 * Resize the GL context.
 */
void PlatformResizeGLContext(FPlatformOpenGLDevice* Device,
	FPlatformOpenGLContext* Context,
	struct FPlatformOpenGLSurfaceContext* SurfaceContext,
	uint32 SizeX, uint32 SizeY,
	bool bFullscreen,
	bool bWasFullscreen,
	GLenum BackBufferTarget,
	GLuint BackBufferResource)
{
	FScopeLock ScopeLock(Device->ContextUsageGuard);
	{
		FScopeContext ScopeContext(Context);

		if (Context->ViewportFramebuffer == 0)
		{
			glGenFramebuffers(1, &Context->ViewportFramebuffer);
		}
		glBindFramebuffer(GL_FRAMEBUFFER, Context->ViewportFramebuffer);
		glFramebufferTexture2D(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, BackBufferTarget, BackBufferResource, 0);
		FOpenGL::CheckFrameBuffer();

		glViewport(0, 0, SizeX, SizeY);

		static GLfloat ZeroColor[4] = { 0.0f, 0.0f, 0.0f, 0.0f };
		glClearBufferfv(GL_COLOR, 0, ZeroColor);
	}


	if (bFullscreen)
	{
		//	Deregister all components
		//	this line will fix the geometry missing and color distortion bug on Linux/Nvidia machine
		//	it detach all the components and re attach all the components
		FGlobalComponentReregisterContext RecreateComponents;
	}
	else if (bWasFullscreen)
	{
		FGlobalComponentReregisterContext RecreateComponents;
	}
}

void PlatformGetSupportedResolution(uint32& Width, uint32& Height)
{
	uint32 InitializedMode = false;
	uint32 BestWidth = 0;
	uint32 BestHeight = 0;
	uint32 ModeIndex = 0;

	SDL_DisplayID DisplayID = SDL_GetPrimaryDisplay();
	int NumModes = 0;
	SDL_DisplayMode** Modes = SDL_GetFullscreenDisplayModes(DisplayID, &NumModes);
	if (Modes)
	{
		for (int i=0; i < NumModes; ++i)
		{
			SDL_DisplayMode* Mode = Modes[i];
			bool IsEqualOrBetterWidth = FMath::Abs((int32)Mode->w - (int32)Width) <= FMath::Abs((int32)BestWidth - (int32)Width);
			bool IsEqualOrBetterHeight = FMath::Abs((int32)Mode->h - (int32)Height) <= FMath::Abs((int32)BestHeight - (int32)Height);
			if (!InitializedMode || (IsEqualOrBetterWidth && IsEqualOrBetterHeight))
			{
				BestWidth = Mode->w;
				BestHeight = Mode->h;
				InitializedMode = true;
			}

		}

		SDL_free(Modes);
	}
	check(InitializedMode);
	Width = BestWidth;
	Height = BestHeight;
}


bool PlatformGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	int32 MinAllowableResolutionX = 0;
	int32 MinAllowableResolutionY = 0;
	int32 MaxAllowableResolutionX = 10480;
	int32 MaxAllowableResolutionY = 10480;
	int32 MinAllowableRefreshRate = 0;
	int32 MaxAllowableRefreshRate = 10480;

	if (MaxAllowableResolutionX == 0)
	{
		MaxAllowableResolutionX = 10480;
	}
	if (MaxAllowableResolutionY == 0)
	{
		MaxAllowableResolutionY = 10480;
	}
	if (MaxAllowableRefreshRate == 0)
	{
		MaxAllowableRefreshRate = 10480;
	}


	SDL_DisplayID Display = SDL_GetPrimaryDisplay();
	int NumModes = 0;
	SDL_DisplayMode** Modes = SDL_GetFullscreenDisplayModes(Display, &NumModes);
	if (Modes)
	{
		for (int32 i = 0; i < NumModes; i++)
		{
			SDL_DisplayMode* Mode = Modes[i];
			if (((int32)Mode->w >= MinAllowableResolutionX) &&
				((int32)Mode->w <= MaxAllowableResolutionX) &&
				((int32)Mode->h >= MinAllowableResolutionY) &&
				((int32)Mode->h <= MaxAllowableResolutionY)
			   	)
			{
				bool bAddIt = true;
				if (bIgnoreRefreshRate == false)
				{
					if (((int32)Mode->refresh_rate < MinAllowableRefreshRate) ||
						((int32)Mode->refresh_rate > MaxAllowableRefreshRate))
					{
						continue;
					}
				}
				else
				{
					// See if it is in the list already
					for (int32 CheckIndex = 0; CheckIndex < Resolutions.Num(); CheckIndex++)
					{
						FScreenResolutionRHI& CheckResolution = Resolutions[CheckIndex];
						if ((CheckResolution.Width == Mode->w) &&
							(CheckResolution.Height == Mode->h))
						{
							// Already in the list...
							// bAddIt = false;
							break;
						}
					}
				}

				if (bAddIt)
				{
					// Add the mode to the list
					int32 Temp2Index = Resolutions.AddZeroed();
					FScreenResolutionRHI& ScreenResolution = Resolutions[Temp2Index];

					ScreenResolution.Width = Mode->w;
					ScreenResolution.Height = Mode->h;
					ScreenResolution.RefreshRate = Mode->refresh_rate;
				}
			}
		}
		SDL_free(Modes);
	}

	return true;
}

void PlatformRestoreDesktopDisplayMode()
{
}


bool PlatformInitOpenGL()
{
	static bool bInitialized = false;
	static bool bOpenGLSupported = false;

	if (!FLinuxPlatformApplicationMisc::InitSDL()) //	will not initialize more than once
	{
		UE_LOGF(LogInit, Error, "PlatformInitOpenGL() : InitSDL() failed, cannot initialize OpenGL.");
		// unreachable
		return false;
	}

#if DO_CHECK
	uint32 InitializedSubsystems = SDL_WasInit(SDL_INIT_VIDEO);
	check(InitializedSubsystems & SDL_INIT_VIDEO);
#endif // DO_CHECK

	if (!bInitialized)
	{
		if (SDL_GL_LoadLibrary(NULL))
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok,
				*FString::Printf(TEXT("%s. SDL error: \"%s\""), *(NSLOCTEXT("Renderer", "LinuxCannotLoadLibGLText", "Unable to dynamically load libGL").ToString()), UTF8_TO_TCHAR(SDL_GetError())),
				*(NSLOCTEXT("Renderer", "LinuxInsufficientDriversTitle", "Insufficient drivers or hardware").ToString()));
			FPlatformMisc::RequestExit(true, TEXT("PlatformInitOpenGL"));
			// unreachable
			return false;
		}

		const int MajorVersion = 4;
		const int MinorVersion = 3;

		int DebugFlag = 0;

		if (Linux_PlatformOpenGLDebugCtx())
		{
			DebugFlag = SDL_GL_CONTEXT_DEBUG_FLAG;
		}

		if (SDL_GL_SetAttribute(SDL_GL_CONTEXT_MAJOR_VERSION, MajorVersion) != 0)
		{
			UE_LOGF(LogLinux, Fatal, "SDL_GL_SetAttribute(SDL_GL_CONTEXT_MAJOR_VERSION, %d) failed: %ls", MajorVersion, UTF8_TO_TCHAR(SDL_GetError()));
		}

		if (SDL_GL_SetAttribute(SDL_GL_CONTEXT_MINOR_VERSION, MinorVersion) != 0)
		{
			UE_LOGF(LogLinux, Fatal, "SDL_GL_SetAttribute(SDL_GL_CONTEXT_MINOR_VERSION, %d) failed: %ls", MinorVersion, UTF8_TO_TCHAR(SDL_GetError()));
		}

		if (SDL_GL_SetAttribute(SDL_GL_CONTEXT_FLAGS, DebugFlag) != 0)
		{
			UE_LOGF(LogLinux, Fatal, "SDL_GL_SetAttribute(SDL_GL_CONTEXT_FLAGS, %d) failed: %ls", DebugFlag, UTF8_TO_TCHAR(SDL_GetError()));
		}

		if (SDL_GL_SetAttribute(SDL_GL_CONTEXT_PROFILE_MASK, SDL_GL_CONTEXT_PROFILE_CORE) != 0)
		{
			UE_LOGF(LogLinux, Fatal, "SDL_GL_SetAttribute(SDL_GL_CONTEXT_PROFILE_MASK, SDL_GL_CONTEXT_PROFILE_CORE) failed: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("quad_buffer_stereo")))
		{
			if (SDL_GL_SetAttribute(SDL_GL_STEREO, 1) != 0)
			{
				UE_LOGF(LogLinux, Fatal, "SDL_GL_SetAttribute(SDL_GL_STEREO, 1) failed: %ls", UTF8_TO_TCHAR(SDL_GetError()));
			}
		}

		// Create a dummy context to verify opengl support.
		FPlatformOpenGLContext DummyContext;
		Linux_PlatformCreateDummyGLWindow(&DummyContext);
		Linux_PlatformCreateOpenGLContextCore(&DummyContext);

		if (DummyContext.hGLContext)
		{
			bOpenGLSupported = true;
			Linux_ContextMakeCurrent(DummyContext.hWnd, DummyContext.hGLContext);
		}
		else
		{
			UE_LOGF(LogRHI, Error, "OpenGL %d.%d not supported by driver", MajorVersion, MinorVersion);
		}

		if (bOpenGLSupported)
		{
			// Initialize all entry points required by Unreal.
#define GET_GL_ENTRYPOINTS(Type,Func) GLFuncPointers::Func = reinterpret_cast<Type>(SDL_GL_GetProcAddress(#Func));
			ENUM_GL_ENTRYPOINTS(GET_GL_ENTRYPOINTS);
			ENUM_GL_ENTRYPOINTS_OPTIONAL(GET_GL_ENTRYPOINTS);
#undef GET_GL_ENTRYPOINTS

			// Check that all of the entry points have been initialized.
			bool bFoundAllEntryPoints = true;
#define CHECK_GL_ENTRYPOINTS(Type,Func) if (Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Fatal, "Failed to find entry point for %ls", TEXT(#Func)); }
			ENUM_GL_ENTRYPOINTS(CHECK_GL_ENTRYPOINTS);
#undef CHECK_GL_ENTRYPOINTS
			checkf(bFoundAllEntryPoints, TEXT("Failed to find all OpenGL entry points."));
		}

		// The dummy context can now be released.
		if (DummyContext.hGLContext)
		{
			Linux_ContextMakeCurrent(NULL, NULL);
			SDL_GL_DestroyContext(DummyContext.hGLContext);
		}
		check(DummyContext.bReleaseWindowOnDestroy);
		SDL_DestroyWindow(DummyContext.hWnd);

		bInitialized = true;
	}

	return bOpenGLSupported;
}

int32 PlatformGlGetError()
{
	return glGetError();
}

bool PlatformOpenGLThreadHasRenderingContext()
{
	return Linux_GetCurrentContext() == FPlatformOpenGLDevice::Singleton->RenderingContext.hGLContext;
}

// =============================================================

FOpenGLTexture* PlatformCreateBuiltinBackBuffer(FOpenGLDynamicRHI* OpenGLRHI, FPlatformOpenGLSurfaceContext* SurfaceContext, uint32 SizeX, uint32 SizeY)
{
	return nullptr;
}
