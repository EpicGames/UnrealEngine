// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenGLThirdParty.h"

#define UGL_PLATFORM_SUPPORTS_GLES 0
#define UGL_PLATFORM_SUPPORTS_GL3  1
#define UGL_PLATFORM_SUPPORTS_GL4  1

/** Unreal tokens that maps to different OpenGL tokens by platform. */
#define UGL_DRAW_FRAMEBUFFER	GL_DRAW_FRAMEBUFFER
#define UGL_READ_FRAMEBUFFER	GL_READ_FRAMEBUFFER
#define UGL_ABGR8				GL_UNSIGNED_INT_8_8_8_8_REV
#define UGL_ANY_SAMPLES_PASSED	GL_ANY_SAMPLES_PASSED
#define UGL_SAMPLES_PASSED		GL_SAMPLES_PASSED
#define UGL_TIME_ELAPSED		GL_TIME_ELAPSED
#define UGL_CLAMP_TO_BORDER		GL_CLAMP_TO_BORDER
