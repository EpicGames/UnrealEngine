// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGL.h: Public OpenGL base definitions for non-common functionality
=============================================================================*/
#pragma once

#include "HAL/PreprocessorHelpers.h"

#include COMPILED_PLATFORM_HEADER(OpenGL.h)