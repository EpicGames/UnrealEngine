// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGL3.cpp: OpenGL 3.2 implementation.
=============================================================================*/

#include "OpenGL3.h"

#if UGL_PLATFORM_SUPPORTS_GL3

#include "OpenGLDrvPrivate.h"

bool FOpenGL3::bAndroidGLESCompatibilityMode = false;

GLsizei FOpenGL3::NextTextureName = OPENGL_NAME_CACHE_SIZE;
GLuint FOpenGL3::TextureNamesCache[OPENGL_NAME_CACHE_SIZE];
GLsizei FOpenGL3::NextBufferName= OPENGL_NAME_CACHE_SIZE;
GLuint FOpenGL3::BufferNamesCache[OPENGL_NAME_CACHE_SIZE];
GLsizei FOpenGL3::NextPipelineName = OPENGL_NAME_CACHE_SIZE;
GLuint FOpenGL3::PipelineNamesCache[OPENGL_NAME_CACHE_SIZE];

GLint FOpenGL3::TimestampQueryBits = 0;
bool FOpenGL3::bDebugContext = false;

void FOpenGL3::ProcessQueryGLInt()
{
	GET_GL_INT(GL_MAX_VERTEX_UNIFORM_COMPONENTS, 0, MaxVertexUniformComponents);
	GET_GL_INT(GL_MAX_FRAGMENT_UNIFORM_COMPONENTS, 0, MaxPixelUniformComponents);
	GET_GL_INT(GL_MAX_GEOMETRY_UNIFORM_COMPONENTS, 0, MaxGeometryUniformComponents);

	GET_GL_INT(GL_MAX_GEOMETRY_TEXTURE_IMAGE_UNITS, 0, MaxGeometryTextureImageUnits);
	GET_GL_INT(GL_MAX_VARYING_VECTORS, 0, MaxVaryingVectors);

#if !defined(__GNUC__) && !defined(__clang__)
#define LOG_AND_GET_GL_QUERY_INT(IntEnum, Default, Dest) do { if (IntEnum) {glGetQueryiv(IntEnum, GL_QUERY_COUNTER_BITS, &Dest);} else {Dest = Default;} /*FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" GL_QUERY_COUNTER_BITS: ") ## TEXT(#IntEnum) ## TEXT(": %d"), Dest);*/ } while(0)
#else
#define LOG_AND_GET_GL_QUERY_INT(IntEnum, Default, Dest) do { if (IntEnum) {glGetQueryiv(IntEnum, GL_QUERY_COUNTER_BITS, &Dest);} else {Dest = Default;} /*FPlatformMisc::LowLevelOutputDebugStringf(TEXT(" GL_QUERY_COUNTER_BITS: " #IntEnum ": %d"), Dest);*/ } while(0)
#endif
	LOG_AND_GET_GL_QUERY_INT(GL_TIMESTAMP, 0, TimestampQueryBits);
	
#undef LOG_AND_GET_GL_QUERY_INT
}

void FOpenGL3::ProcessExtensions( const FString& ExtensionsString )
{
	int32 MajorVersion = 0;
	int32 MinorVersion = 0;

	FString Version = ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VERSION));
	FString MajorString, MinorString;
	if (Version.Split(TEXT("."), &MajorString, &MinorString))
	{
		MajorVersion = FCString::Atoi(*MajorString);
		MinorVersion = FCString::Atoi(*MinorString);
	}
	check(MajorVersion!=0);

	ProcessQueryGLInt();
	FOpenGLBase::ProcessExtensions(ExtensionsString);

	// Test whether the GPU can support volume-texture rendering.
	// There is no API to query this - you just have to test whether a 3D texture is framebuffer-complete.
	{
		GLuint FrameBuffer;
		glGenFramebuffers(1, &FrameBuffer);
		glBindFramebuffer(GL_DRAW_FRAMEBUFFER, FrameBuffer);
		GLuint VolumeTexture;
		glGenTextures(1, &VolumeTexture);
		glBindTexture(GL_TEXTURE_3D, VolumeTexture);
		glTexImage3D(GL_TEXTURE_3D, 0, GL_RGBA8, 256, 256, 256, 0, GL_RGBA, GL_UNSIGNED_BYTE, NULL);
		glFramebufferTexture(GL_DRAW_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, VolumeTexture, 0);
		
		bSupportsVolumeTextureRendering = (glCheckFramebufferStatus(GL_DRAW_FRAMEBUFFER) == GL_FRAMEBUFFER_COMPLETE);
		
		glDeleteTextures(1, &VolumeTexture);
		glDeleteFramebuffers(1, &FrameBuffer);
	}

	bAndroidGLESCompatibilityMode = GetFeatureLevel() == ERHIFeatureLevel::ES3_1 && ExtensionsString.Contains(TEXT("GL_ARB_ES3_1_compatibility")) && FParse::Param(FCommandLine::Get(), TEXT("GLESCompat"));
}

#endif // UGL_PLATFORM_SUPPORTS_GL3
