// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGL3.h: Public OpenGL 3.2 definitions for non-common functionality
=============================================================================*/

#pragma once

#include "OpenGLPlatform.h"

#if UGL_PLATFORM_SUPPORTS_GL3

#include "Containers/Map.h"
#include "OpenGLBase.h"

struct FPlatformOpenGLContext;
struct FPlatformOpenGLSurfaceContext;
struct FPlatformOpenGLDevice;

#define USE_OPENGL_NAME_CACHE 1
#define OPENGL_NAME_CACHE_SIZE 1024

struct FOpenGL3 : public FOpenGLBase
{
	static FORCEINLINE bool IsDebugContent()						{ return bDebugContext; }

	static FORCEINLINE bool SupportsTimestampQueries()				{ return TimestampQueryBits > 0; }
	static FORCEINLINE bool SupportsSeamlessCubeMap()				{ return bSupportsSeamlessCubemap; }
	static FORCEINLINE bool SupportsVolumeTextureRendering()		{ return bSupportsVolumeTextureRendering; }
	static FORCEINLINE bool AmdWorkaround()							{ return bAmdWorkaround; }
	static FORCEINLINE bool SupportsBufferStorage()					{ return true; }

	// Optional
	static FORCEINLINE void QueryTimestampCounter(GLuint QueryID)
	{
		glQueryCounter(QueryID, GL_TIMESTAMP);
	}

	static FORCEINLINE void BeginQuery(GLenum QueryType, GLuint QueryId)
	{
		glBeginQuery( QueryType, QueryId );
	}

	static FORCEINLINE void EndQuery(GLenum QueryType)
	{
		glEndQuery( QueryType );
	}

	static FORCEINLINE void GetQueryObject(GLuint QueryId, EQueryMode QueryMode, GLuint64* OutResult)
	{
		GLenum QueryName = (QueryMode == QM_Result) ? GL_QUERY_RESULT : GL_QUERY_RESULT_AVAILABLE;
		GLuint64 Result = 0;
		glGetQueryObjectui64v(QueryId, QueryName, &Result);
		*OutResult = Result;
	}

	static FORCEINLINE void ReadBuffer(GLenum Mode)
	{
		glReadBuffer( Mode );
	}

	static FORCEINLINE void DrawBuffer(GLenum Mode)
	{
		glDrawBuffer( Mode );
	}

	static FORCEINLINE void DeleteSync(UGLsync Sync)
	{
		glDeleteSync( Sync );
	}

	static FORCEINLINE UGLsync FenceSync(GLenum Condition, GLbitfield Flags)
	{
		return glFenceSync( Condition, Flags );
	}

	static FORCEINLINE bool IsSync(UGLsync Sync)
	{
		return (glIsSync( Sync ) == GL_TRUE) ? true : false;
	}

	static FORCEINLINE EFenceResult ClientWaitSync(UGLsync Sync, GLbitfield Flags, GLuint64 Timeout)
	{
		GLenum Result = glClientWaitSync( Sync, Flags, Timeout );
		switch (Result)
		{
			case GL_ALREADY_SIGNALED:		return FR_AlreadySignaled;
			case GL_TIMEOUT_EXPIRED:		return FR_TimeoutExpired;
			case GL_CONDITION_SATISFIED:	return FR_ConditionSatisfied;
		}
		return FR_WaitFailed;
	}

	static FORCEINLINE void GenSamplers(GLsizei Count, GLuint* Samplers)
	{
		glGenSamplers(Count, Samplers);
	}

	static FORCEINLINE void DeleteSamplers(GLsizei Count, GLuint* Samplers)
	{
		glDeleteSamplers(Count, Samplers);
	}

	static FORCEINLINE void SetSamplerParameter(GLuint Sampler, GLenum Parameter, GLint Value)
	{
		glSamplerParameteri(Sampler, Parameter, Value);
	}

	static FORCEINLINE void BindSampler(GLuint Unit, GLuint Sampler)
	{
		glBindSampler(Unit, Sampler);
	}

	static FORCEINLINE void PolygonMode(GLenum Face, GLenum Mode)
	{
		glPolygonMode(Face, Mode);
	}

	static FORCEINLINE void VertexAttribDivisor(GLuint Index, GLuint Divisor)
	{
		glVertexAttribDivisor(Index, Divisor);
	}

	// Required
	static FORCEINLINE void* MapBufferRange(GLenum Type, uint32 InOffset, uint32 InSize, EResourceLockMode LockMode)
	{
		GLenum Access;
		switch ( LockMode )
		{
			case EResourceLockMode::RLM_ReadOnly:
				Access = GL_MAP_READ_BIT;
				break;
			case EResourceLockMode::RLM_ReadOnlyPersistent:
				Access = (GL_MAP_READ_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT);
				break;
			case EResourceLockMode::RLM_WriteOnly:
				Access = (GL_MAP_INVALIDATE_BUFFER_BIT | GL_MAP_WRITE_BIT);
#if 1
				// Temp workaround for synchrnoization when a UBO is discarded while being referenced
				Access |= GL_MAP_UNSYNCHRONIZED_BIT;
#endif
				break;
			case EResourceLockMode::RLM_WriteOnlyUnsynchronized:
				Access = (GL_MAP_WRITE_BIT | GL_MAP_UNSYNCHRONIZED_BIT);
				break;
			case EResourceLockMode::RLM_WriteOnlyPersistent:
				Access = (GL_MAP_WRITE_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT);
				break;
			case EResourceLockMode::RLM_ReadWrite:
			default:
				Access = (GL_MAP_READ_BIT | GL_MAP_WRITE_BIT);
		}
		return glMapBufferRange(Type, InOffset, InSize, Access);
	}

	static FORCEINLINE void UnmapBuffer(GLenum Type)
	{
		glUnmapBuffer(Type);
	}

	static FORCEINLINE void UnmapBufferRange(GLenum Type, uint32 InOffset, uint32 InSize)
	{
		UnmapBuffer(Type);
	}

	static FORCEINLINE void GenQueries(GLsizei NumQueries, GLuint* QueryIDs)
	{
		glGenQueries(NumQueries, QueryIDs);
	}

	static FORCEINLINE void DeleteQueries(GLsizei NumQueries, const GLuint* QueryIDs )
	{
		glDeleteQueries( NumQueries, QueryIDs );
	}

	static FORCEINLINE void GetQueryObject(GLuint QueryId, EQueryMode QueryMode, GLuint* OutResult)
	{
		GLenum QueryName = (QueryMode == QM_Result) ? GL_QUERY_RESULT : GL_QUERY_RESULT_AVAILABLE;
		glGetQueryObjectuiv(QueryId, QueryName, OutResult);
	}

	static FORCEINLINE void BindBufferBase(GLenum Target, GLuint Index, GLuint Buffer)
	{
		glBindBufferBase(Target, Index, Buffer);
	}

	static FORCEINLINE void BindBufferRange(GLenum Target, GLuint Index, GLuint Buffer, GLintptr Offset, GLsizeiptr Size)
	{
		glBindBufferRange(Target, Index, Buffer, Offset, Size);
	}

	static FORCEINLINE GLuint GetUniformBlockIndex(GLuint Program, const GLchar* UniformBlockName)
	{
		return glGetUniformBlockIndex(Program, UniformBlockName);
	}

	static FORCEINLINE void UniformBlockBinding(GLuint Program, GLuint UniformBlockIndex, GLuint UniformBlockBinding)
	{
		glUniformBlockBinding(Program, UniformBlockIndex, UniformBlockBinding);
	}

	static FORCEINLINE void BindFragDataLocation(GLuint Program, GLuint Color, const GLchar* Name)
	{
		glBindFragDataLocation(Program, Color, Name);
	}

	static FORCEINLINE void TexParameter(GLenum Target, GLenum Parameter, GLint Value)
	{
		glTexParameteri(Target, Parameter, Value);
	}

	static FORCEINLINE void FramebufferTexture(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level)
	{
		glFramebufferTexture(Target, Attachment, Texture, Level);
	}

	static FORCEINLINE void FramebufferTexture3D(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level, GLint ZOffset)
	{
		glFramebufferTexture3D(Target, Attachment, TexTarget, Texture, Level, ZOffset);
	}

	static FORCEINLINE void FramebufferTextureLayer(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLint Layer)
	{
		glFramebufferTextureLayer(Target, Attachment, Texture, Level, Layer);
	}

	static FORCEINLINE void Uniform4uiv(GLint Location, GLsizei Count, const GLuint* Value)
	{
		glUniform4uiv(Location, Count, Value);
	}

	static FORCEINLINE void BlitFramebuffer(GLint SrcX0, GLint SrcY0, GLint SrcX1, GLint SrcY1, GLint DstX0, GLint DstY0, GLint DstX1, GLint DstY1, GLbitfield Mask, GLenum Filter)
	{
		glBlitFramebuffer(SrcX0, SrcY0, SrcX1, SrcY1, DstX0, DstY0, DstX1, DstY1, Mask, Filter);
	}

	static FORCEINLINE void DrawBuffers(GLsizei NumBuffers, const GLenum* Buffers)
	{
		glDrawBuffers(NumBuffers, Buffers);
	}

	static FORCEINLINE void DepthRange(GLdouble Near, GLdouble Far)
	{
		glDepthRange(Near, Far);
	}

	static FORCEINLINE void EnableIndexed(GLenum Parameter, GLuint Index)
	{
		glEnablei(Parameter, Index);
	}

	static FORCEINLINE void DisableIndexed(GLenum Parameter, GLuint Index)
	{
		glDisablei(Parameter, Index);
	}

	static FORCEINLINE void ColorMaskIndexed(GLuint Index, GLboolean Red, GLboolean Green, GLboolean Blue, GLboolean Alpha)
	{
		glColorMaski(Index, Red, Green, Blue, Alpha);
	}

	static FORCEINLINE void VertexAttribPointer(GLuint Index, GLint Size, GLenum Type, GLboolean Normalized, GLsizei Stride, const GLvoid* Pointer)
	{
		glVertexAttribPointer(Index, Size, Type, Normalized, Stride, Pointer);
	}

	static FORCEINLINE void VertexAttribIPointer(GLuint Index, GLint Size, GLenum Type, GLsizei Stride, const GLvoid* Pointer)
	{
		glVertexAttribIPointer(Index, Size, Type, Stride, Pointer);
	}

	static FORCEINLINE void VertexAttrib4Nsv(GLuint AttributeIndex, const GLshort* Values)
	{
		glVertexAttrib4Nsv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttrib4sv(GLuint AttributeIndex, const GLshort* Values)
	{
		glVertexAttrib4sv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttribI4sv(GLuint AttributeIndex, const GLshort* Values)
	{
		glVertexAttribI4sv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttribI4usv(GLuint AttributeIndex, const GLushort* Values)
	{
		glVertexAttribI4usv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttrib4Nubv(GLuint AttributeIndex, const GLubyte* Values)
	{
		glVertexAttrib4Nubv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttrib4ubv(GLuint AttributeIndex, const GLubyte* Values)
	{
		glVertexAttrib4ubv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttribI4ubv(GLuint AttributeIndex, const GLubyte* Values)
	{
		glVertexAttribI4ubv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttrib4Nbv(GLuint AttributeIndex, const GLbyte* Values)
	{
		glVertexAttrib4Nbv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttrib4bv(GLuint AttributeIndex, const GLbyte* Values)
	{
		glVertexAttrib4bv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttribI4bv(GLuint AttributeIndex, const GLbyte* Values)
	{
		glVertexAttribI4bv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttrib4dv(GLuint AttributeIndex, const GLdouble* Values)
	{
		glVertexAttrib4dv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttribI4iv(GLuint AttributeIndex, const GLint* Values)
	{
		glVertexAttribI4iv(AttributeIndex, Values);
	}

	static FORCEINLINE void VertexAttribI4uiv(GLuint AttributeIndex, const GLuint* Values)
	{
		glVertexAttribI4uiv(AttributeIndex, Values);
	}

	static FORCEINLINE void DrawArraysInstanced(GLenum Mode, GLint First, GLsizei Count, GLsizei InstanceCount)
	{
		glDrawArraysInstanced(Mode, First, Count, InstanceCount);
	}

	static FORCEINLINE void DrawElementsInstanced(GLenum Mode, GLsizei Count, GLenum Type, const GLvoid* Indices, GLsizei InstanceCount)
	{
		glDrawElementsInstanced(Mode, Count, Type, Indices, InstanceCount);
	}

	static FORCEINLINE void DrawRangeElements(GLenum Mode, GLuint Start, GLuint End, GLsizei Count, GLenum Type, const GLvoid* Indices)
	{
		glDrawRangeElements(Mode, Start, End, Count, Type, Indices);
	}

	static FORCEINLINE void ClearBufferfv(GLenum Buffer, GLint DrawBufferIndex, const GLfloat* Value)
	{
		glClearBufferfv(Buffer, DrawBufferIndex, Value);
	}

	static FORCEINLINE void ClearBufferfi(GLenum Buffer, GLint DrawBufferIndex, GLfloat Depth, GLint Stencil)
	{
		glClearBufferfi(Buffer, DrawBufferIndex, Depth, Stencil);
	}

	static FORCEINLINE void ClearBufferiv(GLenum Buffer, GLint DrawBufferIndex, const GLint* Value)
	{
		glClearBufferiv(Buffer, DrawBufferIndex, Value);
	}

	static FORCEINLINE void ClearDepth(GLdouble Depth)
	{
		glClearDepth(Depth);
	}

	static FORCEINLINE void TexImage3D(GLenum Target, GLint Level, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLint Border, GLenum Format, GLenum Type, const GLvoid* PixelData)
	{
		glTexImage3D(Target, Level, InternalFormat, Width, Height, Depth, Border, Format, Type, PixelData);
	}

	static FORCEINLINE void CompressedTexImage3D(GLenum Target, GLint Level, GLenum InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLint Border, GLsizei ImageSize, const GLvoid* PixelData)
	{
		glCompressedTexImage3D(Target, Level, InternalFormat, Width, Height, Depth, Border, ImageSize, PixelData);
	}

	static FORCEINLINE void CompressedTexSubImage2D(GLenum Target, GLint Level, GLsizei Width, GLsizei Height, GLenum Format, GLsizei ImageSize, const GLvoid* PixelData)
	{
		glCompressedTexSubImage2D(Target, Level, 0, 0, Width, Height, Format, ImageSize, PixelData);
	}

	static FORCEINLINE void TexImage2DMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height, GLboolean FixedSampleLocations)
	{
		glTexImage2DMultisample(Target, Samples, InternalFormat, Width, Height, FixedSampleLocations);
	}

	static FORCEINLINE void TexBuffer(GLenum Target, GLenum InternalFormat, GLuint Buffer)
	{
		glTexBuffer(Target, InternalFormat, Buffer);
	}

	static FORCEINLINE void TexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type, const GLvoid* PixelData)
	{
		glTexSubImage3D(Target, Level, XOffset, YOffset, ZOffset, Width, Height, Depth, Format, Type, PixelData);
	}

	static FORCEINLINE void	CopyTexSubImage1D(GLenum Target, GLint Level, GLint XOffset, GLint X, GLint Y, GLsizei Width)
	{
		glCopyTexSubImage1D(Target, Level, XOffset, X, Y, Width);
	}

	static FORCEINLINE void	CopyTexSubImage2D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint X, GLint Y, GLsizei Width, GLsizei Height)
	{
		glCopyTexSubImage2D(Target, Level, XOffset, YOffset, X, Y, Width, Height);
	}

	static FORCEINLINE void	CopyTexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLint X, GLint Y, GLsizei Width, GLsizei Height)
	{
		glCopyTexSubImage3D(Target, Level, XOffset, YOffset, ZOffset, X, Y, Width, Height);
	}

	static FORCEINLINE void GetCompressedTexImage(GLenum Target, GLint Level, GLvoid* OutImageData)
	{
		glGetCompressedTexImage(Target, Level, OutImageData);
	}

	static FORCEINLINE void GetTexImage(GLenum Target, GLint Level, GLenum Format, GLenum Type, GLvoid* OutPixelData)
	{
		glGetTexImage(Target, Level, Format, Type, OutPixelData);
	}

	static FORCEINLINE void CopyBufferSubData(GLenum ReadTarget, GLenum WriteTarget, GLintptr ReadOffset, GLintptr WriteOffset, GLsizeiptr Size)
	{
		glCopyBufferSubData(ReadTarget, WriteTarget, ReadOffset, WriteOffset, Size);
	}
	
	static FORCEINLINE GLuint CreateShader(GLenum Type)
	{
#if USE_OPENGL_NAME_CACHE
		static TMap<GLenum, TArray<GLuint>> ShaderNames;
		TArray<GLuint>& Shaders = ShaderNames.FindOrAdd(Type);
		if(!Shaders.Num())
		{
			while(Shaders.Num() < OPENGL_NAME_CACHE_SIZE)
			{
				GLuint Resource = glCreateShader(Type);
				Shaders.Add(Resource);
			}
		}
		return Shaders.Pop();
#else
		return glCreateShader(Type);
#endif
	}
	
	static FORCEINLINE GLuint CreateProgram()
	{
#if USE_OPENGL_NAME_CACHE
		static TArray<GLuint> ProgramNames;
		if(!ProgramNames.Num())
		{
			while(ProgramNames.Num() < OPENGL_NAME_CACHE_SIZE)
			{
				GLuint Resource = glCreateProgram();
				ProgramNames.Add(Resource);
			}
		}
		return ProgramNames.Pop();
#else
		return glCreateProgram();
#endif
	}

	static FORCEINLINE void GenBuffers( GLsizei n, GLuint *buffers)
	{
#if USE_OPENGL_NAME_CACHE
		if ( n < OPENGL_NAME_CACHE_SIZE - NextBufferName)
		{
			FMemory::Memcpy( buffers, &BufferNamesCache[NextBufferName], sizeof(GLuint)*n);
			NextBufferName += n;
		}
		else
		{
			if ( n >= OPENGL_NAME_CACHE_SIZE)
			{
				glGenBuffers( n, buffers);
			}
			else
			{
				GLsizei Leftover = OPENGL_NAME_CACHE_SIZE - NextBufferName;

				FMemory::Memcpy( buffers, &BufferNamesCache[NextBufferName], sizeof(GLuint)*Leftover);

				glGenBuffers( OPENGL_NAME_CACHE_SIZE, BufferNamesCache);

				n -= Leftover;
				buffers += Leftover;

				FMemory::Memcpy( buffers, BufferNamesCache, sizeof(GLuint)*n);
				NextBufferName = n;
			}
		}
#else
		glGenBuffers( n, buffers);
#endif
	}

	static FORCEINLINE void GenTextures( GLsizei n, GLuint *textures)
	{
#if USE_OPENGL_NAME_CACHE
		if ( n < OPENGL_NAME_CACHE_SIZE - NextTextureName)
		{
			FMemory::Memcpy( textures, &TextureNamesCache[NextTextureName], sizeof(GLuint)*n);
			NextTextureName += n;
		}
		else
		{
			if ( n >= OPENGL_NAME_CACHE_SIZE)
			{
				glGenTextures( n, textures);
			}
			else
			{
				GLsizei Leftover = OPENGL_NAME_CACHE_SIZE - NextTextureName;

				FMemory::Memcpy( textures, &TextureNamesCache[NextTextureName], sizeof(GLuint)*Leftover);

				glGenTextures( OPENGL_NAME_CACHE_SIZE, TextureNamesCache);

				n -= Leftover;
				textures += Leftover;

				FMemory::Memcpy( textures, TextureNamesCache, sizeof(GLuint)*n);
				NextTextureName = n;
			}
		}
#else
		glGenTextures( n, textures);
#endif
	}
	
	static FORCEINLINE void CompressedTexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLsizei ImageSize, const GLvoid* PixelData)
	{
		glCompressedTexSubImage3D( Target, Level, XOffset, YOffset, ZOffset, Width, Height, Depth, Format, ImageSize, PixelData);
	}

	static FORCEINLINE void GenerateMipmap( GLenum Target )
	{
		glGenerateMipmap( Target);
	}

	static FORCEINLINE const ANSICHAR* GetStringIndexed(GLenum Name, GLuint Index)
	{
		return (const ANSICHAR*)glGetStringi(Name, Index);
	}

	static FORCEINLINE GLuint GetMajorVersion()
	{
		GLint MajorVersion = 0;
		glGetIntegerv(GL_MAJOR_VERSION, &MajorVersion);
		return MajorVersion;
	}

	static FORCEINLINE GLuint GetMinorVersion()
	{
		GLint MinorVersion = 0;
		glGetIntegerv(GL_MINOR_VERSION, &MinorVersion);
		return MinorVersion;
	}
	static FORCEINLINE void ProgramParameter(GLuint Program, GLenum PName, GLint Value)
	{
		check(PName != GL_PROGRAM_SEPARABLE);
		glProgramParameteri(Program, PName, Value);
	}
	static FORCEINLINE void ProgramUniform1i(GLuint Program, GLint Location, GLint V0)
	{
		glUniform1i(Location, V0);
	}
	static FORCEINLINE void ProgramUniform4iv(GLuint Program, GLint Location, GLsizei Count, const GLint *Value)
	{
		glUniform4iv(Location, Count, Value);
	}
	static FORCEINLINE void ProgramUniform4fv(GLuint Program, GLint Location, GLsizei Count, const GLfloat *Value)
	{
		glUniform4fv(Location, Count, Value);
	}
	static FORCEINLINE void ProgramUniform4uiv(GLuint Program, GLint Location, GLsizei Count, const GLuint *Value)
	{
		glUniform4uiv(Location, Count, Value);
	}
	static FORCEINLINE void GetProgramPipelineiv(GLuint Pipeline, GLenum Pname, GLint* Params)
	{
		glGetProgramPipelineiv(Pipeline, Pname, Params);
	}
	static FORCEINLINE void ValidateProgramPipeline(GLuint Pipeline)
	{
		glValidateProgramPipeline(Pipeline);
	}
	static FORCEINLINE void GetProgramPipelineInfoLog(GLuint Pipeline, GLsizei BufSize, GLsizei* Length, GLchar* InfoLog)
	{
		glGetProgramPipelineInfoLog(Pipeline, BufSize, Length, InfoLog);
	}
	static FORCEINLINE bool IsProgramPipeline(GLuint Pipeline)
	{
		return (glIsProgramPipeline(Pipeline) == GL_TRUE);
	}

	static FORCEINLINE ERHIFeatureLevel::Type GetFeatureLevel()
	{
		return ERHIFeatureLevel::ES3_1;
	}

	static FORCEINLINE EShaderPlatform GetShaderPlatform()
	{
		return bAndroidGLESCompatibilityMode ? SP_OPENGL_ES3_1_ANDROID : SP_OPENGL_PCES3_1;
	}

	static FORCEINLINE FString GetAdapterName()
	{
		return ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_RENDERER));
	}

	static FPlatformOpenGLDevice*	CreateDevice()	UGL_REQUIRED(NULL)
	static FPlatformOpenGLContext*	CreateContext( FPlatformOpenGLDevice* Device, void* WindowHandle )	UGL_REQUIRED(NULL)

	static void ProcessQueryGLInt();
	static void ProcessExtensions(const FString& ExtensionsString);
	
	static FORCEINLINE int32 GetReadHalfFloatPixelsEnum() { return GL_HALF_FLOAT; }

	static FORCEINLINE bool IsAndroidGLESCompatibilityModeEnabled() { return bAndroidGLESCompatibilityMode; }

	static GLenum GetPlatfrom5551Format() { return GL_UNSIGNED_SHORT_1_5_5_5_REV; }
protected:
	static GLsizei NextTextureName;
	static GLuint TextureNamesCache[OPENGL_NAME_CACHE_SIZE];
	static GLsizei NextBufferName;
	static GLuint BufferNamesCache[OPENGL_NAME_CACHE_SIZE];
	static GLsizei NextPipelineName;
	static GLuint PipelineNamesCache[OPENGL_NAME_CACHE_SIZE];

	static GLint TimestampQueryBits;
	
	static bool bDebugContext;
	static bool bAndroidGLESCompatibilityMode;
};

#endif // UGL_PLATFORM_SUPPORTS_GL3
