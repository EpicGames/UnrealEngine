// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGL4.cpp: OpenGL 4.3 implementation.
=============================================================================*/

#include "OpenGL4.h"

#if UGL_PLATFORM_SUPPORTS_GL4

#include "OpenGLDrvPrivate.h"

GLint FOpenGL4::MaxComputeUniformComponents = -1;
GLint FOpenGL4::MaxCombinedUAVUnits = 0;
GLint FOpenGL4::MaxComputeUAVUnits = -1;
GLint FOpenGL4::MaxPixelUAVUnits = -1;

bool FOpenGL4::bSupportsGPUMemoryInfo = false;

void FOpenGL4::ProcessQueryGLInt()
{
	GET_GL_INT(GL_MAX_COMBINED_IMAGE_UNIFORMS, 0, MaxCombinedUAVUnits);
	GET_GL_INT(GL_MAX_FRAGMENT_IMAGE_UNIFORMS, 0, MaxPixelUAVUnits);
	GET_GL_INT(GL_TEXTURE_BUFFER_OFFSET_ALIGNMENT, 0, TextureBufferAlignment);

	GET_GL_INT(GL_MAX_COMPUTE_UNIFORM_COMPONENTS, 0, MaxComputeUniformComponents);
	GET_GL_INT(GL_MAX_COMPUTE_IMAGE_UNIFORMS, 0, MaxComputeUAVUnits);
	
	GLint MaxCombinedSSBOUnits = 0;
	GET_GL_INT(GL_MAX_COMBINED_SHADER_STORAGE_BLOCKS, 0, MaxCombinedSSBOUnits);
	// UAVs slots in UE are shared between Images and SSBO, so this should be max(GL_MAX_COMBINED_IMAGE_UNIFORMS, GL_MAX_COMBINED_SHADER_STORAGE_BLOCKS)
	MaxCombinedUAVUnits = FMath::Max(MaxCombinedUAVUnits, MaxCombinedSSBOUnits);
	
	// clamp UAV units to a sensible limit
	MaxCombinedUAVUnits = FMath::Min(MaxCombinedUAVUnits, 16);
	MaxComputeUAVUnits = FMath::Min(MaxComputeUAVUnits, 16);
	// this is split between VS and PS, 4 to each stage
	MaxPixelUAVUnits = FMath::Min(MaxPixelUAVUnits, 4);
}

void FOpenGL4::ProcessExtensions( const FString& ExtensionsString )
{
    int32 MajorVersion =0;
    int32 MinorVersion =0;
 
	FString Version = ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VERSION));
	FString MajorString, MinorString;
	if (Version.Split(TEXT("."), &MajorString, &MinorString))
	{
		MajorVersion = FCString::Atoi(*MajorString);
		MinorVersion = FCString::Atoi(*MinorString);
	}
	check(MajorVersion!=0);


	bSupportsGPUMemoryInfo = ExtensionsString.Contains(TEXT("GL_NVX_gpu_memory_info"));

	//Process Queries after extensions to avoid queries that use functionality that might not be present
	ProcessQueryGLInt();

	FOpenGL3::ProcessExtensions(ExtensionsString);
}

#ifndef GL_NVX_gpu_memory_info
#define GL_NVX_gpu_memory_info 1
#define GL_GPU_MEMORY_INFO_DEDICATED_VIDMEM_NVX          0x9047
#define GL_GPU_MEMORY_INFO_TOTAL_AVAILABLE_MEMORY_NVX    0x9048
#define GL_GPU_MEMORY_INFO_CURRENT_AVAILABLE_VIDMEM_NVX  0x9049
#define GL_GPU_MEMORY_INFO_EVICTION_COUNT_NVX            0x904A
#define GL_GPU_MEMORY_INFO_EVICTED_MEMORY_NVX            0x904B
#endif

uint64 FOpenGL4::GetVideoMemorySize()
{
	uint64 VideoMemorySize = 0;

	if (bSupportsGPUMemoryInfo)
	{
		GLint VMSizeKB = 0;

		glGetIntegerv( GL_GPU_MEMORY_INFO_DEDICATED_VIDMEM_NVX, &VMSizeKB);

		VideoMemorySize = VMSizeKB * 1024ll;
	}

	return VideoMemorySize;
}

static FORCEINLINE void BindVertexBuffer(GLuint BindingIndex, GLuint Buffer, GLintptr Offset, GLsizei Stride)
{
	glBindVertexBuffer(BindingIndex, Buffer, Offset, Stride);
}

static FORCEINLINE void VertexAttribFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLboolean Normalized, GLuint RelativeOffset)
{
	glVertexAttribFormat(AttribIndex, Size, Type, Normalized, RelativeOffset);
}

static FORCEINLINE void VertexAttribIFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLuint RelativeOffset)
{
	glVertexAttribIFormat(AttribIndex, Size, Type, RelativeOffset);
}

static FORCEINLINE void VertexAttribBinding(GLuint AttribIndex, GLuint BindingIndex)
{
	glVertexAttribBinding(AttribIndex, BindingIndex);
}

static FORCEINLINE void VertexBindingDivisor(GLuint BindingIndex, GLuint Divisor)
{
	glVertexBindingDivisor(BindingIndex, Divisor);
}

#endif
