// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGL4.h: Public OpenGL 4.3 definitions for non-common functionality
=============================================================================*/

#pragma once

#include "OpenGLPlatform.h"

#if UGL_PLATFORM_SUPPORTS_GL4

#include "UObject/UObjectHierarchyFwd.h"
#include "Misc/AssertionMacros.h"
#include "OpenGL3.h"

struct FOpenGL4 : public FOpenGL3
{
	static FORCEINLINE bool SupportsDrawIndirect()						{ return true; }

	// Optional

	// Required
	static FORCEINLINE void BlendFuncSeparatei(GLuint Buf, GLenum SrcRGB, GLenum DstRGB, GLenum SrcAlpha, GLenum DstAlpha)
	{
		glBlendFuncSeparatei(Buf, SrcRGB, DstRGB, SrcAlpha, DstAlpha);
	}
	static FORCEINLINE void BlendEquationSeparatei(GLuint Buf, GLenum ModeRGB, GLenum ModeAlpha)
	{
		glBlendEquationSeparatei(Buf, ModeRGB, ModeAlpha);
	}
	static FORCEINLINE void BlendFunci(GLuint Buf, GLenum Src, GLenum Dst)
	{
		glBlendFunci(Buf, Src, Dst);
	}
	static FORCEINLINE void BlendEquationi(GLuint Buf, GLenum Mode)
	{
		glBlendEquationi(Buf, Mode);
	}
	static FORCEINLINE void PatchParameteri(GLenum Pname, GLint Value)
	{
		glPatchParameteri(Pname, Value);
	}
	static FORCEINLINE void BindImageTexture(GLuint Unit, GLuint Texture, GLint Level, GLboolean Layered, GLint Layer, GLenum Access, GLenum Format)
	{
		glBindImageTexture(Unit, Texture, Level, Layered, Layer, Access, Format);
	}
	static FORCEINLINE void DispatchCompute(GLuint NumGroupsX, GLuint NumGroupsY, GLuint NumGroupsZ)
	{
		glDispatchCompute(NumGroupsX, NumGroupsY, NumGroupsZ);
	}
	static FORCEINLINE void DispatchComputeIndirect(GLintptr Offset)
	{
		glDispatchComputeIndirect(Offset);
	}
	static FORCEINLINE void MemoryBarrier(GLbitfield Barriers)
	{
		glMemoryBarrier(Barriers);
	}
	static FORCEINLINE void DrawArraysIndirect (GLenum Mode, const void *Offset)
	{
		glDrawArraysIndirect( Mode, Offset);
	}
	static FORCEINLINE void DrawElementsIndirect (GLenum Mode, GLenum Type, const void *Offset)
	{
		glDrawElementsIndirect( Mode, Type, Offset);
	}
	static FORCEINLINE void BindVertexBuffer(GLuint BindingIndex, GLuint Buffer, GLintptr Offset, GLsizei Stride)
	{
		glBindVertexBuffer(BindingIndex, Buffer, Offset, Stride);
	}
	static FORCEINLINE void VertexAttribFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLboolean Normalized, GLuint RelativeOffset)
	{
		glVertexAttribFormat(AttribIndex, Size, Type, Normalized, RelativeOffset);
	}
	static FORCEINLINE void VertexAttribIFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLuint RelativeOffset)
	{
		glVertexAttribIFormat(AttribIndex, Size, Type, RelativeOffset);
	}
	static FORCEINLINE void VertexAttribBinding(GLuint AttribIndex, GLuint BindingIndex)
	{
		glVertexAttribBinding(AttribIndex, BindingIndex);
	}
	static FORCEINLINE void VertexBindingDivisor(GLuint BindingIndex, GLuint Divisor)
	{
		glVertexBindingDivisor(BindingIndex, Divisor);
	}
	static FORCEINLINE void TexBufferRange(GLenum Target, GLenum InternalFormat, GLuint Buffer, GLintptr Offset, GLsizeiptr Size)
	{
		glTexBufferRange(Target, InternalFormat, Buffer, Offset, Size);
	}
	static FORCEINLINE void TextureView(GLuint ViewName, GLenum ViewTarget, GLuint SrcName, GLenum InternalFormat, GLuint MinLevel, GLuint NumLevels, GLuint MinLayer, GLuint NumLayers)
	{
		glTextureView(ViewName, ViewTarget, SrcName, InternalFormat, MinLevel, NumLevels, MinLayer, NumLayers);
	}
	static FORCEINLINE void ClearBufferData(GLenum Target, GLenum InternalFormat, GLenum Format, GLenum Type, const uint32* Data)
	{
		glClearBufferData(Target, InternalFormat, Format, Type, Data);
	}

	static void ProcessQueryGLInt();
	static void ProcessExtensions( const FString& ExtensionsString );

	static FORCEINLINE GLint GetMaxComputeUniformComponents() { check(MaxComputeUniformComponents != -1); return MaxComputeUniformComponents; }

	static FORCEINLINE GLint GetFirstComputeUAVUnit()			{ return 0; }
	static FORCEINLINE GLint GetMaxComputeUAVUnits()			{ check(MaxComputeUAVUnits != -1); return MaxComputeUAVUnits; }
	static FORCEINLINE GLint GetFirstVertexUAVUnit()			{ return 0; }
	static FORCEINLINE GLint GetFirstPixelUAVUnit()				{ return 0; }
	static FORCEINLINE GLint GetMaxPixelUAVUnits()				{ check(MaxPixelUAVUnits != -1); return MaxPixelUAVUnits; }
	static FORCEINLINE GLint GetMaxCombinedUAVUnits()			{ return MaxCombinedUAVUnits; }
		
	static uint64 GetVideoMemorySize();

	static FORCEINLINE int32 GetReadHalfFloatPixelsEnum() { return GL_HALF_FLOAT; }

protected:
	static GLint MaxComputeUniformComponents;

	static GLint MaxCombinedUAVUnits;
	static GLint MaxComputeUAVUnits;
	static GLint MaxPixelUAVUnits;

	static bool bSupportsGPUMemoryInfo;
};

#endif // UGL_PLATFORM_SUPPORTS_GL4
