// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Logging/LogMacros.h"
#include "OpenGLFunctions.h"
#include "OpenGLPlatform.h"
#include "PixelFormat.h"
#include "RHIFeatureLevel.h"
#include "RHIShaderPlatform.h"

struct FPlatformOpenGLContext;
struct FPlatformOpenGLDevice;

/** OpenGL Logging. */
OPENGLDRV_API DECLARE_LOG_CATEGORY_EXTERN(LogOpenGL, Log, VeryVerbose);

#define UGL_REQUIRED_VOID			{ UE_LOGF(LogOpenGL,Fatal,"%ls is not supported.", ANSI_TO_TCHAR(__FUNCTION__)); }
#define UGL_REQUIRED(ReturnValue)	{ UE_LOGF(LogOpenGL,Fatal,"%ls is not supported.", ANSI_TO_TCHAR(__FUNCTION__)); return (ReturnValue); }
#define UGL_OPTIONAL_VOID			{ }
#define UGL_OPTIONAL(ReturnValue)	{ return (ReturnValue); }

#define UGL_SUPPORTS_PIXELBUFFERS		1
#define UGL_SUPPORTS_UNIFORMBUFFERS		1

// for platform extensions: from OpenGLShaders.h
extern "C" struct FOpenGLShaderDeviceCapabilities;

// for platform extensions: from OpenGLResources.h
typedef TArray<ANSICHAR> FAnsiCharArray;

// for platform extensions: from OpenGLDrvPrivate.h
extern "C" struct FOpenGLTextureFormat;

// Base static class
class FOpenGLBase
{
public:
	enum class EResourceLockMode : uint8
	{
		RLM_ReadWrite,
		RLM_ReadOnly,
		RLM_WriteOnly,
		RLM_WriteOnlyUnsynchronized,
		RLM_WriteOnlyPersistent,
		RLM_ReadOnlyPersistent,
	};

	enum EQueryMode
	{
		QM_Result,
		QM_ResultAvailable,
	};

	enum EFenceResult
	{
		FR_AlreadySignaled,
		FR_TimeoutExpired,
		FR_ConditionSatisfied,
		FR_WaitFailed,
	};

	static void ProcessQueryGLInt();
	static void ProcessExtensions(const FString& ExtensionsString);
	static void SetupDefaultGLContextState(const FString& ExtensionsString) {};

	static FORCEINLINE bool SupportsUniformBuffers()					{ return true; }
	static FORCEINLINE bool SupportsStructuredBuffers()					{ return true; }
	static FORCEINLINE bool SupportsTimestampQueries()					{ return true; }
	static FORCEINLINE bool SupportsDisjointTimeQueries()				{ return true; }
	static FORCEINLINE bool SupportsExactOcclusionQueries()				{ return true; }
	static FORCEINLINE bool SupportsDepthStencilReadSurface()			{ return true; }
	static FORCEINLINE bool SupportsFloatReadSurface()					{ return true; }
	static FORCEINLINE bool SupportsWideMRT()							{ return true; }
	static FORCEINLINE bool SupportsPolygonMode()						{ return true; }
	static FORCEINLINE bool SupportsTexture3D()							{ return true; }
	static FORCEINLINE bool SupportsMobileMultiView()					{ return false; }
	static FORCEINLINE bool SupportsImageExternal()						{ return false; }
	static FORCEINLINE bool SupportsTextureLODBias()					{ return true; }
	static FORCEINLINE bool SupportsTextureCompare()					{ return true; }
	static FORCEINLINE bool SupportsDrawIndexOffset()					{ return true; }
	static FORCEINLINE bool SupportsDiscardFrameBuffer()				{ return false; }
	static FORCEINLINE bool SupportsIndexedExtensions()					{ return true; }
	static FORCEINLINE bool SupportsColorBufferFloat()					{ return true; }
	static FORCEINLINE bool SupportsColorBufferHalfFloat()				{ return true; }
	static FORCEINLINE bool SupportsVolumeTextureRendering()			{ return false; }
	static FORCEINLINE bool SupportsShaderFramebufferFetch()			{ return false; }
	static FORCEINLINE bool SupportsShaderFramebufferFetchProgrammableBlending() { return false; }
	static FORCEINLINE bool SupportsShaderMRTFramebufferFetch()			{ return false; }
	static FORCEINLINE bool SupportsShaderDepthStencilFetch()			{ return false; }
	static FORCEINLINE bool SupportsPixelLocalStorage()					{ return false; }
	static FORCEINLINE bool SupportsVertexArrayBGRA()					{ return true; }
	static FORCEINLINE bool SupportsBGRA8888()							{ return true; }
	static FORCEINLINE bool SupportsDXT()								{ return true; }
	static FORCEINLINE bool SupportsASTC()								{ return bSupportsASTC; }
	static FORCEINLINE bool SupportsASTCHDR()							{ return bSupportsASTCHDR; }
	static FORCEINLINE bool SupportsETC2()								{ return false; }
	static FORCEINLINE bool SupportsFramebufferSRGBEnable()				{ return true; }
	static FORCEINLINE bool SupportsFastBufferData()					{ return true; }
	static FORCEINLINE bool SupportsTextureFilterAnisotropic()			{ return bSupportsTextureFilterAnisotropic; }
	static FORCEINLINE bool SupportsSeparateAlphaBlend()				{ return bSupportsDrawBuffersBlend; }
	static FORCEINLINE void EnableSupportsClipControl()					{ bSupportsClipControl = true; }
	static FORCEINLINE bool SupportsClipControl()						{ return bSupportsClipControl; }
	static FORCEINLINE bool SupportsSeamlessCubeMap()					{ return false; }
	static FORCEINLINE bool SupportsDrawIndirect()						{ return false; }
	static FORCEINLINE bool SupportsBufferStorage()						{ return false; }
	static FORCEINLINE bool SupportsDepthBoundsTest()					{ return false; }
	static FORCEINLINE bool SupportsTextureRange()						{ return false; }
	static FORCEINLINE bool HasHardwareHiddenSurfaceRemoval()			{ return false; }
	static FORCEINLINE bool AmdWorkaround()								{ return false; }
	static FORCEINLINE bool SupportsProgramBinary()						{ return false; }
	static FORCEINLINE bool SupportsDepthClamp()						{ return true; }
	
	static FORCEINLINE bool SupportsASTCDecodeMode()					{ return false; }

	static FORCEINLINE GLenum GetDepthFormat()							{ return GL_DEPTH_COMPONENT16; }
	static FORCEINLINE GLenum GetShadowDepthFormat()					{ return GL_DEPTH_COMPONENT16; }

	static FORCEINLINE GLint GetMaxTextureImageUnits()			{ check(MaxTextureImageUnits != -1); return MaxTextureImageUnits; }
	static FORCEINLINE GLint GetMaxVertexTextureImageUnits()	{ check(MaxVertexTextureImageUnits != -1); return MaxVertexTextureImageUnits; }
	static FORCEINLINE GLint GetMaxGeometryTextureImageUnits()	{ check(MaxGeometryTextureImageUnits != -1); return MaxGeometryTextureImageUnits; }
	static FORCEINLINE GLint GetMaxComputeTextureImageUnits()	{ check(MaxComputeTextureImageUnits != -1); return MaxComputeTextureImageUnits; }
	static FORCEINLINE GLint GetMaxCombinedTextureImageUnits()	{ check(MaxCombinedTextureImageUnits != -1); return MaxCombinedTextureImageUnits; }
	static FORCEINLINE GLint GetTextureBufferAlignment()		{ return TextureBufferAlignment; }


	// Indices per unit are set in this order [Pixel, Vertex, Geometry]
	static FORCEINLINE GLint GetFirstPixelTextureUnit()			{ return 0; }
	static FORCEINLINE GLint GetFirstVertexTextureUnit()		{ return GetFirstPixelTextureUnit() + GetMaxTextureImageUnits(); }
	static FORCEINLINE GLint GetFirstGeometryTextureUnit()		{ return GetFirstVertexTextureUnit() + GetMaxVertexTextureImageUnits(); }

	static FORCEINLINE GLint GetFirstComputeTextureUnit()		{ return 0; }
	
	// Image load/store units
	static FORCEINLINE GLint GetFirstComputeUAVUnit()			{ return 0; }
	static FORCEINLINE GLint GetMaxComputeUAVUnits()			{ return 0; }
	static FORCEINLINE GLint GetFirstVertexUAVUnit()			{ return 0; }
	static FORCEINLINE GLint GetFirstPixelUAVUnit()				{ return 0; }
	static FORCEINLINE GLint GetMaxPixelUAVUnits()				{ return 0; }
	static FORCEINLINE GLint GetMaxCombinedUAVUnits()			{ return 0; }
	
	static FORCEINLINE GLint GetMaxVaryingVectors()				{ check(MaxVaryingVectors != -1); return MaxVaryingVectors; }
	static FORCEINLINE GLint GetMaxPixelUniformComponents()		{ check(MaxPixelUniformComponents != -1); return MaxPixelUniformComponents; }
	static FORCEINLINE GLint GetMaxVertexUniformComponents()	{ check(MaxVertexUniformComponents != -1); return MaxVertexUniformComponents; }
	static FORCEINLINE GLint GetMaxGeometryUniformComponents()	{ check(MaxGeometryUniformComponents != -1); return MaxGeometryUniformComponents; }
	static FORCEINLINE GLint GetMaxComputeUniformComponents()	{ return 0; }

	static FORCEINLINE uint64 GetVideoMemorySize()				{ return 0; }

	static FORCEINLINE bool IsDebugContent()					{ return false; }
	static FORCEINLINE void InitDebugContext()					{ }

	static FORCEINLINE int32 GetReadHalfFloatPixelsEnum() UGL_REQUIRED(0)

	static FORCEINLINE GLint GetMaxMSAASamplesTileMem()			{ return 0; /* not supported */ }

	// Silently ignored if not implemented:
	static FORCEINLINE void QueryTimestampCounter(GLuint QueryID) UGL_OPTIONAL_VOID
	static FORCEINLINE void BeginQuery(GLenum QueryType, GLuint QueryId) UGL_OPTIONAL_VOID
	static FORCEINLINE void EndQuery(GLenum QueryType) UGL_OPTIONAL_VOID
	static FORCEINLINE void GetQueryObject(GLuint QueryId, EQueryMode QueryMode, GLuint64 *OutResult) UGL_OPTIONAL_VOID
	static FORCEINLINE void BindFragDataLocation(GLuint Program, GLuint Color, const GLchar *Name) UGL_OPTIONAL_VOID
	static FORCEINLINE void ReadBuffer(GLenum Mode) UGL_OPTIONAL_VOID
	static FORCEINLINE void DrawBuffer(GLenum Mode) UGL_OPTIONAL_VOID
	static FORCEINLINE void DeleteSync(UGLsync Sync) UGL_OPTIONAL_VOID
	static FORCEINLINE UGLsync FenceSync(GLenum Condition, GLbitfield Flags) UGL_OPTIONAL(UGLsync())
	static FORCEINLINE bool IsSync(UGLsync Sync) UGL_OPTIONAL(false)
	static FORCEINLINE EFenceResult ClientWaitSync(UGLsync Sync, GLbitfield Flags, GLuint64 Timeout) UGL_OPTIONAL(FR_WaitFailed)
	static FORCEINLINE void GenSamplers(GLsizei Count, GLuint *Samplers) UGL_OPTIONAL_VOID
	static FORCEINLINE void DeleteSamplers(GLsizei Count, GLuint *Samplers) UGL_OPTIONAL_VOID
	static FORCEINLINE void SetSamplerParameter(GLuint Sampler, GLenum Parameter, GLint Value) UGL_OPTIONAL_VOID
	static FORCEINLINE void BindSampler(GLuint Unit, GLuint Sampler) UGL_OPTIONAL_VOID
	static FORCEINLINE void PolygonMode(GLenum Face, GLenum Mode) UGL_OPTIONAL_VOID
	static FORCEINLINE void VertexAttribDivisor(GLuint Index, GLuint Divisor) UGL_OPTIONAL_VOID
	static FORCEINLINE void PushGroupMarker(const ANSICHAR* Name) UGL_OPTIONAL_VOID
	static FORCEINLINE void PopGroupMarker() UGL_OPTIONAL_VOID
	static FORCEINLINE void LabelObject(GLenum Type, GLuint Object, const ANSICHAR* Name) UGL_OPTIONAL_VOID
	static FORCEINLINE GLsizei GetLabelObject(GLenum Type, GLuint Object, GLsizei BufferSize, ANSICHAR* OutName) UGL_OPTIONAL(0)
	static FORCEINLINE void InvalidateFramebuffer(GLenum Target, GLsizei NumAttachments, const GLenum* Attachments) UGL_OPTIONAL_VOID

	// Will assert at run-time if not implemented:
	static FORCEINLINE void* MapBufferRange(GLenum Type, uint32 InOffset, uint32 InSize, EResourceLockMode LockMode) UGL_REQUIRED(NULL)
	static FORCEINLINE void UnmapBufferRange(GLenum Type, uint32 InOffset, uint32 InSize) UGL_REQUIRED_VOID
	static FORCEINLINE void UnmapBuffer(GLenum Type) UGL_REQUIRED_VOID
	static FORCEINLINE void GenQueries(GLsizei NumQueries, GLuint* QueryIDs) UGL_REQUIRED_VOID
	static FORCEINLINE void DeleteQueries(GLsizei NumQueries, const GLuint* QueryIDs) UGL_REQUIRED_VOID
	static FORCEINLINE void GetQueryObject(GLuint QueryId, EQueryMode QueryMode, GLuint *OutResult) UGL_REQUIRED_VOID
	static FORCEINLINE void BindBufferBase(GLenum Target, GLuint Index, GLuint Buffer) UGL_REQUIRED_VOID
	static FORCEINLINE void BindBufferRange(GLenum Target, GLuint Index, GLuint Buffer, GLintptr Offset, GLsizeiptr Size) UGL_REQUIRED_VOID
	static FORCEINLINE GLuint GetUniformBlockIndex(GLuint Program, const GLchar *UniformBlockName) UGL_REQUIRED(-1)
	static FORCEINLINE void UniformBlockBinding(GLuint Program, GLuint UniformBlockIndex, GLuint UniformBlockBinding) UGL_REQUIRED_VOID
	static FORCEINLINE void Uniform4uiv(GLint Location, GLsizei Count, const GLuint* Value) UGL_REQUIRED_VOID
	static FORCEINLINE void TexParameter(GLenum Target, GLenum Parameter, GLint Value) UGL_REQUIRED_VOID
	static FORCEINLINE void FramebufferTexture(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level) UGL_REQUIRED_VOID
	static FORCEINLINE void FramebufferTexture2D(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level)
	{
		glFramebufferTexture2D(Target, Attachment, TexTarget, Texture, Level);
	}
	static FORCEINLINE void FramebufferTexture2DMultisample(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level, GLint NumSamples) UGL_REQUIRED_VOID
	static FORCEINLINE void FramebufferTexture3D(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level, GLint ZOffset) UGL_REQUIRED_VOID
	static FORCEINLINE void FramebufferTextureLayer(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLint Layer) UGL_REQUIRED_VOID
	static FORCEINLINE void FramebufferRenderbuffer(GLenum Target, GLenum Attachment, GLenum RenderBufferTarget, GLuint RenderBuffer)
	{
		glFramebufferRenderbuffer(Target, Attachment, RenderBufferTarget, RenderBuffer);
	}
	static FORCEINLINE void FramebufferTextureMultiviewOVR(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLint BaseViewIndex, GLsizei NumViews) UGL_REQUIRED_VOID
	static FORCEINLINE void FramebufferTextureMultisampleMultiviewOVR(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLsizei NumSamples, GLint BaseViewIndex, GLsizei NumViews) UGL_REQUIRED_VOID
	static FORCEINLINE void BlitFramebuffer(GLint SrcX0, GLint SrcY0, GLint SrcX1, GLint SrcY1, GLint DstX0, GLint DstY0, GLint DstX1, GLint DstY1, GLbitfield Mask, GLenum Filter) UGL_REQUIRED_VOID
	static FORCEINLINE void DrawBuffers(GLsizei NumBuffers, const GLenum *Buffers) UGL_REQUIRED_VOID
	static FORCEINLINE void DepthRange(GLdouble Near, GLdouble Far) UGL_REQUIRED_VOID
	static FORCEINLINE void EnableIndexed(GLenum Parameter, GLuint Index) UGL_REQUIRED_VOID
	static FORCEINLINE void DisableIndexed(GLenum Parameter, GLuint Index) UGL_REQUIRED_VOID
	static FORCEINLINE void ColorMaskIndexed(GLuint Index, GLboolean Red, GLboolean Green, GLboolean Blue, GLboolean Alpha) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribPointer(GLuint Index, GLint Size, GLenum Type, GLboolean Normalized, GLsizei Stride, const GLvoid* Pointer) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribIPointer(GLuint Index, GLint Size, GLenum Type, GLsizei Stride, const GLvoid* Pointer) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4Nsv(GLuint AttributeIndex, const GLshort* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4sv(GLuint AttributeIndex, const GLshort* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribI4sv(GLuint AttributeIndex, const GLshort* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribI4usv(GLuint AttributeIndex, const GLushort* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4Nubv(GLuint AttributeIndex, const GLubyte* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4ubv(GLuint AttributeIndex, const GLubyte* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribI4ubv(GLuint AttributeIndex, const GLubyte* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4Nbv(GLuint AttributeIndex, const GLbyte* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4bv(GLuint AttributeIndex, const GLbyte* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribI4bv(GLuint AttributeIndex, const GLbyte* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttrib4dv(GLuint AttributeIndex, const GLdouble* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribI4iv(GLuint AttributeIndex, const GLint* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribI4uiv(GLuint AttributeIndex, const GLuint* Values) UGL_REQUIRED_VOID
	static FORCEINLINE void DrawArraysInstanced(GLenum Mode, GLint First, GLsizei Count, GLsizei InstanceCount) UGL_REQUIRED_VOID
	static FORCEINLINE void DrawElementsInstanced(GLenum Mode, GLsizei Count, GLenum Type, const GLvoid* Indices, GLsizei InstanceCount) UGL_REQUIRED_VOID
	static FORCEINLINE void DrawRangeElements(GLenum Mode, GLuint Start, GLuint End, GLsizei Count, GLenum Type, const GLvoid* Indices) UGL_REQUIRED_VOID
	static FORCEINLINE void ClearBufferfv(GLenum Buffer, GLint DrawBufferIndex, const GLfloat* Value) UGL_REQUIRED_VOID
	static FORCEINLINE void ClearBufferfi(GLenum Buffer, GLint DrawBufferIndex, GLfloat Depth, GLint Stencil) UGL_REQUIRED_VOID
	static FORCEINLINE void ClearBufferiv(GLenum Buffer, GLint DrawBufferIndex, const GLint* Value) UGL_REQUIRED_VOID
	static FORCEINLINE void ClearDepth(GLdouble Depth) UGL_REQUIRED_VOID
	static FORCEINLINE void TexImage3D(GLenum Target, GLint Level, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLint Border, GLenum Format, GLenum Type, const GLvoid* PixelData) UGL_REQUIRED_VOID
	static FORCEINLINE void CompressedTexImage3D(GLenum Target, GLint Level, GLenum InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLint Border, GLsizei ImageSize, const GLvoid* PixelData) UGL_REQUIRED_VOID
	static FORCEINLINE void TexImage2DMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height, GLboolean FixedSampleLocations) UGL_REQUIRED_VOID
	static FORCEINLINE void TexBuffer(GLenum Target, GLenum InternalFormat, GLuint Buffer) UGL_REQUIRED_VOID
	static FORCEINLINE void TexBufferRange(GLenum Target, GLenum InternalFormat, GLuint Buffer, GLintptr Offset, GLsizeiptr Size) UGL_REQUIRED_VOID
	static FORCEINLINE void TexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type, const GLvoid* PixelData) UGL_REQUIRED_VOID
	static FORCEINLINE void	CopyTexSubImage2D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint X, GLint Y, GLsizei Width, GLsizei Height) UGL_REQUIRED_VOID
	static FORCEINLINE void	CopyTexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLint X, GLint Y, GLsizei Width, GLsizei Height) UGL_REQUIRED_VOID
	static FORCEINLINE void GetCompressedTexImage(GLenum Target, GLint Level, GLvoid* OutImageData) UGL_REQUIRED_VOID
	static FORCEINLINE void GetTexImage(GLenum Target, GLint Level, GLenum Format, GLenum Type, GLvoid* OutPixelData) UGL_REQUIRED_VOID
	static FORCEINLINE void CopyBufferSubData(GLenum ReadTarget, GLenum WriteTarget, GLintptr ReadOffset, GLintptr WriteOffset, GLsizeiptr Size) UGL_REQUIRED_VOID
	static FORCEINLINE const ANSICHAR* GetStringIndexed(GLenum Name, GLuint Index) UGL_REQUIRED(NULL)
	static FORCEINLINE GLuint GetMajorVersion() UGL_REQUIRED(0)
	static FORCEINLINE GLuint GetMinorVersion() UGL_REQUIRED(0)
	static FORCEINLINE ERHIFeatureLevel::Type GetFeatureLevel() UGL_REQUIRED(ERHIFeatureLevel::SM5)
	static FORCEINLINE EShaderPlatform GetShaderPlatform() UGL_REQUIRED(SP_NumPlatforms)
	static FORCEINLINE FString GetAdapterName() UGL_REQUIRED(TEXT(""))
	static FORCEINLINE void BlendFuncSeparatei(GLuint Buf, GLenum SrcRGB, GLenum DstRGB, GLenum SrcAlpha, GLenum DstAlpha) UGL_REQUIRED_VOID
	static FORCEINLINE void BlendEquationSeparatei(GLuint Buf, GLenum ModeRGB, GLenum ModeAlpha) UGL_REQUIRED_VOID
	static FORCEINLINE void BlendFunci(GLuint Buf, GLenum Src, GLenum Dst) UGL_REQUIRED_VOID
	static FORCEINLINE void BlendEquationi(GLuint Buf, GLenum Mode) UGL_REQUIRED_VOID
	static FORCEINLINE void PatchParameteri(GLenum Pname, GLint Value) UGL_REQUIRED_VOID
	static FORCEINLINE void BindImageTexture(GLuint Unit, GLuint Texture, GLint Level, GLboolean Layered, GLint Layer, GLenum Access, GLenum Format) UGL_REQUIRED_VOID
	static FORCEINLINE void DispatchCompute(GLuint NumGroupsX, GLuint NumGroupsY, GLuint NumGroupsZ) UGL_REQUIRED_VOID
	static FORCEINLINE void DispatchComputeIndirect(GLintptr Offset) UGL_REQUIRED_VOID
	static FORCEINLINE void MemoryBarrier(GLbitfield Barriers) UGL_REQUIRED_VOID
	static FORCEINLINE bool TexStorage2D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLenum Format, GLenum Type, ETextureCreateFlags Flags) UGL_OPTIONAL(false)
	static FORCEINLINE bool TexStorage2DMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height, GLboolean FixedSampleLocations) UGL_OPTIONAL(false)
	static FORCEINLINE void RenderbufferStorageMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height) UGL_REQUIRED_VOID
	static FORCEINLINE void TexStorage3D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type) UGL_REQUIRED_VOID
	static FORCEINLINE void CompressedTexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLsizei ImageSize, const GLvoid* PixelData) UGL_REQUIRED_VOID
	static FORCEINLINE void CopyImageSubData(GLuint SrcName, GLenum SrcTarget, GLint SrcLevel, GLint SrcX, GLint SrcY, GLint SrcZ, GLuint DstName, GLenum DstTarget, GLint DstLevel, GLint DstX, GLint DstY, GLint DstZ, GLsizei Width, GLsizei Height, GLsizei Depth) UGL_REQUIRED_VOID
	static FORCEINLINE void TextureView(GLuint ViewName, GLenum ViewTarget, GLuint SrcName, GLenum InternalFormat, GLuint MinLevel, GLuint NumLevels, GLuint MinLayer, GLuint NumLayers) UGL_REQUIRED_VOID
	static FORCEINLINE void DrawArraysIndirect(GLenum Mode, const void *Offset) UGL_REQUIRED_VOID
	static FORCEINLINE void DrawElementsIndirect(GLenum Mode, GLenum Type, const void *Offset) UGL_REQUIRED_VOID
	static FORCEINLINE void GenerateMipmap(GLenum Target) UGL_REQUIRED_VOID
	static FORCEINLINE void BindVertexBuffer(GLuint BindingIndex, GLuint Nuffer, GLintptr Offset, GLsizei Stride) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLboolean Normalized, GLuint RelativeOffset) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribIFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLuint RelativeOffset) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexAttribBinding(GLuint AttribIndex, GLuint BindingIndex) UGL_REQUIRED_VOID
	static FORCEINLINE void ClearBufferData(GLenum Target, GLenum InternalFormat, GLenum Format, GLenum Type, const uint32* Data) UGL_REQUIRED_VOID
	static FORCEINLINE void VertexBindingDivisor(GLuint BindingIndex, GLuint Divisor) UGL_REQUIRED_VOID
	static FORCEINLINE void BufferStorage(GLenum Target, GLsizeiptr Size, const void *Data, GLbitfield Flags) UGL_REQUIRED_VOID
	static FORCEINLINE void DepthBounds(GLfloat Min, GLfloat Max) UGL_REQUIRED_VOID
	static FORCEINLINE void TextureRange(GLenum Target, GLsizei Length, const GLvoid *Pointer) UGL_OPTIONAL_VOID
	static FORCEINLINE void ProgramParameter (GLuint Program, GLenum PName, GLint Value) UGL_OPTIONAL_VOID
	static FORCEINLINE void UseProgramStages(GLuint Pipeline, GLbitfield Stages, GLuint Program) { glAttachShader(Pipeline, Program); }
	static FORCEINLINE void BindProgramPipeline(GLuint Pipeline) { glUseProgram(Pipeline); }
	static FORCEINLINE void DeleteShader(GLuint Program) { glDeleteShader(Program); }
	static FORCEINLINE void DeleteProgramPipelines(GLsizei Number, const GLuint *Pipelines) { for(GLsizei i = 0; i < Number; i++) { glDeleteProgram(Pipelines[i]); } }
	static FORCEINLINE void GenProgramPipelines(GLsizei Number, GLuint *Pipelines) { check(Pipelines); for(GLsizei i = 0; i < Number; i++) { Pipelines[i] = glCreateProgram(); } }
	static FORCEINLINE void ProgramUniform1i(GLuint Program, GLint Location, GLint V0) { glUniform1i( Location, V0 ); }
	static FORCEINLINE void ProgramUniform4iv(GLuint Program, GLint Location, GLsizei Count, const GLint *Value) { glUniform4iv(Location, Count, Value); }
	static FORCEINLINE void ProgramUniform4fv(GLuint Program, GLint Location, GLsizei Count, const GLfloat *Value) { glUniform4fv(Location, Count, Value); }
	static FORCEINLINE void ProgramUniform4uiv(GLuint Program, GLint Location, GLsizei Count, const GLuint *Value) UGL_REQUIRED_VOID
	static FORCEINLINE void GetProgramPipelineiv(GLuint Pipeline, GLenum Pname, GLint* Params) UGL_OPTIONAL_VOID
	static FORCEINLINE void ValidateProgramPipeline(GLuint Pipeline) UGL_OPTIONAL_VOID
	static FORCEINLINE void GetProgramPipelineInfoLog(GLuint Pipeline, GLsizei BufSize, GLsizei* Length, GLchar* InfoLog) UGL_OPTIONAL_VOID
	static FORCEINLINE bool IsProgramPipeline(GLuint Pipeline) UGL_OPTIONAL(false)

	static FORCEINLINE GLuint64 GetTextureSamplerHandle(GLuint Texture, GLuint Sampler) UGL_REQUIRED(0)
	static FORCEINLINE GLuint64 GetTextureHandle(GLuint Texture) UGL_REQUIRED(0)
	static FORCEINLINE void MakeTextureHandleResident(GLuint64 TextureHandle) UGL_REQUIRED_VOID
	static FORCEINLINE void MakeTextureHandleNonResident(GLuint64 TextureHandle) UGL_REQUIRED_VOID
	static FORCEINLINE void UniformHandleui64(GLint Location, GLuint64 Value) UGL_REQUIRED_VOID

	static FORCEINLINE void GetProgramBinary(GLuint Program, GLsizei BufSize, GLsizei *Length, GLenum *BinaryFormat, void *Binary) UGL_OPTIONAL_VOID
	static FORCEINLINE void ProgramBinary(GLuint Program, GLenum BinaryFormat, const void *Binary, GLsizei Length) UGL_OPTIONAL_VOID

	static FORCEINLINE void FrameBufferFetchBarrier() UGL_OPTIONAL_VOID
	
	static FPlatformOpenGLDevice*	CreateDevice() UGL_REQUIRED(NULL)
	static FPlatformOpenGLContext*	CreateContext( FPlatformOpenGLDevice* Device, void* WindowHandle ) UGL_REQUIRED(NULL)

	static FORCEINLINE void CheckFrameBuffer()
	{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT 
		GLenum CompleteResult = glCheckFramebufferStatus(GL_FRAMEBUFFER);
		if (CompleteResult != GL_FRAMEBUFFER_COMPLETE)
		{
				UE_LOGF(LogOpenGL, Fatal,"Framebuffer not complete. Status = 0x%x", CompleteResult);
		}
#endif 
	}

	static FORCEINLINE void BufferSubData(GLenum Target, GLintptr Offset, GLsizeiptr Size, const GLvoid* Data)	{ glBufferSubData(Target, Offset, Size, Data); }
	static FORCEINLINE void DeleteBuffers(GLsizei Number, const GLuint* Buffers)	{ glDeleteBuffers(Number, Buffers); }
	static FORCEINLINE void DeleteTextures(GLsizei Number, const GLuint* Textures)	{ glDeleteTextures(Number, Textures); }
	static FORCEINLINE void Flush()													{ glFlush(); }
	static FORCEINLINE GLuint CreateShader(GLenum Type)								{ return glCreateShader(Type); }
	static FORCEINLINE GLuint CreateProgram()										{ return glCreateProgram(); }
	static FORCEINLINE bool TimerQueryDisjoint()									{ return false; }

	// Calling glBufferData() to discard-reupload is slower than calling glBufferSubData() on some platforms,
	// because changing glBufferData() with a different size (from before) may incur extra validation.
	// To use glBufferData() discard trick: set to this to true - otherwise, glBufferSubData() will used.
	static FORCEINLINE bool DiscardFrameBufferToResize()							{ return true; }

	// for platform extensions
	static void PE_GetCurrentOpenGLShaderDeviceCapabilities(FOpenGLShaderDeviceCapabilities& Capabilities);
	static bool PE_GLSLToDeviceCompatibleGLSL(FAnsiCharArray& GlslCodeOriginal, const FString& ShaderName, GLenum TypeEnum, const FOpenGLShaderDeviceCapabilities& Capabilities, FAnsiCharArray& GlslCode) UGL_OPTIONAL(false)
	static void PE_SetupTextureFormat(void(*SetupTextureFormat)(EPixelFormat, const FOpenGLTextureFormat&)) UGL_OPTIONAL_VOID

	static GLenum GetPlatfrom5551Format() UGL_OPTIONAL(GL_UNSIGNED_SHORT_5_5_5_1)
protected:
	static GLint MaxTextureImageUnits;
	static GLint MaxCombinedTextureImageUnits;
	static GLint MaxComputeTextureImageUnits;
	static GLint MaxVertexTextureImageUnits;
	static GLint MaxGeometryTextureImageUnits;
	static GLint MaxVertexUniformComponents;
	static GLint MaxPixelUniformComponents;
	static GLint MaxGeometryUniformComponents;
	static GLint MaxVaryingVectors;
	static GLint TextureBufferAlignment;

	/** GL_ARB_clip_control */
	static bool bSupportsClipControl;

	/** GL_KHR_texture_compression_astc_ldr */
	static bool bSupportsASTC;

	/** GL_KHR_texture_compression_astc_hdr */
	static bool bSupportsASTCHDR;

	/** GL_ARB_seamless_cube_map */
	static bool bSupportsSeamlessCubemap;
	
	/** Can we render to texture 2D array or 3D */
	static bool bSupportsVolumeTextureRendering;
	
	/** GL_EXT_texture_filter_anisotropic Can we use anisotropic filtering? */
	static bool bSupportsTextureFilterAnisotropic;

	/** GL_ARB_draw_buffers_blend */
	static bool bSupportsDrawBuffersBlend;

	/** Workaround AMD driver issues. */
	static bool bAmdWorkaround;

};

