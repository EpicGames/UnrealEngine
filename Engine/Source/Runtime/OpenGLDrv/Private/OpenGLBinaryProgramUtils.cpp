// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLBinaryProgramUtils.cpp
=============================================================================*/

#include "OpenGLBinaryProgramUtils.h"
#include "Misc/Compression.h"

namespace UE
{
	namespace OpenGL
	{
		static TAutoConsoleVariable<int32> CVarStoreCompressedBinaries(
			TEXT("r.OpenGL.StoreCompressedProgramBinaries"),
			0,
			TEXT(""),
			ECVF_ReadOnly | ECVF_RenderThreadSafe
		);

		bool IsStoringCompressedBinaryPrograms()
		{
			return CVarStoreCompressedBinaries.GetValueOnAnyThread() != 0;
		}

		bool UncompressCompressedBinaryProgram(const TArrayView<const uint8>& CompressedProgramBinary, TArray<uint8>& UncompressedProgramBinaryOUT)
		{
			if (ensure(CompressedProgramBinary.Num() > sizeof(FCompressedProgramBinaryHeader)))
			{
				FCompressedProgramBinaryHeader* Header = (FCompressedProgramBinaryHeader*)CompressedProgramBinary.GetData();

				if (Header->UncompressedSize == FCompressedProgramBinaryHeader::NotCompressed)
				{
					const uint32 ProgramSize = CompressedProgramBinary.Num() - sizeof(FCompressedProgramBinaryHeader);
					UncompressedProgramBinaryOUT.SetNumUninitialized(ProgramSize);
					FMemory::Memcpy(UncompressedProgramBinaryOUT.GetData(), CompressedProgramBinary.GetData() + sizeof(FCompressedProgramBinaryHeader), ProgramSize);
					return true;
				}
				else
				{
					UncompressedProgramBinaryOUT.AddUninitialized(Header->UncompressedSize);

					if (Header->UncompressedSize > 0
						&& FCompression::UncompressMemory(NAME_Zlib, UncompressedProgramBinaryOUT.GetData(), UncompressedProgramBinaryOUT.Num(), CompressedProgramBinary.GetData() + sizeof(FCompressedProgramBinaryHeader), CompressedProgramBinary.Num() - sizeof(FCompressedProgramBinaryHeader)))
					{
						return true;
					}
				}
			}
			return false;
		}

		bool GetUncompressedProgramBinaryFromGLProgram(GLuint Program, TArray<uint8>& ProgramBinaryOUT)
		{
			VERIFY_GL_SCOPE();
			check(ProgramBinaryOUT.IsEmpty());
			// pull binary from linked program
			GLint BinaryLength = -1;
			glGetProgramiv(Program, GL_PROGRAM_BINARY_LENGTH, &BinaryLength);
			if (BinaryLength > 0)
			{
				ProgramBinaryOUT.SetNumUninitialized(BinaryLength + sizeof(GLenum));
				uint8* ProgramBinaryPtr = ProgramBinaryOUT.GetData();
				// BinaryFormat is stored at the start of ProgramBinary array
				FOpenGL::GetProgramBinary(Program, BinaryLength, &BinaryLength, (GLenum*)ProgramBinaryPtr, ProgramBinaryPtr + sizeof(GLenum));
				return true;
			}
			return false;
		}

		void CompressProgramBinary(const TArrayView<const uint8>& UncompressedProgramBinary, TArray<uint8>& ProgramBinaryOUT)
		{
			check(IsStoringCompressedBinaryPrograms());
			check(ProgramBinaryOUT.IsEmpty());

			int32 CompressedSize = FCompression::CompressMemoryBound(NAME_Zlib, UncompressedProgramBinary.Num());
			uint32 CompressedHeaderSize = sizeof(FCompressedProgramBinaryHeader);
			ProgramBinaryOUT.AddUninitialized(CompressedSize + CompressedHeaderSize);
			bool bSuccess = FCompression::CompressMemory(NAME_Zlib, ProgramBinaryOUT.GetData() + CompressedHeaderSize, CompressedSize, UncompressedProgramBinary.GetData(), UncompressedProgramBinary.Num());
			if (bSuccess)
			{
				ProgramBinaryOUT.SetNum(CompressedSize + CompressedHeaderSize);
				ProgramBinaryOUT.Shrink();
				FCompressedProgramBinaryHeader* Header = (FCompressedProgramBinaryHeader*)ProgramBinaryOUT.GetData();
				Header->UncompressedSize = UncompressedProgramBinary.Num();
			}
			else
			{
				// failed, store the uncompressed version.
				UE_LOGF(LogRHI, Log, "Storing binary program uncompressed (%d, %d, %d)", UncompressedProgramBinary.Num(), ProgramBinaryOUT.Num(), CompressedSize);
				ProgramBinaryOUT.SetNumUninitialized(UncompressedProgramBinary.Num() + CompressedHeaderSize);
				FCompressedProgramBinaryHeader* Header = (FCompressedProgramBinaryHeader*)ProgramBinaryOUT.GetData();
				Header->UncompressedSize = FCompressedProgramBinaryHeader::NotCompressed;
				FMemory::Memcpy(ProgramBinaryOUT.GetData() + sizeof(FCompressedProgramBinaryHeader), UncompressedProgramBinary.GetData(), UncompressedProgramBinary.Num());
			}
		}

		bool GetCompressedProgramBinaryFromGLProgram(GLuint Program, TArray<uint8>& ProgramBinaryOUT)
		{
			// get uncompressed binary
			TArray<uint8> UncompressedProgramBinary;
			if (GetUncompressedProgramBinaryFromGLProgram(Program, UncompressedProgramBinary))
			{
				CompressProgramBinary(UncompressedProgramBinary, ProgramBinaryOUT);
				return true;
			}
			return false;
		}

		FOpenGLProgramBinary GetProgramBinaryFromGLProgram(GLuint Program)
		{
			TArray<uint8> ProgramBinaryData;
			if (IsStoringCompressedBinaryPrograms())
			{
				GetCompressedProgramBinaryFromGLProgram(Program, ProgramBinaryData);
			}
			else
			{
				GetUncompressedProgramBinaryFromGLProgram(Program, ProgramBinaryData);
			}
			return FOpenGLProgramBinary(MoveTemp(ProgramBinaryData));
		}
	}
}