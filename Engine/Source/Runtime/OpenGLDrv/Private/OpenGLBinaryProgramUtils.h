// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLBinaryProgramUtils.h
=============================================================================*/

#pragma once
#include "OpenGLDrvPrivate.h"

class FOpenGLProgramBinary
{
public:
	FOpenGLProgramBinary() {};
	FOpenGLProgramBinary(TArray<uint8>&& ProgramMemory) : OwnedData(MoveTemp(ProgramMemory)), DataView(OwnedData) 
	{
		INC_MEMORY_STAT_BY(STAT_OpenGLShaderLRUProgramMemory, DataView.Num());
		check(IsOwned());
		check(IsValid());
	}

	// No ownership, just a view to a region within the mmapped PSO cache.
	FOpenGLProgramBinary(TArrayView<const uint8> ProgramMemory) : DataView(ProgramMemory) 
	{
		INC_MEMORY_STAT_BY(STAT_OpenGLShaderLRUProgramMemoryMapped, DataView.Num());
		check(!IsOwned());
		check(IsValid());
	}

	FOpenGLProgramBinary& operator= (FOpenGLProgramBinary&& rhs) = default;

	FOpenGLProgramBinary(FOpenGLProgramBinary&& Src) = default;

	~FOpenGLProgramBinary()
	{
		if(IsOwned())
		{
			DEC_MEMORY_STAT_BY(STAT_OpenGLShaderLRUProgramMemory, DataView.Num());
		}
		else
		{
			// This should only change when shutting down the PSO cache, i.e. DataView.IsEmpty() during normal use.
			DEC_MEMORY_STAT_BY(STAT_OpenGLShaderLRUProgramMemoryMapped, DataView.Num());
		}
	}

	bool IsValid() const { return DataView.Num() > 0; }
	bool IsOwned() const { return OwnedData.Num() > 0; }
	const TArrayView<const uint8> GetDataView() const { return DataView; }
private:
	// The data when programs are owned.
	TArray<uint8> OwnedData;
	// a view to the raw program binary, either points to OwnedData or an array view within the mmapped prebuilt PSO cache.
	TArrayView<const uint8> DataView;
};

namespace UE
{
	namespace OpenGL
	{
		// Program Binary helpers.

		bool IsStoringCompressedBinaryPrograms();

		struct FCompressedProgramBinaryHeader
		{
			static const uint32 NotCompressed = 0xFFFFFFFF;
			uint32 UncompressedSize;
		};

		FOpenGLProgramBinary GetProgramBinaryFromGLProgram(GLuint Program);
		bool GetUncompressedProgramBinaryFromGLProgram(GLuint Program, TArray<uint8>& ProgramBinaryOUT);
		bool GetCompressedProgramBinaryFromGLProgram(GLuint Program, TArray<uint8>& ProgramBinaryOUT);
		void CompressProgramBinary(const TArrayView<const uint8>& UncompressedProgramBinary, TArray<uint8>& ProgramBinaryOUT);
		bool UncompressCompressedBinaryProgram(const TArrayView<const uint8>& CompressedProgramBinary, TArray<uint8>& UncompressedProgramBinaryOUT);
	}
}