// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLVertexBuffer.cpp: OpenGL vertex buffer RHI implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Containers/ResourceArray.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "OpenGLDrv.h"
#include "RHICoreBufferInitializer.h"

namespace OpenGLConsoleVariables
{
#if PLATFORM_ANDROID
	int32 bUseStagingBuffer = 0;
#else
	int32 bUseStagingBuffer = 1;
#endif

	static FAutoConsoleVariableRef CVarUseStagingBuffer(
		TEXT("OpenGL.UseStagingBuffer"),
		bUseStagingBuffer,
		TEXT("Enables maps of dynamic vertex buffers to go to a staging buffer"),
		ECVF_ReadOnly
		);

	extern int32 bUsePersistentMappingStagingBuffer;
};

static const uint32 MAX_ALIGNMENT_BITS = 8;
static const uint32 MAX_OFFSET_BITS = 32 - MAX_ALIGNMENT_BITS;

struct PoolAllocation
{
	uint8* BasePointer;
	uint32 SizeWithoutPadding;
	uint32 Offset				: MAX_OFFSET_BITS;		// into the target buffer
	uint32 AlignmentPadding		: MAX_ALIGNMENT_BITS;
	int32 FrameRetired;
};

static TArray<PoolAllocation*> AllocationList;
static TMap<void*,PoolAllocation*> AllocationMap;

static GLuint PoolVB = 0;
static uint8* PoolPointer = 0;
static uint32 FrameBytes = 0;
static uint32 FreeSpace = 0;
static uint32 OffsetVB = 0;
static const uint32 PerFrameMax = 1024*1024*4;
static const uint32 MaxAlignment = 1 << MAX_ALIGNMENT_BITS;
static const uint32 MaxOffset = 1 << MAX_OFFSET_BITS;

void* GetAllocation( void* Target, uint32 Size, uint32 Offset, uint32 Alignment = 16)
{
	check(Alignment < MaxAlignment);
	check(Offset < MaxOffset);
	check(FMath::IsPowerOfTwo(Alignment));

	uintptr_t AlignmentSubOne = Alignment - 1;

	if (FOpenGL::SupportsBufferStorage() && OpenGLConsoleVariables::bUseStagingBuffer)
	{
		if (PoolVB == 0)
		{
			FOpenGL::GenBuffers(1, &PoolVB);
			glBindBuffer(GL_COPY_READ_BUFFER, PoolVB);
			FOpenGL::BufferStorage(GL_COPY_READ_BUFFER, PerFrameMax * 4, NULL, GL_MAP_WRITE_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT);
			PoolPointer = (uint8*)FOpenGL::MapBufferRange(GL_COPY_READ_BUFFER, 0, PerFrameMax * 4, FOpenGL::EResourceLockMode::RLM_WriteOnlyPersistent);

			FreeSpace = PerFrameMax * 4;

			check(PoolPointer);
		}
		check (PoolVB);

		uintptr_t AllocHeadPtr = *reinterpret_cast<const uintptr_t*>(&PoolPointer) + OffsetVB;
		uint32 AlignmentPadBytes = ((AllocHeadPtr + AlignmentSubOne) & (~AlignmentSubOne)) - AllocHeadPtr;
		uint32 SizeWithAlignmentPad = Size + AlignmentPadBytes;

		if (SizeWithAlignmentPad > PerFrameMax - FrameBytes || SizeWithAlignmentPad > FreeSpace)
		{
			return nullptr;
		}

		if (SizeWithAlignmentPad > (PerFrameMax*4 - OffsetVB))
		{
			// We're wrapping, create dummy allocation and start at the begining
			uint32 Leftover = PerFrameMax*4 - OffsetVB;
			PoolAllocation* Alloc = new PoolAllocation;
			Alloc->BasePointer = 0;
			Alloc->Offset = 0;
			Alloc->AlignmentPadding = 0;
			Alloc->SizeWithoutPadding = Leftover;
			Alloc->FrameRetired = GFrameNumberRenderThread;

			AllocationList.Add(Alloc);
			OffsetVB = 0;
			FreeSpace -= Leftover;

			AllocHeadPtr = *reinterpret_cast<const uintptr_t*>(&PoolPointer) + OffsetVB;
			AlignmentPadBytes = ((AllocHeadPtr + AlignmentSubOne) & (~AlignmentSubOne)) - AllocHeadPtr;
			SizeWithAlignmentPad = Size + AlignmentPadBytes;
		}

		//Again check if we have room
		if (SizeWithAlignmentPad > FreeSpace)
		{
			return nullptr;
		}

		PoolAllocation* Alloc = new PoolAllocation;
		Alloc->BasePointer = PoolPointer + OffsetVB;
		Alloc->Offset = Offset;
		Alloc->AlignmentPadding = AlignmentPadBytes;
		Alloc->SizeWithoutPadding = Size;
		Alloc->FrameRetired = -1;

		AllocationList.Add(Alloc);
		AllocationMap.Add(Target, Alloc);
		OffsetVB += SizeWithAlignmentPad;
		FreeSpace -= SizeWithAlignmentPad;
		FrameBytes += SizeWithAlignmentPad;

		return Alloc->BasePointer + Alloc->AlignmentPadding;

	}

	return nullptr;
}

bool RetireAllocation( FOpenGLBuffer* Target)
{
	if (FOpenGL::SupportsBufferStorage() && OpenGLConsoleVariables::bUseStagingBuffer)
	{
		PoolAllocation *Alloc = 0;

		if ( AllocationMap.RemoveAndCopyValue(Target, Alloc))
		{
			check(Alloc);
			Target->Bind();

			FOpenGL::CopyBufferSubData(GL_COPY_READ_BUFFER, GL_ARRAY_BUFFER, (Alloc->BasePointer + Alloc->AlignmentPadding) - PoolPointer, Alloc->Offset, Alloc->SizeWithoutPadding);

			Alloc->FrameRetired = GFrameNumberRenderThread;

			return true;
		}
	}
	return false;
}

void BeginFrame_VertexBufferCleanup()
{
	if (GFrameNumberRenderThread < 3)
	{
		return;
	}

	int32 NumToRetire = 0;
	int32 FrameToRecover = GFrameNumberRenderThread - 3;

	while (NumToRetire < AllocationList.Num())
	{
		PoolAllocation *Alloc = AllocationList[NumToRetire];
		if (Alloc->FrameRetired < 0 || Alloc->FrameRetired > FrameToRecover)
		{
			break;
		}
		FreeSpace += (Alloc->SizeWithoutPadding + Alloc->AlignmentPadding);
		delete Alloc;
		NumToRetire++;
	}

	AllocationList.RemoveAt(0,NumToRetire);
	FrameBytes = 0;
}

void FOpenGLBufferBase::Bind()
{
	VERIFY_GL_SCOPE();
	FOpenGLDynamicRHI::Get().CachedBindBuffer(Type, Resource);
}

void FOpenGLBufferBase::OnBufferDeletion()
{
	FOpenGLDynamicRHI::Get().OnBufferDeletion(Type, Resource);
}

static GLenum GetNewBufferType(const FRHIBufferCreateDesc& CreateDesc)
{
	GLenum BufferType = 0;

	if (!CreateDesc.IsNull())
	{
		BufferType = GL_ARRAY_BUFFER;
		if (EnumHasAnyFlags(CreateDesc.Usage, BUF_StructuredBuffer))
		{
			BufferType = GL_SHADER_STORAGE_BUFFER;
		}
		else if (EnumHasAnyFlags(CreateDesc.Usage, BUF_IndexBuffer))
		{
			BufferType = GL_ELEMENT_ARRAY_BUFFER;
		}
	}

	return BufferType;
}

FRHIBufferInitializer FOpenGLDynamicRHI::RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	const GLenum BufferType = GetNewBufferType(CreateDesc);

	if (CreateDesc.IsNull())
	{
		FOpenGLBuffer* Buffer = new FOpenGLBuffer(&RHICmdList, BufferType, CreateDesc, nullptr);
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	const void* InitialData = nullptr;
	if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray)
	{
		// TODO: move this initialization code path into this method
		InitialData = CreateDesc.InitialData->GetResourceData();
	}

	FOpenGLBuffer* Buffer = new FOpenGLBuffer(&RHICmdList, BufferType, CreateDesc, InitialData);

	if (CreateDesc.InitialData)
	{
		// Discard the resource array's contents.
		CreateDesc.InitialData->Discard();
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::ResourceArray || CreateDesc.InitAction == ERHIBufferInitAction::Default)
	{
		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Zeroed)
	{
		void* WritableData = RHICmdList.LockBuffer(Buffer, 0, CreateDesc.Size, RLM_WriteOnly);
		FMemory::Memzero(WritableData, CreateDesc.Size);
		RHICmdList.UnlockBuffer(Buffer);

		return UE::RHICore::FDefaultBufferInitializer(RHICmdList, Buffer);
	}

	if (CreateDesc.InitAction == ERHIBufferInitAction::Initializer)
	{
		// Use LockBuffer + UnlockBuffer to allow the caller to write initial buffer data
		return UE::RHICore::FLockBufferInitializer(RHICmdList, Buffer);
	}

	return UE::RHICore::HandleUnknownBufferInitializerInitAction(RHICmdList, CreateDesc);
}

void* FOpenGLDynamicRHI::LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI, uint32 Offset, uint32 Size, EResourceLockMode LockMode)
{
	check(Size > 0);

	VERIFY_GL_SCOPE();
	FOpenGLBuffer* Buffer = ResourceCast(BufferRHI);
	if (Buffer->IsDynamic() && LockMode == EResourceLockMode::RLM_WriteOnly)
	{
		void *Staging = GetAllocation(Buffer, Size, Offset);
		if (Staging)
		{
			return Staging;
		}
	}

	const bool bReadOnly = (LockMode == EResourceLockMode::RLM_ReadOnly);
	const bool bDiscard = !bReadOnly; // Always use 'orphaning' on write as buffer could be in use by GPU atm
	return (void*)Buffer->Lock(Offset, Size, bReadOnly, bDiscard);
}

void FOpenGLDynamicRHI::UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* BufferRHI)
{
	VERIFY_GL_SCOPE();
	FOpenGLBuffer* Buffer = ResourceCast(BufferRHI);
	if (!RetireAllocation(Buffer))
	{
		Buffer->Unlock();
	}
}

void FOpenGLDynamicRHI::RHICopyBufferRegion(FRHIBuffer* DestBufferRHI, uint64 DstOffset, FRHIBuffer* SourceBufferRHI, uint64 SrcOffset, uint64 NumBytes)
{
	VERIFY_GL_SCOPE();
	FOpenGLBuffer* SourceBuffer = ResourceCast(SourceBufferRHI);
	FOpenGLBuffer* DestBuffer = ResourceCast(DestBufferRHI);

	glBindBuffer(GL_COPY_READ_BUFFER, SourceBuffer->Resource);
	glBindBuffer(GL_COPY_WRITE_BUFFER, DestBuffer->Resource);
	FOpenGL::CopyBufferSubData(GL_COPY_READ_BUFFER, GL_COPY_WRITE_BUFFER, SrcOffset, DstOffset, NumBytes);
	glBindBuffer(GL_COPY_READ_BUFFER, 0);
	glBindBuffer(GL_COPY_WRITE_BUFFER, 0);
}

FStagingBufferRHIRef FOpenGLDynamicRHI::RHICreateStagingBuffer()
{
	return new FOpenGLStagingBuffer();
}

void FOpenGLStagingBuffer::Initialize()
{
	ShadowBuffer = 0;
	ShadowSize = 0;
	Mapping = nullptr;
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda([this](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();
		glGenBuffers(1, &ShadowBuffer);
	});
	RHITHREAD_GLTRACE_BLOCKING;
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}

FOpenGLStagingBuffer::~FOpenGLStagingBuffer()
{
	VERIFY_GL_SCOPE();
	glDeleteBuffers(1, &ShadowBuffer);
}

// If we do not support the BufferStorage extension or if PersistentMapping is set to false, this will send the command to the RHI and flush it
// If we do support BufferStorage extension and PersistentMapping is set to true, we just return the pointer + offset
void* FOpenGLStagingBuffer::Lock(uint32 Offset, uint32 NumBytes)
{
	if (!FOpenGL::SupportsBufferStorage() || !OpenGLConsoleVariables::bUsePersistentMappingStagingBuffer)
	{
		void* ReturnValue = nullptr;
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		RHICmdList.EnqueueLambda([this, Offset, NumBytes, &ReturnValue](FRHICommandListImmediate&) {
			VERIFY_GL_SCOPE();
			check(ShadowBuffer != 0);
			glBindBuffer(GL_COPY_WRITE_BUFFER, ShadowBuffer);
			void* LocalMapping = FOpenGL::MapBufferRange(GL_COPY_WRITE_BUFFER, 0, NumBytes, FOpenGL::EResourceLockMode::RLM_ReadOnly);
			check(LocalMapping);
			ReturnValue = reinterpret_cast<uint8*>(LocalMapping) + Offset;
			});
		RHITHREAD_GLTRACE_BLOCKING;
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		return ReturnValue;
	}
	else
	{
		check(Mapping != nullptr);
		return reinterpret_cast<uint8*>(Mapping) + Offset;
	}
}

// If we do not support the BufferStorage extension or if PersistentMapping is set to false, this will send the command to the RHI and flush it
// If we do support BufferStorage extension and PersistentMapping is set to true, we do nothing
void FOpenGLStagingBuffer::Unlock()
{
	if (!FOpenGL::SupportsBufferStorage() || !OpenGLConsoleVariables::bUsePersistentMappingStagingBuffer)
	{
		FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
		RHICmdList.EnqueueLambda([this](FRHICommandListImmediate&) {
			FOpenGL::UnmapBuffer(GL_COPY_WRITE_BUFFER);
			Mapping = nullptr;
			glBindBuffer(GL_COPY_WRITE_BUFFER, 0);
		});
	}
}

void* FOpenGLDynamicRHI::RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI)
{
	FOpenGLStagingBuffer* Buffer = ResourceCast(StagingBuffer);
	return Buffer->Lock(Offset, SizeRHI);	
}

void FOpenGLDynamicRHI::RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer)
{
	FOpenGLStagingBuffer* Buffer = ResourceCast(StagingBuffer);
	Buffer->Unlock();
}

void* FOpenGLDynamicRHI::LockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI)
{
	check(IsInRenderingThread());
	if (!Fence || !Fence->Poll() || Fence->NumPendingWriteCommands.GetValue() != 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FDynamicRHI_LockStagingBuffer_Flush);
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	}
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FDynamicRHI_LockStagingBuffer_RenderThread);
		return GDynamicRHI->RHILockStagingBuffer(StagingBuffer, Fence, Offset, SizeRHI);
	}
}

void FOpenGLDynamicRHI::UnlockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FDynamicRHI_UnlockStagingBuffer_RenderThread);
	check(IsInRenderingThread());
	GDynamicRHI->RHIUnlockStagingBuffer(StagingBuffer);
}


#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
void FOpenGLDynamicRHI::RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer)
{
}
#endif

