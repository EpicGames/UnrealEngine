// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLCommands.cpp: OpenGL RHI commands implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "RHIDefinitions.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "EngineGlobals.h"
#include "RenderResource.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "RenderUtils.h"
#include "RHICoreShader.h"
#include "RHIShaderParametersShared.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIUniformBufferUtilities.h"

/*
#define DECLARE_ISBOUNDSHADER(ShaderType) template <typename TShaderType> inline void ValidateBoundShader(TRefCountPtr<FOpenGLBoundShaderState> InBoundShaderState, FRHIGraphicsShader* GfxShader, TShaderType* ShaderTypeRHI) \
{ \
	FOpenGL##ShaderType* ShaderType = FOpenGLDynamicRHI::ResourceCast(static_cast<TShaderType*>(GfxShader)); \
	ensureMsgf(InBoundShaderState && ShaderType == InBoundShaderState->Get##ShaderType(), TEXT("Parameters are being set for a %s which is not currently bound"), TEXT(#ShaderType)); \
}

DECLARE_ISBOUNDSHADER(VertexShader)
DECLARE_ISBOUNDSHADER(PixelShader)
DECLARE_ISBOUNDSHADER(GeometryShader)
*/

#if 0//DO_CHECK
	#define VALIDATE_BOUND_SHADER(s, t) ValidateBoundShader<t>(PendingState.BoundShaderState, s)
#else
	#define VALIDATE_BOUND_SHADER(s, t)
#endif

namespace OpenGLConsoleVariables
{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
	int32 bUseMapBuffer = 0;
#else
	int32 bUseMapBuffer = 1;
#endif
	static FAutoConsoleVariableRef CVarUseMapBuffer(
		TEXT("OpenGL.UseMapBuffer"),
		bUseMapBuffer,
		TEXT("If true, use glMapBuffer otherwise use glBufferSubdata.")
		);

	int32 bSkipCompute = 0;
	static FAutoConsoleVariableRef CVarSkipCompute(
		TEXT("OpenGL.SkipCompute"),
		bSkipCompute,
		TEXT("If true, don't issue dispatch work.")
		);

#if PLATFORM_WINDOWS || PLATFORM_LINUX
	int32 MaxSubDataSize = 256*1024;
#else
	int32 MaxSubDataSize = 0;
#endif
	static FAutoConsoleVariableRef CVarMaxSubDataSize(
		TEXT("OpenGL.MaxSubDataSize"),
		MaxSubDataSize,
		TEXT("Maximum amount of data to send to glBufferSubData in one call"),
		ECVF_ReadOnly
		);
	
	int32 bRebindTextureBuffers = 0;
	static FAutoConsoleVariableRef CVarRebindTextureBuffers(
		TEXT("OpenGL.RebindTextureBuffers"),
		bRebindTextureBuffers,
		TEXT("If true, rebind GL_TEXTURE_BUFFER's to their GL_TEXTURE name whenever the buffer is modified.")
		);

	int32 bUseBufferDiscard = 1;
	static FAutoConsoleVariableRef CVarUseBufferDiscard(
		TEXT("OpenGL.UseBufferDiscard"),
		bUseBufferDiscard,
		TEXT("If true, use dynamic buffer orphaning hint.")
		);
	int32 bUseUnsynchronizedBufferMapping =1;
	static FAutoConsoleVariableRef CVarUseUnsynchronizedBufferMapping(
		TEXT("OpenGL.UseUnsynchronizedBufferMapping"),
		bUseUnsynchronizedBufferMapping,
		TEXT("If true, use GL_MAP_UNSYNCHRONIZED_BIT when mapping write only buffers.")
	);
	int32 bUsePersistentMappingStagingBuffer= 1;
	static FAutoConsoleVariableRef CVarUsePersistentMappingStagingBuffer(
		TEXT("OpenGL.UsePersistentMappingStagingBuffer"),
		bUsePersistentMappingStagingBuffer,
		TEXT("If true, it will use persistent mapping for the Staging Buffer."),
		ECVF_ReadOnly
	);

	int32 GOpenGLForceBilinear = 0;
	static FAutoConsoleVariableRef CVarOpenGLForceBilinearSampling(
		TEXT("r.OpenGL.ForceBilinear"),
		GOpenGLForceBilinear,
		TEXT("Force GL to override all trilinear or aniso texture filtering states to bilinear.\n")
		TEXT("0: disabled. (default)\n")
		TEXT("1: enabled."),
		ECVF_ReadOnly | ECVF_RenderThreadSafe
	);

	int32 GOpenGLFenceKickPerDrawCount = 0;
	static FAutoConsoleVariableRef CVarOpenGLFenceKickPerDrawCount(
		TEXT("r.OpenGL.FenceKickPerDrawCount"),
		GOpenGLFenceKickPerDrawCount,
		TEXT("Insert a GL fence after the specified number of draws has been issued.\n")
		TEXT("This hint can encourage some drivers to begin processing geometry as soon as a sufficient workload has built up.\n")
		TEXT("0: disabled (default)"),
		ECVF_RenderThreadSafe
	);

};

#define INDEX_TO_VOID(Index) (void*)((uint64)(Index))

enum EClearType
{
	CT_None				= 0x0,
	CT_Depth			= 0x1,
	CT_Stencil			= 0x2,
	CT_Color			= 0x4,
	CT_DepthStencil		= CT_Depth | CT_Stencil,
};

struct FPendingSamplerDataValue
{
	GLenum	Enum;
	GLint	Value;
};

static FORCEINLINE GLint ModifyFilterByMips(GLint Filter, bool bHasMips)
{
	if (!bHasMips)
	{
		switch (Filter)
		{
			case GL_LINEAR_MIPMAP_NEAREST:
			case GL_LINEAR_MIPMAP_LINEAR:
				return GL_LINEAR;

			case GL_NEAREST_MIPMAP_NEAREST:
			case GL_NEAREST_MIPMAP_LINEAR:
				return GL_NEAREST;

			default:
				break;
		}
	}

	return Filter;
}

static FORCEINLINE bool ValidateShader(FRHIComputeShader* ShaderRHI)
{
	return true;
}

static FORCEINLINE bool ValidateShader(FRHIGraphicsShader* ShaderRHI)
{
	switch (ShaderRHI->GetFrequency())
	{
	case SF_Vertex:
		VALIDATE_BOUND_SHADER(ShaderRHI, Vertex);
		return true;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	case SF_Geometry:
		VALIDATE_BOUND_SHADER(ShaderRHI, Geometry);
		return true;
#endif
	case SF_Pixel:
		VALIDATE_BOUND_SHADER(ShaderRHI, Pixel);
		return true;
	default:
		checkf(0, TEXT("Undefined FRHIShader Type %d!"), (int32)ShaderRHI->GetFrequency());
		return false;
	}
}

// Vertex state.
void FOpenGLDynamicRHI::RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBufferRHI, uint32 Offset)
{
	VERIFY_GL_SCOPE();
	FOpenGLBuffer* VertexBuffer = ResourceCast(VertexBufferRHI);
	PendingState.Streams[StreamIndex].VertexBufferResource = VertexBuffer ? VertexBuffer->Resource : 0;
	PendingState.Streams[StreamIndex].Stride = PendingState.BoundShaderState ? PendingState.BoundShaderState->StreamStrides[StreamIndex] : 0;
	PendingState.Streams[StreamIndex].Offset = Offset;
}

// Rasterizer state.
void FOpenGLDynamicRHI::RHISetRasterizerState(FRHIRasterizerState* NewStateRHI)
{
	VERIFY_GL_SCOPE();
	FOpenGLRasterizerState* NewState = ResourceCast(NewStateRHI);
	PendingState.RasterizerState = NewState->Data;
}

void FOpenGLDynamicRHI::UpdateRasterizerStateInOpenGLContext()
{
	VERIFY_GL_SCOPE();
	if (FOpenGL::SupportsPolygonMode() && ContextState.RasterizerState.FillMode != PendingState.RasterizerState.FillMode)
	{
		FOpenGL::PolygonMode(GL_FRONT_AND_BACK, PendingState.RasterizerState.FillMode);
		ContextState.RasterizerState.FillMode = PendingState.RasterizerState.FillMode;
	}

	if (ContextState.RasterizerState.CullMode != PendingState.RasterizerState.CullMode)
	{
		if (PendingState.RasterizerState.CullMode != GL_NONE)
		{
			// Only call glEnable if needed
			if (ContextState.RasterizerState.CullMode == GL_NONE)
			{
				glEnable(GL_CULL_FACE);
			}
			glCullFace(PendingState.RasterizerState.CullMode);
		}
		else
		{
			glDisable(GL_CULL_FACE);
		}
		ContextState.RasterizerState.CullMode = PendingState.RasterizerState.CullMode;
	}

	if (FOpenGL::SupportsDepthClamp() && ContextState.RasterizerState.DepthClipMode != PendingState.RasterizerState.DepthClipMode)
	{
		if (PendingState.RasterizerState.DepthClipMode == ERasterizerDepthClipMode::DepthClamp)
		{
			glEnable(GL_DEPTH_CLAMP);
		}
		else
		{
			glDisable(GL_DEPTH_CLAMP);
		}
		ContextState.RasterizerState.DepthClipMode = PendingState.RasterizerState.DepthClipMode;
	}

	// Convert our platform independent depth bias into an OpenGL depth bias.
	const float BiasScale = float((1<<24)-1);	// Warning: this assumes depth bits == 24, and won't be correct with 32.
	float DepthBias = PendingState.RasterizerState.DepthBias * BiasScale;
	if (ContextState.RasterizerState.DepthBias != PendingState.RasterizerState.DepthBias
		|| ContextState.RasterizerState.SlopeScaleDepthBias != PendingState.RasterizerState.SlopeScaleDepthBias)
	{
		if ((DepthBias == 0.0f) && (PendingState.RasterizerState.SlopeScaleDepthBias == 0.0f))
		{
			// If we're here, both previous 2 'if' conditions are true, and this implies that cached state was not all zeroes, so we need to glDisable.
			glDisable(GL_POLYGON_OFFSET_FILL);
			if ( FOpenGL::SupportsPolygonMode() )
			{
				glDisable(GL_POLYGON_OFFSET_LINE);
				glDisable(GL_POLYGON_OFFSET_POINT);
			}
		}
		else
		{
			if (ContextState.RasterizerState.DepthBias == 0.0f && ContextState.RasterizerState.SlopeScaleDepthBias == 0.0f)
			{
				glEnable(GL_POLYGON_OFFSET_FILL);
				if ( FOpenGL::SupportsPolygonMode() )
				{
					glEnable(GL_POLYGON_OFFSET_LINE);
					glEnable(GL_POLYGON_OFFSET_POINT);
				}
			}
			glPolygonOffset(PendingState.RasterizerState.SlopeScaleDepthBias, DepthBias);
		}

		ContextState.RasterizerState.DepthBias = PendingState.RasterizerState.DepthBias;
		ContextState.RasterizerState.SlopeScaleDepthBias = PendingState.RasterizerState.SlopeScaleDepthBias;
	}
}

void FOpenGLDynamicRHI::UpdateViewportInOpenGLContext()
{
	VERIFY_GL_SCOPE();
	if (ContextState.Viewport != PendingState.Viewport)
	{
		//@todo the viewport defined by glViewport does not clip, unlike the viewport in d3d
		// Set the scissor rect to the viewport unless it is explicitly set smaller to emulate d3d.
		glViewport(
			PendingState.Viewport.Min.X,
			PendingState.Viewport.Min.Y,
			PendingState.Viewport.Max.X - PendingState.Viewport.Min.X,
			PendingState.Viewport.Max.Y - PendingState.Viewport.Min.Y);

		ContextState.Viewport = PendingState.Viewport;
	}

	if (ContextState.DepthMinZ != PendingState.DepthMinZ || ContextState.DepthMaxZ != PendingState.DepthMaxZ)
	{
		FOpenGL::DepthRange(PendingState.DepthMinZ, PendingState.DepthMaxZ);
		ContextState.DepthMinZ = PendingState.DepthMinZ;
		ContextState.DepthMaxZ = PendingState.DepthMaxZ;
	}
}

void FOpenGLDynamicRHI::RHISetViewport(float MinX, float MinY,float MinZ, float MaxX, float MaxY,float MaxZ)
{
	VERIFY_GL_SCOPE();
	PendingState.Viewport.Min.X = (uint32)MinX;
	PendingState.Viewport.Min.Y = (uint32)MinY;
	PendingState.Viewport.Max.X = (uint32)MaxX;
	PendingState.Viewport.Max.Y = (uint32)MaxY;
	PendingState.DepthMinZ = MinZ;
	PendingState.DepthMaxZ = MaxZ;

	RHISetScissorRect(false, 0, 0, 0, 0);
}

void FOpenGLDynamicRHI::RHISetScissorRect(bool bEnable,uint32 MinX,uint32 MinY,uint32 MaxX,uint32 MaxY)
{
	VERIFY_GL_SCOPE();
	PendingState.bScissorEnabled = bEnable;
	PendingState.Scissor.Min.X = MinX;
	PendingState.Scissor.Min.Y = MinY;
	PendingState.Scissor.Max.X = MaxX;
	PendingState.Scissor.Max.Y = MaxY;
}

void FOpenGLDynamicRHI::UpdateScissorRectInOpenGLContext()
{
	VERIFY_GL_SCOPE();
	if (ContextState.bScissorEnabled != PendingState.bScissorEnabled)
	{
		if (PendingState.bScissorEnabled)
		{
			glEnable(GL_SCISSOR_TEST);
		}
		else
		{
			glDisable(GL_SCISSOR_TEST);
		}
		ContextState.bScissorEnabled = PendingState.bScissorEnabled;
	}

	if (PendingState.bScissorEnabled && ContextState.Scissor != PendingState.Scissor)
	{
		check(PendingState.Scissor.Min.X <= PendingState.Scissor.Max.X);
		check(PendingState.Scissor.Min.Y <= PendingState.Scissor.Max.Y);
		glScissor(PendingState.Scissor.Min.X, PendingState.Scissor.Min.Y, PendingState.Scissor.Max.X - PendingState.Scissor.Min.X, PendingState.Scissor.Max.Y - PendingState.Scissor.Min.Y);
		ContextState.Scissor = PendingState.Scissor;
	}
}

/**
* Set bound shader state. This will set the vertex decl/shader, and pixel shader
* @param BoundShaderState - state resource
*/
void FOpenGLDynamicRHI::RHISetBoundShaderState(FRHIBoundShaderState* BoundShaderStateRHI)
{
	VERIFY_GL_SCOPE();
	FOpenGLBoundShaderState* BoundShaderState = ResourceCast(BoundShaderStateRHI);
	PendingState.BoundShaderState = BoundShaderState;

	// Prevent transient bound shader states from being recreated for each use by keeping a history of the most recently used bound shader states.
	// The history keeps them alive, and the bound shader state cache allows them to be reused if needed.
	BoundShaderStateHistory.Add(BoundShaderState);
}

void FOpenGLDynamicRHI::InternalSetShaderTexture(FOpenGLTexture* Texture, FOpenGLShaderResourceView* SRV, GLint TextureIndex, GLenum Target, GLuint Resource, int NumMips, int LimitMip)
{
	auto& PendingTextureState = PendingState.Textures[TextureIndex];
	PendingTextureState.Texture = Texture;
	PendingTextureState.SRV = SRV;
	PendingTextureState.Target = Target;
	PendingTextureState.Resource = Resource;
	PendingTextureState.LimitMip = LimitMip;
	PendingTextureState.bHasMips = (NumMips == 0 || NumMips > 1);
	PendingTextureState.NumMips = NumMips;
}

void FOpenGLDynamicRHI::InternalSetSamplerStates(GLint TextureIndex, FOpenGLSamplerState* SamplerState)
{
	PendingState.SamplerStates[TextureIndex] = SamplerState;
}

void FOpenGLDynamicRHI::CachedSetupTextureStageInner(GLint TextureIndex, GLenum Target, GLuint Resource, GLint LimitMip, GLint NumMips)
{
	DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_CachedSetupTextureStage);
	VERIFY_GL_SCOPE();
	FTextureStage& TextureState = ContextState.Textures[TextureIndex];

	// Something will have to be changed. Switch to the stage in question.
	if (ContextState.ActiveTexture != TextureIndex)
	{
		glActiveTexture( GL_TEXTURE0 + TextureIndex );
		ContextState.ActiveTexture = TextureIndex;
	}

	if (TextureState.Target == Target)
	{
		glBindTexture(Target, Resource);
	}
	else
	{
		if (TextureState.Target != GL_NONE)
		{
			// Unbind different texture target on the same stage, to avoid OpenGL keeping its data, and potential driver problems.
			glBindTexture(TextureState.Target, 0);
		}

		if (Target != GL_NONE)
		{
			glBindTexture(Target, Resource);
		}
	}
	
	// Use the texture SRV's LimitMip value to specify the mip available for sampling
	// This requires SupportsTextureBaseLevel & is a fallback for TextureView
	if (Target != GL_NONE && Target != GL_TEXTURE_BUFFER && Target != GL_TEXTURE_EXTERNAL_OES)
	{
		TPair<GLenum, GLenum>* MipLimits;
		
		{
			DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_CachedSetupTextureStage_Find);
			MipLimits = TextureMipLimits.Find(Resource);
		}
		
		GLint BaseMip = LimitMip == -1 ? 0 : LimitMip;
		GLint MaxMip = LimitMip == -1 ? NumMips - 1 : LimitMip;
		
		const bool bSameLimitMip = MipLimits && MipLimits->Key == BaseMip;
		const bool bSameNumMips = MipLimits && MipLimits->Value == MaxMip;

		if (!bSameLimitMip || !bSameNumMips)
		{
			DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_CachedSetupTextureStage_TexParameter);
			
			bool bBoundAsRenderTarget = false;
			for (uint32 RenderTargetIndex = 0; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
			{
				if (PendingState.RenderTargets[RenderTargetIndex] == 0)
				{
					break;
				}
				else
				{
					if (PendingState.RenderTargets[RenderTargetIndex]->GetResource() == Resource)
					{
						bBoundAsRenderTarget = true;
						break;
					}
				}
			}

			// If a SRV is bound as render target, skip the BASE_LEVEL and MAX_LEVEL settings because it would cause crash on some android devices.
			if (!bBoundAsRenderTarget)
			{
				if (!bSameLimitMip)
				{
					FOpenGL::TexParameter(Target, GL_TEXTURE_BASE_LEVEL, BaseMip);
				}
				if (MipLimits)
				{
					MipLimits->Key = BaseMip;
					MipLimits->Value = MaxMip;
				}
				else
				{
					TextureMipLimits.Add(Resource, TPair<GLenum, GLenum>(BaseMip, MaxMip));
				}
			}
			else
			{
				LimitMip = 0;
				NumMips = 0;
			}
		}
	}
	else
	{
		LimitMip = 0;
		NumMips = 0;
	}

	TextureState.LimitMip = LimitMip;
	TextureState.NumMips = NumMips;
	TextureState.Target = Target;
	TextureState.Resource = Resource;
}

inline void FOpenGLDynamicRHI::ApplyTextureStage(GLint TextureIndex, const FTextureStage& TextureStage, FOpenGLSamplerState* SamplerState)
{
	GLenum Target = TextureStage.Target;
	VERIFY_GL_SCOPE();
	const bool bHasTexture = (TextureStage.Texture != NULL);
	if (!bHasTexture || TextureStage.Texture->SamplerState != SamplerState)
	{
		// Texture must be bound first
		if (ContextState.ActiveTexture != TextureIndex)
		{
			glActiveTexture(GL_TEXTURE0 + TextureIndex);
			ContextState.ActiveTexture = TextureIndex;
		}

		GLint WrapS = SamplerState->Data.WrapS;
		GLint WrapT = SamplerState->Data.WrapT;

		// Sets parameters of currently bound texture
		FOpenGL::TexParameter(Target, GL_TEXTURE_WRAP_S, WrapS);
		FOpenGL::TexParameter(Target, GL_TEXTURE_WRAP_T, WrapT);
		if( FOpenGL::SupportsTexture3D() )
		{
			FOpenGL::TexParameter(Target, GL_TEXTURE_WRAP_R, SamplerState->Data.WrapR);
		}

		if( FOpenGL::SupportsTextureLODBias() )
		{
			FOpenGL::TexParameter(Target, GL_TEXTURE_LOD_BIAS, SamplerState->Data.LODBias);
		}
		// Make sure we don't set mip filtering on if the texture has no mip levels, as that will cause a crash/black render on ES.
		GLint MinFilter = ModifyFilterByMips(SamplerState->Data.MinFilter, TextureStage.bHasMips);
		if (OpenGLConsoleVariables::GOpenGLForceBilinear && MinFilter == GL_LINEAR_MIPMAP_LINEAR)
		{
			MinFilter = GL_LINEAR_MIPMAP_NEAREST;
		}

		FOpenGL::TexParameter(Target, GL_TEXTURE_MIN_FILTER, MinFilter);
		FOpenGL::TexParameter(Target, GL_TEXTURE_MAG_FILTER, SamplerState->Data.MagFilter);
		if( FOpenGL::SupportsTextureFilterAnisotropic() )
		{
			// GL_EXT_texture_filter_anisotropic requires value to be at least 1
			GLint MaxAnisotropy = FMath::Max(1, SamplerState->Data.MaxAnisotropy);
			FOpenGL::TexParameter(Target, GL_TEXTURE_MAX_ANISOTROPY_EXT, MaxAnisotropy);
		}

		if( FOpenGL::SupportsTextureCompare() )
		{
			FOpenGL::TexParameter(Target, GL_TEXTURE_COMPARE_MODE, SamplerState->Data.CompareMode);
			FOpenGL::TexParameter(Target, GL_TEXTURE_COMPARE_FUNC, SamplerState->Data.CompareFunc);
		}

		if (bHasTexture)
		{
			TextureStage.Texture->SamplerState = SamplerState;
		}
	}
}

template <typename StateType>
void FOpenGLDynamicRHI::SetupTexturesForDraw(const StateType& ShaderState, int32 MaxTexturesNeeded)
{
	VERIFY_GL_SCOPE();
	SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLTextureBindTime);

	int32 MaxProgramTexture = 0;
	const TBitArray<>& NeededBits = ShaderState->GetTextureNeeds(MaxProgramTexture);

	for( int32 TextureStageIndex = 0; TextureStageIndex <= MaxProgramTexture; ++TextureStageIndex )
	{
		if (!NeededBits[TextureStageIndex])
		{
			// Current program doesn't make use of this texture stage. No matter what UnrealEditor wants to have on in,
			// it won't be useful for this draw, so telling OpenGL we don't really need it to give the driver
			// more leeway in memory management, and avoid false alarms about same texture being set on
			// texture stage and in framebuffer.
			CachedSetupTextureStage(TextureStageIndex, GL_NONE, 0, -1, 1 );
		}
		else
		{
			const FTextureStage& TextureStage = PendingState.Textures[TextureStageIndex];
			
#if UE_BUILD_DEBUG
			// Use the texture SRV's LimitMip value to specify the mip available for sampling
			// This requires SupportsTextureBaseLevel & is a fallback for TextureView
			{
				// When trying to limit the mip available for sampling (as part of texture SRV)
				// ensure that the texture is bound to only one sampler, or that all samplers
				// share the same restriction.
				if(TextureStage.LimitMip != -1)
				{
					for( int32 TexIndex = 0; TexIndex <= MaxProgramTexture; ++TexIndex )
					{
						if(TexIndex != TextureStageIndex && ShaderState->NeedsTextureStage(TexIndex))
						{
							const FTextureStage& OtherStage = PendingState.Textures[TexIndex];
							const bool bSameResource = OtherStage.Resource == TextureStage.Resource;
							const bool bSameTarget = OtherStage.Target == TextureStage.Target;
							const GLint TextureStageBaseMip = TextureStage.LimitMip == -1 ? 0 : TextureStage.LimitMip;
							const GLint OtherStageBaseMip = OtherStage.LimitMip == -1 ? 0 : OtherStage.LimitMip;
							const bool bSameLimitMip = TextureStageBaseMip == OtherStageBaseMip;
							const GLint TextureStageMaxMip = TextureStage.LimitMip == -1 ? TextureStage.NumMips - 1 : TextureStage.LimitMip;
							const GLint OtherStageMaxMip = OtherStage.LimitMip == -1 ? OtherStage.NumMips - 1 : OtherStage.LimitMip;
							const bool bSameMaxMip = TextureStageMaxMip == OtherStageMaxMip;
							if( bSameTarget && bSameResource && !bSameLimitMip && !bSameMaxMip )
							{
								UE_LOGF(LogRHI, Warning, "Texture SRV fallback requires that each texture SRV be bound with the same mip-range restrictions. Expect rendering errors.");
							}
						}
					}
				}
			}
#endif
			CachedSetupTextureStage(TextureStageIndex, TextureStage.Target, TextureStage.Resource, TextureStage.LimitMip, TextureStage.NumMips );
			
			bool bExternalTexture = (TextureStage.Target == GL_TEXTURE_EXTERNAL_OES);
			if (!bExternalTexture)
			{
				FOpenGLSamplerState* PendingSampler = PendingState.SamplerStates[TextureStageIndex];
			
				if (ContextState.SamplerStates[TextureStageIndex] != PendingSampler)
				{
					FOpenGL::BindSampler(TextureStageIndex, PendingSampler ? PendingSampler->Resource : 0);
					ContextState.SamplerStates[TextureStageIndex] = PendingSampler;
				}

				if (TextureStage.Texture && TextureStage.Texture->bInStencilTextureMode)
				{
					//set the texture to return the depth component, and then force the components to match D3D
					glTexParameteri(TextureStage.Target, GL_DEPTH_STENCIL_TEXTURE_MODE, GL_DEPTH_COMPONENT);
					glTexParameteri(TextureStage.Target, GL_TEXTURE_SWIZZLE_R, GL_RED);
					glTexParameteri(TextureStage.Target, GL_TEXTURE_SWIZZLE_G, GL_ZERO);
					glTexParameteri(TextureStage.Target, GL_TEXTURE_SWIZZLE_B, GL_ZERO);
					glTexParameteri(TextureStage.Target, GL_TEXTURE_SWIZZLE_A, GL_ZERO);

					TextureStage.Texture->bInStencilTextureMode = false;
				}
			}
			else if (TextureStage.Target != GL_TEXTURE_BUFFER)
			{
				FOpenGL::BindSampler(TextureStageIndex, 0);
				ContextState.SamplerStates[TextureStageIndex] = nullptr;
				ApplyTextureStage(TextureStageIndex, TextureStage, PendingState.SamplerStates[TextureStageIndex] );
			}
		}
	}

	// For now, continue to clear unused stages
	for (int32 TextureStageIndex = MaxProgramTexture + 1; TextureStageIndex < MaxTexturesNeeded; ++TextureStageIndex)
	{
		CachedSetupTextureStage(TextureStageIndex, GL_NONE, 0, -1, 1 );
	}
}

void FOpenGLDynamicRHI::SetupTexturesForDraw()
{
	SetupTexturesForDraw(PendingState.BoundShaderState, FOpenGL::GetMaxCombinedTextureImageUnits());
}

void FOpenGLDynamicRHI::InternalSetShaderImageUAV(GLint UAVIndex, GLenum Format, GLuint Resource, bool bLayered, GLint Layer, GLenum Access, GLint Level)
{
	VERIFY_GL_SCOPE();
	PendingState.UAVs[UAVIndex].Format = Format;
	PendingState.UAVs[UAVIndex].Resource = Resource;
	PendingState.UAVs[UAVIndex].Access = Access;
	PendingState.UAVs[UAVIndex].Layer = Layer;
	PendingState.UAVs[UAVIndex].bLayered = bLayered;
	PendingState.UAVs[UAVIndex].Level = Level;
}

void FOpenGLDynamicRHI::InternalSetShaderBufferUAV(GLint UAVIndex, GLuint Resource)
{
	VERIFY_GL_SCOPE();
	PendingState.UAVs[UAVIndex].Format = 0;
	PendingState.UAVs[UAVIndex].Resource = Resource;
	PendingState.UAVs[UAVIndex].Access = GL_READ_WRITE;
	PendingState.UAVs[UAVIndex].Layer = 0;
	PendingState.UAVs[UAVIndex].bLayered = false;
}

void FOpenGLDynamicRHI::SetupUAVsForDraw()
{
	int32 MaxUAVUnitUsed = 0;
	const TBitArray<>& NeededBits = PendingState.BoundShaderState->GetUAVNeeds(MaxUAVUnitUsed);
	SetupUAVsForProgram(NeededBits, MaxUAVUnitUsed);
}

void FOpenGLDynamicRHI::SetupUAVsForCompute(const FOpenGLComputeShader* ComputeShader)
{
	int32 MaxUAVUnitUsed = 0;
	const TBitArray<>& NeededBits = ComputeShader->GetUAVNeeds(MaxUAVUnitUsed);
	SetupUAVsForProgram(NeededBits, MaxUAVUnitUsed);
}

void FOpenGLDynamicRHI::SetupUAVsForProgram(const TBitArray<>& NeededBits, int32 MaxUAVUnitUsed)
{
	if (MaxUAVUnitUsed < 0 && ContextState.ActiveUAVMask == 0)
	{
		// Quit early if program does not use UAVs and context has no active UAV units
		return;
	}

	for (int32 UAVStageIndex = 0; UAVStageIndex <= MaxUAVUnitUsed; ++UAVStageIndex)
	{
		if (!NeededBits[UAVStageIndex])
		{
			CachedSetupUAVStage(UAVStageIndex, 0, 0, false, 0, GL_READ_WRITE, 0);
		}
		else
		{
			const FUAVStage& UAVStage = PendingState.UAVs[UAVStageIndex];
			CachedSetupUAVStage(UAVStageIndex, UAVStage.Format, UAVStage.Resource, UAVStage.bLayered, UAVStage.Layer, UAVStage.Access, UAVStage.Level);
		}
	}

	// clear rest of the units
	int32 UAVStageIndex = (MaxUAVUnitUsed + 1);
	if ((ContextState.ActiveUAVMask >> UAVStageIndex) != 0)
	{
		const int32 NumUAVs = ContextState.UAVs.Num();
		for (; UAVStageIndex < NumUAVs; ++UAVStageIndex)
		{
			CachedSetupUAVStage(UAVStageIndex, 0, 0, false, 0, GL_READ_WRITE, 0);
		}
	}
}

static bool IsImageTextureFormatSupported(GLenum Format)
{
#if PLATFORM_ANDROID
	if (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// from GLES 3.1 spec
		switch (Format)
		{
			case GL_RGBA32F:
			case GL_RGBA16F:
			case GL_R32F:
			case GL_RGBA32UI:
			case GL_RGBA16UI:
			case GL_RGBA8UI:
			case GL_R32UI:
			case GL_RGBA32I:
			case GL_RGBA16I:
			case GL_RGBA8I:
			case GL_R32I:
			case GL_RGBA8:
			case GL_RGBA8_SNORM:
				return true;
			default:
				return false;
		}
	}
#endif
	return true;
}

void FOpenGLDynamicRHI::CachedSetupUAVStage(GLint UAVIndex, GLenum Format, GLuint Resource, bool bLayered, GLint Layer, GLenum Access, GLint Level)
{
	VERIFY_GL_SCOPE();
	
	FUAVStage& UAVStage = ContextState.UAVs[UAVIndex];

	if (UAVStage.Format == Format && 
		UAVStage.Resource == Resource &&
		UAVStage.Access == Access &&
		UAVStage.Layer == Layer &&
		UAVStage.bLayered == bLayered &&
		UAVStage.Level == Level)
	{
		// Nothing's changed, no need to update
		return;
	}

	// unbind any SSBO or Image in this slot
	if (Resource == 0)
	{
		if (UAVStage.Resource != 0)
		{
			// SSBO
			if (UAVStage.Format == 0)
			{
				FOpenGL::BindBufferBase(GL_SHADER_STORAGE_BUFFER, UAVIndex, 0);
				ContextState.StorageBufferBound = 0;
			}
			else // Image
			{
				FOpenGL::BindImageTexture(UAVIndex, 0, 0, GL_FALSE, 0, GL_READ_WRITE, GL_R32F);
			}

			UAVStage.Format = 0;
			UAVStage.Resource = 0;
			UAVStage.Access = GL_READ_WRITE;
			UAVStage.Layer = 0;
			UAVStage.bLayered = false;
			UAVStage.Level = 0;
		}
	}
	else
	{
		// SSBO
		if (Format == 0)
		{
			// make sure we dont end up binding both SSBO and Image to the same UAV slot
			if (UAVStage.Resource != 0 && UAVStage.Format != 0)
			{
				FOpenGL::BindImageTexture(UAVIndex, 0, 0, GL_FALSE, 0, GL_READ_WRITE, GL_R32F);
			}

			FOpenGL::BindBufferBase(GL_SHADER_STORAGE_BUFFER, UAVIndex, Resource);
			
			UAVStage.Format = 0;
			UAVStage.Resource = Resource;
			UAVStage.Access = GL_READ_WRITE;
			UAVStage.Layer = 0;
			UAVStage.bLayered = false;
			UAVStage.Level = 0;
			ContextState.StorageBufferBound = Resource;
		}
		else // Image
		{
			// make sure we dont end up binding both SSBO and Image to the same UAV slot
			if (UAVStage.Resource != 0 && UAVStage.Format == 0)
			{
				FOpenGL::BindBufferBase(GL_SHADER_STORAGE_BUFFER, UAVIndex, 0);
				ContextState.StorageBufferBound = 0;
			}
			
			check(IsImageTextureFormatSupported(Format));
	
			FOpenGL::BindImageTexture(UAVIndex, Resource, Level, bLayered ? GL_TRUE : GL_FALSE, Layer, Access, Format);
	
			UAVStage.Format = Format;
			UAVStage.Resource = Resource;
			UAVStage.Access = Access;
			UAVStage.Layer = Layer;
			UAVStage.bLayered = bLayered;
			UAVStage.Level = Level;
		}
	}
	
	uint32 UAVBit = 1 << UAVIndex;
	if (Resource != 0)
	{
		ContextState.ActiveUAVMask |= UAVBit;
	}
	else
	{
		ContextState.ActiveUAVMask &= ~UAVBit;
	}
}

void FOpenGLDynamicRHI::UpdateSRV(FOpenGLShaderResourceView* SRV)
{
	check(SRV);
	// For Depth/Stencil textures whose Stencil component we wish to sample we must blit the stencil component out to an intermediate texture when we 'Store' the texture.
#if PLATFORM_DESKTOP
	if (FOpenGL::GetFeatureLevel() >= ERHIFeatureLevel::SM5 && SRV->IsTexture())
	{
		FOpenGLTexture* Texture = ResourceCast(SRV->GetTexture());
		
		uint32 ArrayIndices = 0;
		uint32 MipmapLevels = 0;
		
		GLuint SourceFBO = GetOpenGLFramebuffer(0, nullptr, &ArrayIndices, &MipmapLevels, Texture);
		
		glBindFramebuffer(GL_FRAMEBUFFER, SourceFBO);
		
		uint32 SizeX = Texture->GetSizeX();
		uint32 SizeY = Texture->GetSizeY();
		
		uint32 MipBytes = SizeX * SizeY;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("PixelBuffer"), MipBytes, 0, EBufferUsageFlags::Dynamic);

		TRefCountPtr<FOpenGLPixelBuffer> PixelBuffer = new FOpenGLPixelBuffer(nullptr, GL_PIXEL_UNPACK_BUFFER, CreateDesc, nullptr);
		
		glBindBuffer( GL_PIXEL_PACK_BUFFER, 0 );
		glBindBuffer( GL_PIXEL_PACK_BUFFER, PixelBuffer->Resource );
		glPixelStorei(GL_PACK_ALIGNMENT, 1);
		glReadPixels(0, 0, SizeX, SizeY, GL_STENCIL_INDEX, GL_UNSIGNED_BYTE, nullptr );
		glPixelStorei(GL_PACK_ALIGNMENT, 4);
		glBindBuffer( GL_PIXEL_PACK_BUFFER, 0 );
		
		GLenum Target = SRV->Target;
		
		CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Target, SRV->Resource, -1, 1);
		
		CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, PixelBuffer->Resource);
		
		glPixelStorei(GL_UNPACK_ROW_LENGTH, SizeX);
		
		glPixelStorei(GL_UNPACK_ALIGNMENT, 1);
		glTexSubImage2D(Target, 0, 0, 0, SizeX, SizeY, GL_RED_INTEGER, GL_UNSIGNED_BYTE, nullptr);
		glPixelStorei(GL_UNPACK_ALIGNMENT, 4);
		
		glPixelStorei(GL_UNPACK_ROW_LENGTH, 0);
		
		CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);
		
		glBindFramebuffer(GL_FRAMEBUFFER, ContextState.Framebuffer);
		ContextState.Framebuffer = -1;
	}
#endif
}

void FOpenGLDynamicRHI::RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers)
{
	FMemory::Memzero(GlobalUniformBuffers.GetData(), GlobalUniformBuffers.Num() * sizeof(FRHIUniformBuffer*));

	for (int32 Index = 0; Index < InUniformBuffers.GetUniformBufferCount(); ++Index)
	{
		GlobalUniformBuffers[InUniformBuffers.GetSlot(Index)] = InUniformBuffers.GetUniformBuffer(Index);
	}
}

void FOpenGLDynamicRHI::RHISetStaticUniformBuffer(FUniformBufferStaticSlot InSlot, FRHIUniformBuffer* InBuffer)
{
	GlobalUniformBuffers[InSlot] = InBuffer;
}

void FOpenGLDynamicRHI::RHISetUniformBufferDynamicOffset(FUniformBufferStaticSlot InSlot, uint32 InOffset)
{
	VERIFY_GL_SCOPE();

	// FIXME: GLES does not seem to expose UBO offset aligments requirements, using worst case here 
	check(IsAligned(InOffset, 256u));

	static const EShaderFrequency ShaderStages[2] =
	{
		SF_Vertex,
		SF_Pixel
	};

	FRHIShader* Shaders[2] =
	{
		PendingState.BoundShaderState->GetVertexShader(),
		PendingState.BoundShaderState->GetPixelShader()
	};

	for (int32 ShaderIdx = 0; ShaderIdx < UE_ARRAY_COUNT(ShaderStages); ++ShaderIdx)
	{
		EShaderFrequency Stage = ShaderStages[ShaderIdx];
		FRHIShader* Shader = Shaders[ShaderIdx];
		if (Shader == nullptr)
		{
			continue;
		}

		TArray<FUniformBufferStaticSlot> const& StaticSlots = Shader->GetStaticSlots();

		for (int32 BufferIndex = 0; BufferIndex < StaticSlots.Num(); ++BufferIndex)
		{
			FUniformBufferStaticSlot const& Slot = StaticSlots[BufferIndex];
			if (InSlot == Slot)
			{
				FRHIUniformBuffer* Buffer = PendingState.BoundUniformBuffers[Stage][BufferIndex];
				if (Buffer)
				{
					PendingState.BoundUniformBuffersDynamicOffset[Stage][BufferIndex] = InOffset;
					PendingState.bAnyDirtyRealUniformBuffers[Stage] = true;
				}
				break;
			}
		}
	}
}

void FOpenGLDynamicRHI::BindUniformBuffer(EShaderFrequency ShaderFrequency, uint32 BufferIndex, FRHIUniformBuffer* BufferRHI)
{
	PendingState.BoundUniformBuffers[ShaderFrequency][BufferIndex] = BufferRHI;
	PendingState.BoundUniformBuffersDynamicOffset[ShaderFrequency][BufferIndex] = 0u;
	PendingState.DirtyUniformBuffers[ShaderFrequency] |= 1 << BufferIndex;
	PendingState.bAnyDirtyGraphicsUniformBuffers = true;

	if (!GUseEmulatedUniformBuffers || !((FOpenGLUniformBuffer*)BufferRHI)->bIsEmulatedUniformBuffer)
	{
		PendingState.bAnyDirtyRealUniformBuffers[ShaderFrequency] = true;
	}
}

struct FOpenGLResourceBinder
{
	FOpenGLDynamicRHI& RHI;
	EShaderFrequency ShaderFrequency;
	CrossCompiler::EShaderStage ShaderStage = CrossCompiler::NUM_SHADER_STAGES;
	GLint FirstTextureUnit = 0;
	GLint MaxTextureImageUnits = 0;
	GLint FirstUAVUnit = 0;
	GLint MaxUAVUnits = 0;
	bool bClearTextureSamplers = false;

	FOpenGLResourceBinder() = delete;
	FOpenGLResourceBinder(FOpenGLDynamicRHI& InRHI, EShaderFrequency InShaderFrequency, bool bInClearTextureSamplers = false)
		: RHI(InRHI)
		, ShaderFrequency(InShaderFrequency)
		, bClearTextureSamplers(bInClearTextureSamplers)
	{
		Init();
	}

	void Init()
	{
		switch (ShaderFrequency)
		{
		case SF_Vertex:
			ShaderStage = CrossCompiler::SHADER_STAGE_VERTEX;
			FirstTextureUnit = FOpenGL::GetFirstVertexTextureUnit();
			MaxTextureImageUnits = FOpenGL::GetMaxVertexTextureImageUnits();
			FirstUAVUnit = FOpenGL::GetFirstVertexUAVUnit();
			MaxUAVUnits = FOpenGL::GetMaxPixelUAVUnits();
			break;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		case SF_Geometry:
			ShaderStage = CrossCompiler::SHADER_STAGE_GEOMETRY;
			FirstTextureUnit = FOpenGL::GetFirstGeometryTextureUnit();
			MaxTextureImageUnits = FOpenGL::GetMaxGeometryTextureImageUnits();
			FirstUAVUnit = 0;
			MaxUAVUnits = 0;
			break;
#endif
		case SF_Pixel:
			ShaderStage = CrossCompiler::SHADER_STAGE_PIXEL;
			FirstTextureUnit = FOpenGL::GetFirstPixelTextureUnit();
			MaxTextureImageUnits = FOpenGL::GetMaxTextureImageUnits();
			FirstUAVUnit = FOpenGL::GetFirstPixelUAVUnit();
			MaxUAVUnits = FOpenGL::GetMaxPixelUAVUnits();
			break;
		case SF_Compute:
			ShaderStage = CrossCompiler::SHADER_STAGE_COMPUTE;
			FirstTextureUnit = FOpenGL::GetFirstComputeTextureUnit();
			MaxTextureImageUnits = FOpenGL::GetMaxComputeTextureImageUnits();
			FirstUAVUnit = FOpenGL::GetFirstComputeUAVUnit();
			MaxUAVUnits = FOpenGL::GetMaxComputeUAVUnits();
			break;
		default:
			checkf(0, TEXT("Undefined FRHIShader Type %d!"), (int32)ShaderFrequency);
			break;
		}
	}

	void SetUAV(FRHIUnorderedAccessView* RHIUAV, uint8 Index)
	{
		ensureMsgf(Index < MaxUAVUnits
			, TEXT("Using more %s image units (%d) than allowed (%d) on a shader unit!")
			, GetShaderFrequencyString(ShaderFrequency, false)
			, Index
			, MaxUAVUnits
		);

		const GLint UAVIndex = FirstUAVUnit + Index;

		if (FOpenGLUnorderedAccessView* UAV = FOpenGLDynamicRHI::ResourceCast(RHIUAV))
		{
			if (UAV->Resource)
			{
				const GLenum Access = (ShaderFrequency == SF_Compute) ? GL_READ_WRITE : GL_WRITE_ONLY;
				RHI.InternalSetShaderImageUAV(UAVIndex, UAV->Format, UAV->Resource, UAV->IsLayered(), UAV->GetLayer(), Access, UAV->Level);
			}
			else
			{
				RHI.InternalSetShaderBufferUAV(UAVIndex, UAV->BufferResource);
			}
		}
		else
		{
			RHI.InternalSetShaderBufferUAV(UAVIndex, 0);
		}
	}

	void SetSRV(FRHIShaderResourceView* RHISRV, uint8 Index)
	{
		ensureMsgf(Index < MaxTextureImageUnits
			, TEXT("Using more %s texture units (%d) than allowed (%d) on a shader unit!")
			, GetShaderFrequencyString(ShaderFrequency, false)
			, Index
			, MaxTextureImageUnits
		);

		if (FOpenGLShaderResourceView* SRV = FOpenGLDynamicRHI::ResourceCast(RHISRV))
		{
			if (SRV->Target == GL_SHADER_STORAGE_BUFFER)
			{
				RHI.InternalSetShaderBufferUAV(FirstUAVUnit + Index, SRV->Resource);
			}
			else
			{
				RHI.InternalSetShaderTexture(nullptr, SRV, FirstTextureUnit + Index, SRV->Target, SRV->Resource, 1, SRV->LimitMip);
				SetSampler(RHI.GetPointSamplerState(), Index);
			}
		}
		else
		{
			RHI.InternalSetShaderTexture(nullptr, nullptr, FirstTextureUnit + Index, GL_TEXTURE_BUFFER, 0, 0, -1);
			SetSampler(RHI.GetPointSamplerState(), Index);
		}
	}

	void SetTexture(FRHITexture* TextureRHI, uint8 Index)
	{
		ensureMsgf(Index < MaxTextureImageUnits
			, TEXT("Using more %s texture units (%d) than allowed (%d) on a shader unit!")
			, GetShaderFrequencyString(ShaderFrequency, false)
			, Index
			, MaxTextureImageUnits
		);

		if (FOpenGLTexture* Texture = FOpenGLDynamicRHI::ResourceCast(TextureRHI))
		{
			RHI.InternalSetShaderTexture(Texture, nullptr, FirstTextureUnit + Index, Texture->Target, Texture->GetResource(), Texture->GetNumMips(), -1);
		}
		else
		{
			RHI.InternalSetShaderTexture(nullptr, nullptr, FirstTextureUnit + Index, 0, 0, 0, -1);
		}

		if (bClearTextureSamplers)
		{
			// clear any previous sampler state
			RHI.InternalSetSamplerStates(FirstTextureUnit + Index, nullptr);
		}
	}

	void SetSampler(FRHISamplerState* Sampler, uint8 Index)
	{
		RHI.InternalSetSamplerStates(FirstTextureUnit + Index, FOpenGLDynamicRHI::ResourceCast(Sampler));
	}

#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	void SetResourceCollection(FRHIResourceCollection* ResourceCollection, uint32 Index)
	{
		checkNoEntry();
	}
#endif
};

void FOpenGLDynamicRHI::SetShaderParametersCommon(EShaderFrequency ShaderFrequency, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters)
{
	FOpenGLResourceBinder Binder(*this, ShaderFrequency);

	if (InParameters.Num())
	{
		for (const FRHIShaderParameter& Parameter : InParameters)
		{
			PendingState.ShaderParameters[Binder.ShaderStage].Set(Parameter.BufferIndex, Parameter.BaseIndex, Parameter.ByteSize, &InParametersData[Parameter.ByteOffset]);
			PendingState.LinkedProgramAndDirtyFlag = nullptr;
		}
	}

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		if (Parameter.Type == FRHIShaderParameterResource::EType::UnorderedAccessView)
		{
			Binder.SetUAV(static_cast<FRHIUnorderedAccessView*>(Parameter.Resource), Parameter.Index);
		}
	}

	for (const FRHIShaderParameterResource& Parameter : InResourceParameters)
	{
		switch (Parameter.Type)
		{
		case FRHIShaderParameterResource::EType::Texture:
			Binder.SetTexture(static_cast<FRHITexture*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::ResourceView:
			Binder.SetSRV(static_cast<FRHIShaderResourceView*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UnorderedAccessView:
			break;
		case FRHIShaderParameterResource::EType::Sampler:
			Binder.SetSampler(static_cast<FRHISamplerState*>(Parameter.Resource), Parameter.Index);
			break;
		case FRHIShaderParameterResource::EType::UniformBuffer:
			BindUniformBuffer(Binder.ShaderFrequency, Parameter.Index, static_cast<FRHIUniformBuffer*>(Parameter.Resource));
			break;
		default:
			checkf(false, TEXT("Unhandled resource type?"));
			break;
		}
	}
}

void FOpenGLDynamicRHI::RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	VERIFY_GL_SCOPE();
	if (ValidateShader(Shader))
	{
		SetShaderParametersCommon(
			Shader->GetFrequency()
			, InParametersData
			, InParameters
			, InResourceParameters
		);
	}
}

void FOpenGLDynamicRHI::RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters)
{
	VERIFY_GL_SCOPE();
	if (ValidateShader(Shader))
	{
		SetShaderParametersCommon(
			SF_Compute
			, InParametersData
			, InParameters
			, InResourceParameters
		);
	}
}

void FOpenGLDynamicRHI::SetShaderUnbindsCommon(EShaderFrequency ShaderFrequency, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	FOpenGLResourceBinder Binder(*this, ShaderFrequency);

	for (const FRHIShaderParameterUnbind& Unbind : InUnbinds)
	{
		switch (Unbind.Type)
		{
		case FRHIShaderParameterUnbind::EType::ResourceView:
			Binder.SetSRV(nullptr, Unbind.Index);
			break;
		case FRHIShaderParameterUnbind::EType::UnorderedAccessView:
			Binder.SetUAV(nullptr, Unbind.Index);
			break;
		default:
			checkf(false, TEXT("Unhandled unbind resource type?"));
			break;
		}
	}
}

void FOpenGLDynamicRHI::RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	VERIFY_GL_SCOPE();
	if (ValidateShader(Shader))
	{
		SetShaderUnbindsCommon(SF_Compute, InUnbinds);
	}
}

void FOpenGLDynamicRHI::RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds)
{
	VERIFY_GL_SCOPE();
	if (ValidateShader(Shader))
	{
		SetShaderUnbindsCommon(Shader->GetFrequency(), InUnbinds);
	}
}

void FOpenGLDynamicRHI::RHISetDepthStencilState(FRHIDepthStencilState* NewStateRHI,uint32 StencilRef)
{
	VERIFY_GL_SCOPE();
	FOpenGLDepthStencilState* NewState = ResourceCast(NewStateRHI);
	PendingState.DepthStencilState = NewState->Data;
	PendingState.StencilRef = StencilRef;
}

void FOpenGLDynamicRHI::RHISetStencilRef(uint32 StencilRef)
{
	VERIFY_GL_SCOPE();
	PendingState.StencilRef = StencilRef;
}

void FOpenGLDynamicRHI::UpdateDepthStencilStateInOpenGLContext()
{
	VERIFY_GL_SCOPE();
	if (ContextState.DepthStencilState.bZEnable != PendingState.DepthStencilState.bZEnable)
	{
		if (PendingState.DepthStencilState.bZEnable)
		{
			glEnable(GL_DEPTH_TEST);
		}
		else
		{
			glDisable(GL_DEPTH_TEST);
		}
		ContextState.DepthStencilState.bZEnable = PendingState.DepthStencilState.bZEnable;
	}

	if (ContextState.DepthStencilState.bZWriteEnable != PendingState.DepthStencilState.bZWriteEnable)
	{
		glDepthMask(PendingState.DepthStencilState.bZWriteEnable);
		ContextState.DepthStencilState.bZWriteEnable = PendingState.DepthStencilState.bZWriteEnable;
	}

	if (PendingState.DepthStencilState.bZEnable)
	{
		if (ContextState.DepthStencilState.ZFunc != PendingState.DepthStencilState.ZFunc)
		{
			glDepthFunc(PendingState.DepthStencilState.ZFunc);
			ContextState.DepthStencilState.ZFunc = PendingState.DepthStencilState.ZFunc;
		}
	}

	if (ContextState.DepthStencilState.bStencilEnable != PendingState.DepthStencilState.bStencilEnable)
	{
		if (PendingState.DepthStencilState.bStencilEnable)
		{
			glEnable(GL_STENCIL_TEST);
		}
		else
		{
			glDisable(GL_STENCIL_TEST);
		}
		ContextState.DepthStencilState.bStencilEnable = PendingState.DepthStencilState.bStencilEnable;
	}

	// If only two-sided <-> one-sided stencil mode changes, and nothing else, we need to call full set of functions
	// to ensure all drivers handle this correctly - some of them might keep those states in different variables.
	if (ContextState.DepthStencilState.bTwoSidedStencilMode != PendingState.DepthStencilState.bTwoSidedStencilMode)
	{
		// Invalidate cache to enforce update of part of stencil state that needs to be set with different functions, when needed next
		// Values below are all invalid, but they'll never be used, only compared to new values to be set.
		ContextState.DepthStencilState.StencilFunc = 0xFFFF;
		ContextState.DepthStencilState.StencilFail = 0xFFFF;
		ContextState.DepthStencilState.StencilZFail = 0xFFFF;
		ContextState.DepthStencilState.StencilPass = 0xFFFF;
		ContextState.DepthStencilState.CCWStencilFunc = 0xFFFF;
		ContextState.DepthStencilState.CCWStencilFail = 0xFFFF;
		ContextState.DepthStencilState.CCWStencilZFail = 0xFFFF;
		ContextState.DepthStencilState.CCWStencilPass = 0xFFFF;
		ContextState.DepthStencilState.StencilReadMask = 0xFFFF;

		ContextState.DepthStencilState.bTwoSidedStencilMode = PendingState.DepthStencilState.bTwoSidedStencilMode;
	}

	if (PendingState.DepthStencilState.bStencilEnable)
	{
		if (PendingState.DepthStencilState.bTwoSidedStencilMode)
		{
			if (ContextState.DepthStencilState.StencilFunc != PendingState.DepthStencilState.StencilFunc
				|| ContextState.StencilRef != PendingState.StencilRef
				|| ContextState.DepthStencilState.StencilReadMask != PendingState.DepthStencilState.StencilReadMask)
			{
				glStencilFuncSeparate(GL_BACK, PendingState.DepthStencilState.StencilFunc, PendingState.StencilRef, PendingState.DepthStencilState.StencilReadMask);
				ContextState.DepthStencilState.StencilFunc = PendingState.DepthStencilState.StencilFunc;
			}

			if (ContextState.DepthStencilState.StencilFail != PendingState.DepthStencilState.StencilFail
				|| ContextState.DepthStencilState.StencilZFail != PendingState.DepthStencilState.StencilZFail
				|| ContextState.DepthStencilState.StencilPass != PendingState.DepthStencilState.StencilPass)
			{
				glStencilOpSeparate(GL_BACK, PendingState.DepthStencilState.StencilFail, PendingState.DepthStencilState.StencilZFail, PendingState.DepthStencilState.StencilPass);
				ContextState.DepthStencilState.StencilFail = PendingState.DepthStencilState.StencilFail;
				ContextState.DepthStencilState.StencilZFail = PendingState.DepthStencilState.StencilZFail;
				ContextState.DepthStencilState.StencilPass = PendingState.DepthStencilState.StencilPass;
			}

			if (ContextState.DepthStencilState.CCWStencilFunc != PendingState.DepthStencilState.CCWStencilFunc
				|| ContextState.StencilRef != PendingState.StencilRef
				|| ContextState.DepthStencilState.StencilReadMask != PendingState.DepthStencilState.StencilReadMask)
			{
				glStencilFuncSeparate(GL_FRONT, PendingState.DepthStencilState.CCWStencilFunc, PendingState.StencilRef, PendingState.DepthStencilState.StencilReadMask);
				ContextState.DepthStencilState.CCWStencilFunc = PendingState.DepthStencilState.CCWStencilFunc;
			}

			if (ContextState.DepthStencilState.CCWStencilFail != PendingState.DepthStencilState.CCWStencilFail
				|| ContextState.DepthStencilState.CCWStencilZFail != PendingState.DepthStencilState.CCWStencilZFail
				|| ContextState.DepthStencilState.CCWStencilPass != PendingState.DepthStencilState.CCWStencilPass)
			{
				glStencilOpSeparate(GL_FRONT, PendingState.DepthStencilState.CCWStencilFail, PendingState.DepthStencilState.CCWStencilZFail, PendingState.DepthStencilState.CCWStencilPass);
				ContextState.DepthStencilState.CCWStencilFail = PendingState.DepthStencilState.CCWStencilFail;
				ContextState.DepthStencilState.CCWStencilZFail = PendingState.DepthStencilState.CCWStencilZFail;
				ContextState.DepthStencilState.CCWStencilPass = PendingState.DepthStencilState.CCWStencilPass;
			}

			ContextState.DepthStencilState.StencilReadMask = PendingState.DepthStencilState.StencilReadMask;
			ContextState.StencilRef = PendingState.StencilRef;
		}
		else
		{
			if (ContextState.DepthStencilState.StencilFunc != PendingState.DepthStencilState.StencilFunc
				|| ContextState.StencilRef != PendingState.StencilRef
				|| ContextState.DepthStencilState.StencilReadMask != PendingState.DepthStencilState.StencilReadMask)
			{
				glStencilFunc(PendingState.DepthStencilState.StencilFunc, PendingState.StencilRef, PendingState.DepthStencilState.StencilReadMask);
				ContextState.DepthStencilState.StencilFunc = PendingState.DepthStencilState.StencilFunc;
				ContextState.DepthStencilState.StencilReadMask = PendingState.DepthStencilState.StencilReadMask;
				ContextState.StencilRef = PendingState.StencilRef;
			}

			if (ContextState.DepthStencilState.StencilFail != PendingState.DepthStencilState.StencilFail
				|| ContextState.DepthStencilState.StencilZFail != PendingState.DepthStencilState.StencilZFail
				|| ContextState.DepthStencilState.StencilPass != PendingState.DepthStencilState.StencilPass)
			{
				glStencilOp(PendingState.DepthStencilState.StencilFail, PendingState.DepthStencilState.StencilZFail, PendingState.DepthStencilState.StencilPass);
				ContextState.DepthStencilState.StencilFail = PendingState.DepthStencilState.StencilFail;
				ContextState.DepthStencilState.StencilZFail = PendingState.DepthStencilState.StencilZFail;
				ContextState.DepthStencilState.StencilPass = PendingState.DepthStencilState.StencilPass;
			}
		}

		if (ContextState.DepthStencilState.StencilWriteMask != PendingState.DepthStencilState.StencilWriteMask)
		{
			glStencilMask(PendingState.DepthStencilState.StencilWriteMask);
			ContextState.DepthStencilState.StencilWriteMask = PendingState.DepthStencilState.StencilWriteMask;
		}
	}
}

void FOpenGLDynamicRHI::SetPendingBlendStateForActiveRenderTargets()
{
	VERIFY_GL_SCOPE();

	bool bABlendWasSet = false;
	bool bMSAAEnabled = false;

	//
	// Need to expand setting for glBlendFunction and glBlendEquation

	for (uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex)
	{
		if(PendingState.RenderTargets[RenderTargetIndex] == 0)
		{
			// Even if on this stage blend states are incompatible with other stages, we can disregard it, as no render target is assigned to it.
			continue;
		}
		else if (RenderTargetIndex == 0)
		{
			FOpenGLTexture* RenderTarget2D = PendingState.RenderTargets[RenderTargetIndex];
			bMSAAEnabled = PendingState.NumRenderingSamples > 1 || RenderTarget2D->IsMultisampled();
		}

		const FOpenGLBlendStateData::FRenderTarget& RenderTargetBlendState = PendingState.BlendState.RenderTargets[RenderTargetIndex];
		FOpenGLBlendStateData::FRenderTarget& CachedRenderTargetBlendState = ContextState.BlendState.RenderTargets[RenderTargetIndex];

		if (CachedRenderTargetBlendState.bAlphaBlendEnable != RenderTargetBlendState.bAlphaBlendEnable)
		{
			if (RenderTargetBlendState.bAlphaBlendEnable)
			{
				FOpenGL::EnableIndexed(GL_BLEND,RenderTargetIndex);
			}
			else
			{
				FOpenGL::DisableIndexed(GL_BLEND,RenderTargetIndex);
			}
			CachedRenderTargetBlendState.bAlphaBlendEnable = RenderTargetBlendState.bAlphaBlendEnable;
		}

		if (RenderTargetBlendState.bAlphaBlendEnable)
		{
			if ( FOpenGL::SupportsSeparateAlphaBlend() )
			{
				// Set current blend per stage
				if (RenderTargetBlendState.bSeparateAlphaBlendEnable)
				{
					if (CachedRenderTargetBlendState.ColorSourceBlendFactor != RenderTargetBlendState.ColorSourceBlendFactor
						|| CachedRenderTargetBlendState.ColorDestBlendFactor != RenderTargetBlendState.ColorDestBlendFactor
						|| CachedRenderTargetBlendState.AlphaSourceBlendFactor != RenderTargetBlendState.AlphaSourceBlendFactor
						|| CachedRenderTargetBlendState.AlphaDestBlendFactor != RenderTargetBlendState.AlphaDestBlendFactor)
					{
						FOpenGL::BlendFuncSeparatei(
							RenderTargetIndex, 
							RenderTargetBlendState.ColorSourceBlendFactor, RenderTargetBlendState.ColorDestBlendFactor,
							RenderTargetBlendState.AlphaSourceBlendFactor, RenderTargetBlendState.AlphaDestBlendFactor
							);
					}

					if (CachedRenderTargetBlendState.ColorBlendOperation != RenderTargetBlendState.ColorBlendOperation
						|| CachedRenderTargetBlendState.AlphaBlendOperation != RenderTargetBlendState.AlphaBlendOperation)
					{
						FOpenGL::BlendEquationSeparatei(
							RenderTargetIndex, 
							RenderTargetBlendState.ColorBlendOperation,
							RenderTargetBlendState.AlphaBlendOperation
							);
					}
				}
				else
				{
					if (CachedRenderTargetBlendState.ColorSourceBlendFactor != RenderTargetBlendState.ColorSourceBlendFactor
						|| CachedRenderTargetBlendState.ColorDestBlendFactor != RenderTargetBlendState.ColorDestBlendFactor
						|| CachedRenderTargetBlendState.AlphaSourceBlendFactor != RenderTargetBlendState.AlphaSourceBlendFactor
						|| CachedRenderTargetBlendState.AlphaDestBlendFactor != RenderTargetBlendState.AlphaDestBlendFactor)
					{
						FOpenGL::BlendFunci(RenderTargetIndex, RenderTargetBlendState.ColorSourceBlendFactor, RenderTargetBlendState.ColorDestBlendFactor);
					}

					if (CachedRenderTargetBlendState.ColorBlendOperation != RenderTargetBlendState.ColorBlendOperation)
					{
						FOpenGL::BlendEquationi(RenderTargetIndex, RenderTargetBlendState.ColorBlendOperation);
					}
				}

				CachedRenderTargetBlendState.bSeparateAlphaBlendEnable = RenderTargetBlendState.bSeparateAlphaBlendEnable;
				CachedRenderTargetBlendState.ColorBlendOperation = RenderTargetBlendState.ColorBlendOperation;
				CachedRenderTargetBlendState.ColorSourceBlendFactor = RenderTargetBlendState.ColorSourceBlendFactor;
				CachedRenderTargetBlendState.ColorDestBlendFactor = RenderTargetBlendState.ColorDestBlendFactor;
				if( RenderTargetBlendState.bSeparateAlphaBlendEnable )
				{ 
					CachedRenderTargetBlendState.AlphaSourceBlendFactor = RenderTargetBlendState.AlphaSourceBlendFactor;
					CachedRenderTargetBlendState.AlphaDestBlendFactor = RenderTargetBlendState.AlphaDestBlendFactor;
				}
				else
				{
					CachedRenderTargetBlendState.AlphaSourceBlendFactor = RenderTargetBlendState.ColorSourceBlendFactor;
					CachedRenderTargetBlendState.AlphaDestBlendFactor = RenderTargetBlendState.ColorDestBlendFactor;
				}
			}
			else
			{
				if( bABlendWasSet )
				{
					// Detect the case of subsequent render target needing different blend setup than one already set in this call.
					if( CachedRenderTargetBlendState.bSeparateAlphaBlendEnable != RenderTargetBlendState.bSeparateAlphaBlendEnable
						|| CachedRenderTargetBlendState.ColorBlendOperation != RenderTargetBlendState.ColorBlendOperation
						|| CachedRenderTargetBlendState.ColorSourceBlendFactor != RenderTargetBlendState.ColorSourceBlendFactor
						|| CachedRenderTargetBlendState.ColorDestBlendFactor != RenderTargetBlendState.ColorDestBlendFactor
						|| ( RenderTargetBlendState.bSeparateAlphaBlendEnable && 
							( CachedRenderTargetBlendState.AlphaSourceBlendFactor != RenderTargetBlendState.AlphaSourceBlendFactor
							|| CachedRenderTargetBlendState.AlphaDestBlendFactor != RenderTargetBlendState.AlphaDestBlendFactor
							)
							)
						)
						UE_LOGF(LogRHI, Fatal, "OpenGL state on draw requires setting different blend operation or factors to different render targets. This is not supported on Mac OS X!");
				}
				else
				{
					// Set current blend to all stages
					if (RenderTargetBlendState.bSeparateAlphaBlendEnable)
					{
						if (CachedRenderTargetBlendState.ColorSourceBlendFactor != RenderTargetBlendState.ColorSourceBlendFactor
							|| CachedRenderTargetBlendState.ColorDestBlendFactor != RenderTargetBlendState.ColorDestBlendFactor
							|| CachedRenderTargetBlendState.AlphaSourceBlendFactor != RenderTargetBlendState.AlphaSourceBlendFactor
							|| CachedRenderTargetBlendState.AlphaDestBlendFactor != RenderTargetBlendState.AlphaDestBlendFactor)
						{
							glBlendFuncSeparate(
								RenderTargetBlendState.ColorSourceBlendFactor, RenderTargetBlendState.ColorDestBlendFactor,
								RenderTargetBlendState.AlphaSourceBlendFactor, RenderTargetBlendState.AlphaDestBlendFactor
								);
						}

						if (CachedRenderTargetBlendState.ColorBlendOperation != RenderTargetBlendState.ColorBlendOperation
							|| CachedRenderTargetBlendState.AlphaBlendOperation != RenderTargetBlendState.AlphaBlendOperation)
						{
							glBlendEquationSeparate(
								RenderTargetBlendState.ColorBlendOperation,
								RenderTargetBlendState.AlphaBlendOperation
								);
						}
					}
					else
					{
						if (CachedRenderTargetBlendState.ColorSourceBlendFactor != RenderTargetBlendState.ColorSourceBlendFactor
							|| CachedRenderTargetBlendState.ColorDestBlendFactor != RenderTargetBlendState.ColorDestBlendFactor
							|| CachedRenderTargetBlendState.AlphaSourceBlendFactor != RenderTargetBlendState.AlphaSourceBlendFactor
							|| CachedRenderTargetBlendState.AlphaDestBlendFactor != RenderTargetBlendState.AlphaDestBlendFactor)
						{
							glBlendFunc(RenderTargetBlendState.ColorSourceBlendFactor, RenderTargetBlendState.ColorDestBlendFactor);
						}

						if (CachedRenderTargetBlendState.ColorBlendOperation != RenderTargetBlendState.ColorBlendOperation
							|| CachedRenderTargetBlendState.AlphaBlendOperation != RenderTargetBlendState.ColorBlendOperation)
						{
							glBlendEquation(RenderTargetBlendState.ColorBlendOperation);
						}
					}

					// Set cached values of all stages to what they were set by global calls, common to all stages
					for(uint32 RenderTargetIndex2 = 0; RenderTargetIndex2 < MaxSimultaneousRenderTargets; ++RenderTargetIndex2 )
					{
						FOpenGLBlendStateData::FRenderTarget& CachedRenderTargetBlendState2 = ContextState.BlendState.RenderTargets[RenderTargetIndex2];
						CachedRenderTargetBlendState2.bSeparateAlphaBlendEnable = RenderTargetBlendState.bSeparateAlphaBlendEnable;
						CachedRenderTargetBlendState2.ColorBlendOperation = RenderTargetBlendState.ColorBlendOperation;
						CachedRenderTargetBlendState2.ColorSourceBlendFactor = RenderTargetBlendState.ColorSourceBlendFactor;
						CachedRenderTargetBlendState2.ColorDestBlendFactor = RenderTargetBlendState.ColorDestBlendFactor;
						if( RenderTargetBlendState.bSeparateAlphaBlendEnable )
						{
							CachedRenderTargetBlendState2.AlphaBlendOperation = RenderTargetBlendState.AlphaBlendOperation;
							CachedRenderTargetBlendState2.AlphaSourceBlendFactor = RenderTargetBlendState.AlphaSourceBlendFactor;
							CachedRenderTargetBlendState2.AlphaDestBlendFactor = RenderTargetBlendState.AlphaDestBlendFactor;
							CachedRenderTargetBlendState2.AlphaBlendOperation = RenderTargetBlendState.AlphaBlendOperation;
						}
						else
						{
							CachedRenderTargetBlendState2.AlphaBlendOperation = RenderTargetBlendState.ColorBlendOperation;
							CachedRenderTargetBlendState2.AlphaSourceBlendFactor = RenderTargetBlendState.ColorSourceBlendFactor;
							CachedRenderTargetBlendState2.AlphaDestBlendFactor = RenderTargetBlendState.ColorDestBlendFactor;
							CachedRenderTargetBlendState2.AlphaBlendOperation = RenderTargetBlendState.ColorBlendOperation;
						}
					}

					bABlendWasSet = true;
				}
			}
		}

		CachedRenderTargetBlendState.bSeparateAlphaBlendEnable = RenderTargetBlendState.bSeparateAlphaBlendEnable;

		if(CachedRenderTargetBlendState.ColorWriteMaskR != RenderTargetBlendState.ColorWriteMaskR
			|| CachedRenderTargetBlendState.ColorWriteMaskG != RenderTargetBlendState.ColorWriteMaskG
			|| CachedRenderTargetBlendState.ColorWriteMaskB != RenderTargetBlendState.ColorWriteMaskB
			|| CachedRenderTargetBlendState.ColorWriteMaskA != RenderTargetBlendState.ColorWriteMaskA)
		{
			FOpenGL::ColorMaskIndexed(
				RenderTargetIndex,
				RenderTargetBlendState.ColorWriteMaskR,
				RenderTargetBlendState.ColorWriteMaskG,
				RenderTargetBlendState.ColorWriteMaskB,
				RenderTargetBlendState.ColorWriteMaskA
				);

			CachedRenderTargetBlendState.ColorWriteMaskR = RenderTargetBlendState.ColorWriteMaskR;
			CachedRenderTargetBlendState.ColorWriteMaskG = RenderTargetBlendState.ColorWriteMaskG;
			CachedRenderTargetBlendState.ColorWriteMaskB = RenderTargetBlendState.ColorWriteMaskB;
			CachedRenderTargetBlendState.ColorWriteMaskA = RenderTargetBlendState.ColorWriteMaskA;
		}
	}

	PendingState.bAlphaToCoverageEnabled = bMSAAEnabled && PendingState.BlendState.bUseAlphaToCoverage;
	if (PendingState.bAlphaToCoverageEnabled != ContextState.bAlphaToCoverageEnabled)
	{
		if (PendingState.bAlphaToCoverageEnabled)
		{
			glEnable(GL_SAMPLE_ALPHA_TO_COVERAGE);
		}
		else
		{
			glDisable(GL_SAMPLE_ALPHA_TO_COVERAGE);
		}

		ContextState.bAlphaToCoverageEnabled = PendingState.bAlphaToCoverageEnabled;
	}
}

void FOpenGLDynamicRHI::RHISetBlendState(FRHIBlendState* NewStateRHI,const FLinearColor& BlendFactor)
{
	VERIFY_GL_SCOPE();
	FOpenGLBlendState* NewState = ResourceCast(NewStateRHI);
	FMemory::Memcpy(&PendingState.BlendState,&(NewState->Data),sizeof(FOpenGLBlendStateData));
}

void FOpenGLDynamicRHI::SetRenderTargets(
	uint32 NumSimultaneousRenderTargets,
	const FRHIRenderTargetView* NewRenderTargetsRHI,
	const FRHIDepthRenderTargetView* NewDepthStencilTargetRHI)
{
	VERIFY_GL_SCOPE();
	check(NumSimultaneousRenderTargets <= MaxSimultaneousRenderTargets);

	FMemory::Memset(PendingState.RenderTargets,0,sizeof(PendingState.RenderTargets));
	FMemory::Memset(PendingState.RenderTargetMipmapLevels,0,sizeof(PendingState.RenderTargetMipmapLevels));
	FMemory::Memset(PendingState.RenderTargetArrayIndex,0,sizeof(PendingState.RenderTargetArrayIndex));
	PendingState.FirstNonzeroRenderTarget = -1;

	for( int32 RenderTargetIndex = NumSimultaneousRenderTargets - 1; RenderTargetIndex >= 0; --RenderTargetIndex )
	{
		PendingState.RenderTargets[RenderTargetIndex] = ResourceCast(NewRenderTargetsRHI[RenderTargetIndex].Texture);
		PendingState.RenderTargetMipmapLevels[RenderTargetIndex] = NewRenderTargetsRHI[RenderTargetIndex].MipIndex;
		PendingState.RenderTargetArrayIndex[RenderTargetIndex] = NewRenderTargetsRHI[RenderTargetIndex].ArraySliceIndex;

		if( PendingState.RenderTargets[RenderTargetIndex] )
		{
			PendingState.FirstNonzeroRenderTarget = (int32)RenderTargetIndex;
		}
	}

	FOpenGLTexture* NewDepthStencilRT = ResourceCast(NewDepthStencilTargetRHI ? NewDepthStencilTargetRHI->Texture : nullptr);
	FExclusiveDepthStencil NewDepthStencilAccess = NewDepthStencilTargetRHI ? NewDepthStencilTargetRHI->GetDepthStencilAccess() : FExclusiveDepthStencil();
		
	PendingState.DepthStencil = NewDepthStencilRT;
	PendingState.StencilStoreAction = NewDepthStencilTargetRHI ? NewDepthStencilTargetRHI->GetStencilStoreAction() : ERenderTargetStoreAction::ENoAction;
	PendingState.DepthTargetWidth   = NewDepthStencilRT ? NewDepthStencilRT->GetDesc().Extent.X : 0u;
	PendingState.DepthTargetHeight  = NewDepthStencilRT ? NewDepthStencilRT->GetDesc().Extent.Y : 0u;
	
	if (PendingState.FirstNonzeroRenderTarget == -1 && !PendingState.DepthStencil)
	{
		// Special case - invalid setup, but sometimes performed by the engine

		PendingState.Framebuffer = 0;
		PendingState.bFramebufferSetupInvalid = true;
		return;
	}

	PendingState.Framebuffer = GetOpenGLFramebuffer(NumSimultaneousRenderTargets, PendingState.RenderTargets, PendingState.RenderTargetArrayIndex, PendingState.RenderTargetMipmapLevels, PendingState.DepthStencil, NewDepthStencilAccess, PendingState.NumRenderingSamples);
	PendingState.bFramebufferSetupInvalid = false;

	if (PendingState.FirstNonzeroRenderTarget != -1)
	{
		// Set viewport size to new render target size.
		PendingState.Viewport.Min.X = 0;
		PendingState.Viewport.Min.Y = 0;

		const FRHITextureDesc& Desc = NewRenderTargetsRHI[PendingState.FirstNonzeroRenderTarget].Texture->GetDesc();

		uint32 MipIndex = NewRenderTargetsRHI[PendingState.FirstNonzeroRenderTarget].MipIndex;
		uint32 Width    = FMath::Max<uint32>(1, Desc.Extent.X >> MipIndex);
		uint32 Height   = FMath::Max<uint32>(1, Desc.Extent.Y >> MipIndex);

		PendingState.Viewport.Max.X = PendingState.RenderTargetWidth = Width;
		PendingState.Viewport.Max.Y = PendingState.RenderTargetHeight = Height;
	}
	else if (NewDepthStencilTargetRHI)
	{
		// Set viewport size to new depth target size.
		PendingState.Viewport.Min.X = 0;
		PendingState.Viewport.Min.Y = 0;
		PendingState.Viewport.Max.X = NewDepthStencilTargetRHI->Texture->GetDesc().Extent.X;
		PendingState.Viewport.Max.Y = NewDepthStencilTargetRHI->Texture->GetDesc().Extent.Y;
	}
}

void FOpenGLDynamicRHI::DiscardRenderTargets(bool Depth, bool Stencil, uint32 ColorBitMaskIn)
{
	if (FOpenGL::SupportsDiscardFrameBuffer())
	{
		VERIFY_GL_SCOPE();
		const bool bDefaultFramebuffer = (PendingState.Framebuffer == 0);
		uint32 ColorBitMask = ColorBitMaskIn;
		// 8 Color + Depth + Stencil = 10
		GLenum Attachments[MaxSimultaneousRenderTargets + 2];
		uint32 I = 0;
		if (Depth)
		{
			Attachments[I] = bDefaultFramebuffer ? GL_DEPTH : GL_DEPTH_ATTACHMENT;
			I++;
		}
		if (Stencil)
		{
			Attachments[I] = bDefaultFramebuffer ? GL_STENCIL : GL_STENCIL_ATTACHMENT;
			I++;
		}

		if (bDefaultFramebuffer)
		{
			if (ColorBitMask)
			{
				Attachments[I] = GL_COLOR;
				I++;
			}
		}
		else
		{
			ColorBitMask &= (1 << MaxSimultaneousRenderTargets) - 1;
			uint32 J = 0;
			while (ColorBitMask)
			{
				if (ColorBitMask & 1)
				{
					Attachments[I] = GL_COLOR_ATTACHMENT0 + J;
					I++;
				}

				ColorBitMask >>= 1;
				++J;
			}
		}

		FOpenGL::InvalidateFramebuffer(GL_FRAMEBUFFER, I, Attachments);
	}
}

void FOpenGLDynamicRHI::SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo)
{
	VERIFY_GL_SCOPE();
	this->SetRenderTargets(RenderTargetsInfo.NumColorRenderTargets,
		RenderTargetsInfo.ColorRenderTarget,
		&RenderTargetsInfo.DepthStencilRenderTarget);

	/**
	 * Convert all load action from NoAction to Clear for tiled GPU on OpenGL platform to avoid an unnecessary load action.
	 */

	bool bIsTiledGPU = RHIHasTiledGPU(GetFeatureLevelShaderPlatform(FOpenGL::GetFeatureLevel()));

	bool bClearColor = RenderTargetsInfo.bClearColor;
	bool bClearStencil = RenderTargetsInfo.bClearStencil;
	bool bClearDepth = RenderTargetsInfo.bClearDepth;

	FLinearColor ClearColors[MaxSimultaneousRenderTargets];
	bool bClearColorArray[MaxSimultaneousRenderTargets];
	float DepthClear = 0.0;
	uint32 StencilClear = 0;

	for (int32 i = 0; i < RenderTargetsInfo.NumColorRenderTargets; ++i)
	{
		bClearColorArray[i] = RenderTargetsInfo.ColorRenderTarget[i].LoadAction == ERenderTargetLoadAction::EClear;

		if (RenderTargetsInfo.ColorRenderTarget[i].Texture != nullptr)
		{
			const FClearValueBinding& ClearValue = RenderTargetsInfo.ColorRenderTarget[i].Texture->GetClearBinding();

			if (bIsTiledGPU)
			{
				const bool bNoAction = RenderTargetsInfo.ColorRenderTarget[i].LoadAction == ERenderTargetLoadAction::ENoAction;

				bClearColor |= bNoAction;
				bClearColorArray[i] |= bNoAction;

				ClearColors[i] = ClearValue.ColorBinding == EClearBinding::EColorBound ? ClearValue.GetClearColor() : FLinearColor::Black;
			}
			else if(bClearColorArray[i])
			{
				checkf(ClearValue.ColorBinding == EClearBinding::EColorBound, TEXT("Texture: %s does not have a color bound for fast clears"), *RenderTargetsInfo.ColorRenderTarget[i].Texture->GetName().GetPlainNameString());

				ClearColors[i] = ClearValue.GetClearColor();
			}
		}
	}

	if (RenderTargetsInfo.DepthStencilRenderTarget.Texture != nullptr)
	{
		const FClearValueBinding& ClearValue = RenderTargetsInfo.DepthStencilRenderTarget.Texture->GetClearBinding();

		if (bIsTiledGPU)
		{
			bClearStencil |= RenderTargetsInfo.DepthStencilRenderTarget.StencilLoadAction == ERenderTargetLoadAction::ENoAction;

			bClearDepth |= RenderTargetsInfo.DepthStencilRenderTarget.DepthLoadAction == ERenderTargetLoadAction::ENoAction;

			if (ClearValue.ColorBinding == EClearBinding::EDepthStencilBound)
			{
				ClearValue.GetDepthStencil(DepthClear, StencilClear);
			}
		}
		else if (bClearDepth || bClearStencil)
		{
			checkf(ClearValue.ColorBinding == EClearBinding::EDepthStencilBound, TEXT("Texture: %s does not have a DS value bound for fast clears"), *RenderTargetsInfo.DepthStencilRenderTarget.Texture->GetName().GetPlainNameString());

			ClearValue.GetDepthStencil(DepthClear, StencilClear);
		}
	}

	if (bClearColor || bClearStencil || bClearDepth)
	{
		this->RHIClearMRT(bClearColor ? bClearColorArray : nullptr, RenderTargetsInfo.NumColorRenderTargets, ClearColors, bClearDepth, DepthClear, bClearStencil, StencilClear);
	}
}

// Primitive drawing.

void FOpenGLDynamicRHI::SetupVertexArrays(uint32 BaseVertexIndex, FOpenGLStream* Streams, uint32 NumStreams, uint32 MaxVertices)
{
	VERIFY_GL_SCOPE();
	bool KnowsDivisor[NUM_OPENGL_VERTEX_STREAMS] = { 0 };
	uint32 Divisor[NUM_OPENGL_VERTEX_STREAMS] = { 0 };
	bool UpdateDivisors = false;
	uint32 StreamMask = ContextState.ActiveStreamMask;

	check(IsValidRef(PendingState.BoundShaderState));
	FOpenGLVertexDeclaration* VertexDeclaration = PendingState.BoundShaderState->VertexDeclaration;
	const CrossCompiler::FShaderBindingInOutMask& AttributeMask = PendingState.BoundShaderState->GetVertexShader()->Bindings.InOutMask;

	if (ContextState.VertexDecl != VertexDeclaration || AttributeMask.Bitmask != ContextState.VertexAttrs_EnabledBits)
	{
		StreamMask = 0;
		UpdateDivisors = true;

		check(VertexDeclaration->VertexElements.Num() <= 32);

		for (int32 ElementIndex = 0; ElementIndex < VertexDeclaration->VertexElements.Num(); ElementIndex++)
		{
			FOpenGLVertexElement& VertexElement = VertexDeclaration->VertexElements[ElementIndex];
			uint32 AttributeIndex = VertexElement.AttributeIndex;
			const uint32 StreamIndex = VertexElement.StreamIndex;

			//only setup/track attributes actually in use
			FOpenGLCachedAttr& Attr = ContextState.VertexAttrs[AttributeIndex];
			if (AttributeMask.IsFieldEnabled(AttributeIndex))
			{
				if (VertexElement.StreamIndex < NumStreams)
				{
					// Track the actively used streams, to limit the updates to those in use
					StreamMask |= 0x1 << VertexElement.StreamIndex;

					// Verify that the Divisor is consistent across the stream
					check(!KnowsDivisor[StreamIndex] || Divisor[StreamIndex] == VertexElement.Divisor);
					KnowsDivisor[StreamIndex] = true;
					Divisor[StreamIndex] = VertexElement.Divisor;

					if ((Attr.StreamOffset != VertexElement.Offset) || //-V1013
						(Attr.Size != VertexElement.Size) ||
						(Attr.Type != VertexElement.Type) ||
						(Attr.bNormalized != VertexElement.bNormalized) ||
						(Attr.bShouldConvertToFloat != VertexElement.bShouldConvertToFloat))
					{
						if (!VertexElement.bShouldConvertToFloat)
						{
							FOpenGL::VertexAttribIFormat(AttributeIndex, VertexElement.Size, VertexElement.Type, VertexElement.Offset);
						}
						else
						{
							FOpenGL::VertexAttribFormat(AttributeIndex, VertexElement.Size, VertexElement.Type, VertexElement.bNormalized, VertexElement.Offset);
						}

						Attr.StreamOffset = VertexElement.Offset;
						Attr.Size = VertexElement.Size;
						Attr.Type = VertexElement.Type;
						Attr.bNormalized = VertexElement.bNormalized;
						Attr.bShouldConvertToFloat = VertexElement.bShouldConvertToFloat;
					}

					if (Attr.StreamIndex != StreamIndex)
					{
						FOpenGL::VertexAttribBinding(AttributeIndex, VertexElement.StreamIndex);
						Attr.StreamIndex = StreamIndex;
					}

					if (!ContextState.GetVertexAttrEnabled(AttributeIndex))
					{
						ContextState.SetVertexAttrEnabled(AttributeIndex, true);
						glEnableVertexAttribArray(AttributeIndex);
					}
				}
				else
				{
					//workaround attributes with no streams
					VERIFY_GL_SCOPE();

					if (ContextState.GetVertexAttrEnabled(AttributeIndex))
					{		
						ContextState.SetVertexAttrEnabled(AttributeIndex, false);
						glDisableVertexAttribArray(AttributeIndex);
					}
					static float data[4] = { 0.0f };
					glVertexAttrib4fv(AttributeIndex, data);
				}
			}
			else
			{
				if (Attr.StreamIndex != StreamIndex)
				{
					FOpenGL::VertexAttribBinding(AttributeIndex, VertexElement.StreamIndex);
					Attr.StreamIndex = StreamIndex;
				}
			}
		}
		ContextState.VertexDecl = VertexDeclaration;
	}

	// Disable remaining vertex arrays
	uint32 NotUsedButEnabledAttrMask = (ContextState.VertexAttrs_EnabledBits & ~(AttributeMask.Bitmask));
	for (GLuint AttribIndex = 0; AttribIndex < NUM_OPENGL_VERTEX_STREAMS && NotUsedButEnabledAttrMask; AttribIndex++)
	{
		if (NotUsedButEnabledAttrMask & 1)
		{
			glDisableVertexAttribArray(AttribIndex);
			ContextState.SetVertexAttrEnabled(AttribIndex, false);
		}
		NotUsedButEnabledAttrMask >>= 1;
	}

	// Active streams that are no used by this draw
	uint32 NotUsedButActiveStreamMask = (ContextState.ActiveStreamMask & ~(StreamMask));
	
	// Update the stream mask
	ContextState.ActiveStreamMask = StreamMask;

	// Enable used streams
	for (uint32 StreamIndex = 0; StreamIndex < NumStreams && StreamMask; StreamIndex++)
	{
		if (StreamMask & 0x1)
		{
			FOpenGLStream &CachedStream = ContextState.VertexStreams[StreamIndex];
			FOpenGLStream &Stream = Streams[StreamIndex];
			if (Stream.VertexBufferResource)
			{
				uint32 Offset = BaseVertexIndex * Stream.Stride + Stream.Offset;
				bool bAnyDifferent = //bitwise ors to get rid of the branches
					(CachedStream.VertexBufferResource != Stream.VertexBufferResource) ||
					(CachedStream.Stride != Stream.Stride)||
					(CachedStream.Offset != Offset);

				if (bAnyDifferent)
				{
					check(Stream.VertexBufferResource != 0);
					FOpenGL::BindVertexBuffer(StreamIndex, Stream.VertexBufferResource, Offset, Stream.Stride);
					CachedStream.VertexBufferResource = Stream.VertexBufferResource;
					CachedStream.Offset = Offset;
					CachedStream.Stride = Stream.Stride;
				}
				if (UpdateDivisors && CachedStream.Divisor != Divisor[StreamIndex])
				{
					FOpenGL::VertexBindingDivisor(StreamIndex, Divisor[StreamIndex]);
					CachedStream.Divisor = Divisor[StreamIndex];
				}
			}
			else
			{
				UE_LOGF(LogRHI, Error, "Stream %d marked as in use, but vertex buffer provided is NULL (Mask = %x)", StreamIndex, StreamMask);
				
				FOpenGL::BindVertexBuffer(StreamIndex, 0, 0, 0);
				CachedStream.VertexBufferResource = 0;
				CachedStream.Offset = 0;
				CachedStream.Stride = 0;
			}
		}
		StreamMask >>= 1;
	}
	//Ensure that all requested streams were set
	check(StreamMask == 0);

	// Disable active unused streams
	for (uint32 StreamIndex = 0; StreamIndex < NUM_OPENGL_VERTEX_STREAMS && NotUsedButActiveStreamMask; StreamIndex++)
	{
		if (NotUsedButActiveStreamMask & 0x1)
		{
			FOpenGL::BindVertexBuffer(StreamIndex, 0, 0, 0);
			ContextState.VertexStreams[StreamIndex].VertexBufferResource = 0;
			ContextState.VertexStreams[StreamIndex].Offset = 0;
			ContextState.VertexStreams[StreamIndex].Stride = 0;
		}
		NotUsedButActiveStreamMask >>= 1;
	}
	check(NotUsedButActiveStreamMask == 0);
}

void FOpenGLDynamicRHI::OnProgramDeletion(GLint ProgramResource)
{
	VERIFY_GL_SCOPE();
	if (ContextState.Program == ProgramResource)
	{
		ContextState.Program = -1;
	}
}

void FOpenGLDynamicRHI::OnBufferDeletion(GLenum Type, GLuint BufferResource)
{
	VERIFY_GL_SCOPE();
	if (Type == GL_PIXEL_UNPACK_BUFFER)
	{
		if (ContextState.PixelUnpackBufferBound == BufferResource)
		{
			ContextState.PixelUnpackBufferBound = -1;	// will force refresh
		}
	}
	else
	{
		if (ContextState.ArrayBufferBound == BufferResource)
		{
			ContextState.ArrayBufferBound = -1; // will force refresh
		}

		if (ContextState.StorageBufferBound == BufferResource)
		{
			ContextState.StorageBufferBound = -1; // will force refresh
		}
	
		// loop through active streams
		uint32 ActiveStreamMask = ContextState.ActiveStreamMask;
		for (GLuint StreamIndex = 0; StreamIndex < NUM_OPENGL_VERTEX_STREAMS && ActiveStreamMask; StreamIndex++)
		{
			FOpenGLStream& CachedStream = ContextState.VertexStreams[StreamIndex];
			if ((ActiveStreamMask & 0x1) && 
				CachedStream.VertexBufferResource == BufferResource)
			{
				FOpenGL::BindVertexBuffer(StreamIndex, 0, 0, 0); // brianh@nvidia: work around driver bug 1809000
				CachedStream.VertexBufferResource = 0;
				CachedStream.Offset = 0;
				CachedStream.Stride = 0;
			}
			ActiveStreamMask >>= 1;
		}	
	
		// Storage buffer
		{
			int32 NumStorageBuffers = PendingState.UAVs.Num();
			for (int32 StorageBufferIndex = 0; StorageBufferIndex < NumStorageBuffers; StorageBufferIndex++)
			{
				if (PendingState.UAVs[StorageBufferIndex].Format == 0 && PendingState.UAVs[StorageBufferIndex].Resource == BufferResource)
				{
					PendingState.UAVs[StorageBufferIndex].Resource = 0;
				}
			}
		}

		if (ContextState.ElementArrayBufferBound == BufferResource)
		{
			ContextState.ElementArrayBufferBound = -1;	// will force refresh
		}
	}
}

void FOpenGLDynamicRHI::CommitNonComputeShaderConstants()
{
	VERIFY_GL_SCOPE();

	FOpenGLLinkedProgram* LinkedProgram = PendingState.BoundShaderState->LinkedProgram;
	if (GUseEmulatedUniformBuffers)
	{
		PendingState.ShaderParameters[CrossCompiler::SHADER_STAGE_VERTEX].CommitPackedUniformBuffers(LinkedProgram, CrossCompiler::SHADER_STAGE_VERTEX, PendingState.BoundUniformBuffers[SF_Vertex], PendingState.BoundShaderState->GetVertexShader()->UniformBuffersCopyInfo);
		PendingState.ShaderParameters[CrossCompiler::SHADER_STAGE_PIXEL].CommitPackedUniformBuffers(LinkedProgram, CrossCompiler::SHADER_STAGE_PIXEL, PendingState.BoundUniformBuffers[SF_Pixel], PendingState.BoundShaderState->GetPixelShader()->UniformBuffersCopyInfo);
		if (PendingState.BoundShaderState->GetGeometryShader())
		{
			PendingState.ShaderParameters[CrossCompiler::SHADER_STAGE_GEOMETRY].CommitPackedUniformBuffers(LinkedProgram, CrossCompiler::SHADER_STAGE_GEOMETRY, PendingState.BoundUniformBuffers[SF_Geometry], PendingState.BoundShaderState->GetGeometryShader()->UniformBuffersCopyInfo);
		}
	}
	
	if (LinkedProgram == PendingState.LinkedProgramAndDirtyFlag)
	{
		return;
	}
	
	// commit packed global only if current program has changed or any global parameter has changed (RHISetShaderParameter)
	PendingState.ShaderParameters[CrossCompiler::SHADER_STAGE_VERTEX].CommitPackedGlobals(LinkedProgram, CrossCompiler::SHADER_STAGE_VERTEX);
	PendingState.ShaderParameters[CrossCompiler::SHADER_STAGE_PIXEL].CommitPackedGlobals(LinkedProgram, CrossCompiler::SHADER_STAGE_PIXEL);
	if (PendingState.BoundShaderState->GetGeometryShader())
	{
		PendingState.ShaderParameters[CrossCompiler::SHADER_STAGE_GEOMETRY].CommitPackedGlobals(LinkedProgram, CrossCompiler::SHADER_STAGE_GEOMETRY);
	}

	PendingState.LinkedProgramAndDirtyFlag = LinkedProgram;
}

void FOpenGLDynamicRHI::CommitComputeShaderConstants(FOpenGLComputeShader* ComputeShader)
{
	VERIFY_GL_SCOPE();

	const CrossCompiler::EShaderStage Stage = CrossCompiler::SHADER_STAGE_COMPUTE;
	FOpenGLShaderParameterCache& StageShaderParameters = PendingState.ShaderParameters[Stage];

	StageShaderParameters.CommitPackedUniformBuffers(ComputeShader->LinkedProgram, Stage, PendingState.BoundUniformBuffers[Stage], ComputeShader->UniformBuffersCopyInfo);
	StageShaderParameters.CommitPackedGlobals(ComputeShader->LinkedProgram, Stage);
	PendingState.LinkedProgramAndDirtyFlag = nullptr;
}

template <class ShaderType>
FORCEINLINE void FOpenGLDynamicRHI::SetResourcesFromTables(ShaderType* Shader)
{
	checkSlow(Shader);
	VERIFY_GL_SCOPE();

	static constexpr EShaderFrequency Frequency = ShaderType::Frequency;

	UE::RHI::Private::SetUniformBufferResourcesFromTables(
		FOpenGLResourceBinder(*this, Frequency, true)
		, *Shader
		, PendingState.DirtyUniformBuffers[Frequency]
		, PendingState.BoundUniformBuffers[Frequency]
#if ENABLE_RHI_VALIDATION
		, Tracker
#endif
	);
}


void FOpenGLDynamicRHI::CommitGraphicsResourceTablesInner()
{
	VERIFY_GL_SCOPE();

	if (PendingState.DirtyUniformBuffers[SF_Vertex])
	{
		if (auto* Shader = PendingState.BoundShaderState->GetVertexShader())
		{
			SetResourcesFromTables(Shader);
		}
	}
	if (PendingState.DirtyUniformBuffers[SF_Pixel])
	{
		if (auto* Shader = PendingState.BoundShaderState->GetPixelShader())
		{
			SetResourcesFromTables(Shader);
		}
	}
	if (PendingState.DirtyUniformBuffers[SF_Geometry])
	{
		if (auto* Shader = PendingState.BoundShaderState->GetGeometryShader())
		{
			SetResourcesFromTables(Shader);
		}
	}

	PendingState.bAnyDirtyGraphicsUniformBuffers = false;
	PendingState.DirtyUniformBuffers[SF_Vertex] = 0;
	PendingState.DirtyUniformBuffers[SF_Pixel] = 0;
	PendingState.DirtyUniformBuffers[SF_Geometry] = 0;
}


void FOpenGLDynamicRHI::CommitComputeResourceTables(FOpenGLComputeShader* ComputeShader)
{
	VERIFY_GL_SCOPE();

	check(ComputeShader);
	SetResourcesFromTables(ComputeShader);
	PendingState.DirtyUniformBuffers[SF_Compute] = 0;
}

FOpenGLDynamicRHI::FOpenGLFenceKick::~FOpenGLFenceKick()
{
	Reset();
}

void FOpenGLDynamicRHI::FOpenGLFenceKick::OnDrawCall()
{
	if (OpenGLConsoleVariables::GOpenGLFenceKickPerDrawCount)
	{
		// we kick every GOpenGLFenceKickPerDrawCount draws within a render pass.
		// counter is reset when the FBO changes.
		if (RHI.ContextState.Framebuffer != LastSeenFramebuffer)
		{
			LastSeenFramebuffer = RHI.ContextState.Framebuffer;
			DrawCounter = 0;
		}
		else if (++DrawCounter >= OpenGLConsoleVariables::GOpenGLFenceKickPerDrawCount)
		{
			InsertKick();
			DrawCounter = 0;
		}
	}
}

void FOpenGLDynamicRHI::FOpenGLFenceKick::Reset()
{
	for (UGLsync Sync : Syncs)
	{
		FOpenGL::DeleteSync(Sync);
	}
	Syncs.Reset();
	DrawCounter = 0;
	LastSeenFramebuffer = 0;
}

void FOpenGLDynamicRHI::FOpenGLFenceKick::InsertKick()
{
	// record syncs incase a create/destroy pair is optimised by the driver.
	UGLsync Sync = FOpenGL::FenceSync(GL_SYNC_GPU_COMMANDS_COMPLETE, 0);
	Syncs.Add(Sync);
	check(Syncs.Num() < 10000); // arbitrary check to ensure that we do eventually reset the array.
}

void FOpenGLDynamicRHI::RHIDrawPrimitive(uint32 BaseVertexIndex,uint32 NumPrimitives,uint32 NumInstances)
{
	VERIFY_GL_SCOPE();
	SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLDrawPrimitiveTime);

	uint32 VertexCount = GetVertexCountForPrimitiveCount(NumPrimitives, PrimitiveType);

	RHI_DRAW_CALL_STATS(PrimitiveType, VertexCount, NumPrimitives, NumInstances);

	SetupDraw(nullptr, BaseVertexIndex, VertexCount);

	GLenum DrawMode = GL_TRIANGLES;
	GLsizei NumElements = 0;
	FindPrimitiveType(PrimitiveType, NumPrimitives, DrawMode, NumElements);

	if (NumInstances == 1)
	{
		SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLDrawPrimitiveDriverTime);
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderFirstDrawTime, PendingState.BoundShaderState->RequiresDriverInstantiation());

		glDrawArrays(DrawMode, 0, NumElements);
	}
	else
	{
		SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLDrawPrimitiveDriverTime);
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderFirstDrawTime, PendingState.BoundShaderState->RequiresDriverInstantiation());

		FOpenGL::DrawArraysInstanced(DrawMode, 0, NumElements, NumInstances);
	}

	KickHint.OnDrawCall();
}

void FOpenGLDynamicRHI::RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	if (!FOpenGL::SupportsDrawIndirect())
	{
		UE_LOGF(LogRHI, Fatal, "OpenGL RHI does not yet support indirect draw calls.");
		return;
	}

	VERIFY_GL_SCOPE();

	check(ArgumentBufferRHI);

	// Zero-stride buffer emulation won't work here, need to use VAB with proper zero strides
	SetupDraw(nullptr, 0, 1);

	GLenum DrawMode = GL_TRIANGLES;
	GLsizei NumElements = 0;
	FindPrimitiveType(PrimitiveType, 0, DrawMode, NumElements);

	FOpenGLBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);
	glBindBuffer( GL_DRAW_INDIRECT_BUFFER, ArgumentBuffer->Resource);

	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderFirstDrawTime, PendingState.BoundShaderState->RequiresDriverInstantiation());
		FOpenGL::DrawArraysIndirect( DrawMode, INDEX_TO_VOID(ArgumentOffset));
	}

	glBindBuffer(GL_DRAW_INDIRECT_BUFFER, 0);

	KickHint.OnDrawCall();
}

void FOpenGLDynamicRHI::RHIDrawIndexedPrimitive(FRHIBuffer* IndexBufferRHI, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances)
{
	SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLDrawPrimitiveTime);
	VERIFY_GL_SCOPE();

	FOpenGLBuffer* IndexBuffer = ResourceCast(IndexBufferRHI);

	RHI_DRAW_CALL_STATS(PrimitiveType, NumVertices, NumPrimitives, NumInstances);

	SetupDraw(IndexBuffer, BaseVertexIndex, NumVertices + StartIndex);

	GLenum DrawMode = GL_TRIANGLES;
	GLsizei NumElements = 0;
	FindPrimitiveType(PrimitiveType, NumPrimitives, DrawMode, NumElements);

	GLenum IndexType = IndexBuffer->GetStride() == sizeof(uint32) ? GL_UNSIGNED_INT : GL_UNSIGNED_SHORT;
	StartIndex *= IndexBuffer->GetStride() == sizeof(uint32) ? sizeof(uint32) : sizeof(uint16);

	if (NumInstances > 1)
	{
		SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLDrawPrimitiveDriverTime);
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderFirstDrawTime, PendingState.BoundShaderState->RequiresDriverInstantiation());
		checkf(FirstInstance  == 0, TEXT("FirstInstance is currently unsupported on this RHI"));
		FOpenGL::DrawElementsInstanced(DrawMode, NumElements, IndexType, INDEX_TO_VOID(StartIndex), NumInstances);
	}
	else
	{
		SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLDrawPrimitiveDriverTime);
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderFirstDrawTime, PendingState.BoundShaderState->RequiresDriverInstantiation());
		if (FOpenGL::SupportsDrawIndexOffset())
		{
			FOpenGL::DrawRangeElements(DrawMode, 0, NumVertices, NumElements, IndexType, INDEX_TO_VOID(StartIndex));
		}
		else
		{
			glDrawElements(DrawMode, NumElements, IndexType, INDEX_TO_VOID(StartIndex));
		}
	}
	KickHint.OnDrawCall();
}

void FOpenGLDynamicRHI::RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBufferRHI, FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	if (!FOpenGL::SupportsDrawIndirect())
	{
		UE_LOGF(LogRHI, Fatal, "OpenGL RHI does not yet support indirect draw calls.");
		return;
	}

	VERIFY_GL_SCOPE();

	FOpenGLBuffer* IndexBuffer = ResourceCast(IndexBufferRHI);

	check(ArgumentBufferRHI);

	// @ToDo Zero-stride buffer emulation won't work here, need to use VAB with proper zero strides
	SetupDraw(IndexBuffer, 0, 1);

	GLenum DrawMode = GL_TRIANGLES;
	GLsizei NumElements = 0;
	FindPrimitiveType(PrimitiveType, 0, DrawMode, NumElements);

	GLenum IndexType = IndexBuffer->GetStride() == sizeof(uint32)	
		? GL_UNSIGNED_INT
		: GL_UNSIGNED_SHORT;

	FOpenGLBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);
	glBindBuffer(GL_DRAW_INDIRECT_BUFFER, ArgumentBuffer->Resource);

	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderFirstDrawTime, PendingState.BoundShaderState->RequiresDriverInstantiation());
		
		// Offset is based on an index into the list of structures
		FOpenGL::DrawElementsIndirect( DrawMode, IndexType, INDEX_TO_VOID(ArgumentOffset));
	}

	glBindBuffer(GL_DRAW_INDIRECT_BUFFER, 0);

	KickHint.OnDrawCall();
}

void FOpenGLDynamicRHI::SetupDraw(FOpenGLBuffer* IndexBuffer, uint32 BaseVertexIndex, uint32 MaxVertices)
{
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_BindPendingFramebuffer);
		BindPendingFramebuffer();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_SetPendingBlendStateForActiveRenderTargets);
		SetPendingBlendStateForActiveRenderTargets();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateViewportInOpenGLContext);
		UpdateViewportInOpenGLContext();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateScissorRectInOpenGLContext);
		UpdateScissorRectInOpenGLContext();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateRasterizerStateInOpenGLContext);
		UpdateRasterizerStateInOpenGLContext();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateDepthStencilStateInOpenGLContext);
		UpdateDepthStencilStateInOpenGLContext();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_BindPendingShaderState);
		BindPendingShaderState();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_CommitGraphicsResourceTables);
		CommitGraphicsResourceTables();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_SetupTexturesForDraw);
		SetupTexturesForDraw();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_SetupUAVsForDraw);
		SetupUAVsForDraw();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_CommitNonComputeShaderConstants);
		CommitNonComputeShaderConstants();
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_CachedBindBuffer);
		CachedBindBuffer(GL_ELEMENT_ARRAY_BUFFER, IndexBuffer ? IndexBuffer->Resource : 0);
	}
	{
		DETAILED_QUICK_SCOPE_CYCLE_COUNTER(STAT_SetupVertexArrays);
		SetupVertexArrays(BaseVertexIndex, PendingState.Streams, NUM_OPENGL_VERTEX_STREAMS, MaxVertices);
	}
}

// Raster operations.
static inline void ClearCurrentDepthStencilWithCurrentScissor( int8 ClearType, float Depth, uint32 Stencil )
{
	switch (ClearType)
	{
	case CT_DepthStencil:	// Clear depth and stencil
		FOpenGL::ClearBufferfi(GL_DEPTH_STENCIL, 0, Depth, Stencil);
		break;

	case CT_Stencil:	// Clear stencil only
		FOpenGL::ClearBufferiv(GL_STENCIL, 0, (const GLint*)&Stencil);
		break;

	case CT_Depth:	// Clear depth only
		FOpenGL::ClearBufferfv(GL_DEPTH, 0, &Depth);
		break;

	default:
		break;	// impossible anyway
	}
}

void FOpenGLDynamicRHI::ClearCurrentFramebufferWithCurrentScissor(int8 ClearType, int32 NumClearColors, const bool* bClearColorArray, const FLinearColor* ClearColorArray, float Depth, uint32 Stencil)
{
	VERIFY_GL_SCOPE();
		
	// Clear color buffers
	if (ClearType & CT_Color && bClearColorArray)
	{
		for(int32 ColorIndex = 0; ColorIndex < NumClearColors; ++ColorIndex)
		{
			if (bClearColorArray[ColorIndex])
			{
				FOpenGL::ClearBufferfv( GL_COLOR, ColorIndex, (const GLfloat*)&ClearColorArray[ColorIndex] );
			}
		}
	}

	if (ClearType & CT_DepthStencil)
	{
		ClearCurrentDepthStencilWithCurrentScissor(ClearType & CT_DepthStencil, Depth, Stencil);
	}
}

void FOpenGLDynamicRHI::RHIClearMRT(const bool* bClearColorArray,int32 NumClearColors,const FLinearColor* ClearColorArray,bool bClearDepth,float Depth,bool bClearStencil,uint32 Stencil)
{
	FIntRect ExcludeRect;
	VERIFY_GL_SCOPE();

	check((GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5) || !PendingState.bFramebufferSetupInvalid);

	if (bClearColorArray)
	{
		// This is copied from DirectX11 code - apparently there's a silent assumption that there can be no valid render target set at index higher than an invalid one.
		int32 NumActiveRenderTargets = 0;
		for (int32 TargetIndex = 0; TargetIndex < MaxSimultaneousRenderTargets; TargetIndex++)
		{
			if (PendingState.RenderTargets[TargetIndex] != 0)
			{
				NumActiveRenderTargets++;
			}
			else
			{
				break;
			}
		}
		
		// Must specify enough clear colors for all active RTs
		check(NumClearColors >= NumActiveRenderTargets);
	}

	// Remember cached scissor state, and set one to cover viewport
	FIntRect PrevScissor = PendingState.Scissor;
	bool bPrevScissorEnabled = PendingState.bScissorEnabled;

	bool bScissorChanged = false;

	BindPendingFramebuffer();

	if (bPrevScissorEnabled || PendingState.Viewport.Min.X != 0 || PendingState.Viewport.Min.Y != 0 || PendingState.Viewport.Max.X != PendingState.RenderTargetWidth || PendingState.Viewport.Max.Y != PendingState.RenderTargetHeight)
	{
		RHISetScissorRect(false, 0, 0, 0, 0);
		bScissorChanged = true;
	}

	// Always update in case there are uncommitted changes to disable scissor
	UpdateScissorRectInOpenGLContext();

	int8 ClearType = CT_None;

	// Prepare color buffer masks, if applicable
	if (bClearColorArray)
	{
		ClearType |= CT_Color;

		for(int32 ColorIndex = 0; ColorIndex < NumClearColors; ++ColorIndex)
		{
			if (bClearColorArray[ColorIndex])
			{
				if (!ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskR ||
					!ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskG ||
					!ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskB ||
					!ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskA)
				{
					FOpenGL::ColorMaskIndexed(ColorIndex, GL_TRUE, GL_TRUE, GL_TRUE, GL_TRUE);
					ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskR = 1;
					ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskG = 1;
					ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskB = 1;
					ContextState.BlendState.RenderTargets[ColorIndex].ColorWriteMaskA = 1;
				}
			}
		}
	}

	// Prepare depth mask, if applicable
	if (bClearDepth && PendingState.DepthStencil)
	{
		ClearType |= CT_Depth;

		if (!ContextState.DepthStencilState.bZWriteEnable)
		{
			glDepthMask(GL_TRUE);
			ContextState.DepthStencilState.bZWriteEnable = true;
		}
	}

	// Prepare stencil mask, if applicable
	if (bClearStencil && PendingState.DepthStencil)
	{
		ClearType |= CT_Stencil;

		if (ContextState.DepthStencilState.StencilWriteMask != 0xFFFFFFFF)
		{
			glStencilMask(0xFFFFFFFF);
			ContextState.DepthStencilState.StencilWriteMask = 0xFFFFFFFF;
		}
	}

	// Just one clear
	ClearCurrentFramebufferWithCurrentScissor(ClearType, NumClearColors, bClearColorArray, ClearColorArray, Depth, Stencil);

	if (bScissorChanged)
	{
		// Change it back
		RHISetScissorRect(bPrevScissorEnabled,PrevScissor.Min.X, PrevScissor.Min.Y, PrevScissor.Max.X, PrevScissor.Max.Y);
	}
}

// Blocks the CPU until the GPU catches up and goes idle.
void FOpenGLDynamicRHI::RHIBlockUntilGPUIdle()
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda([](FRHICommandListImmediate&)
	{
		FOpenGL::Flush();
	});
	
	// @todo dev-pr use a render query to determine when the GPU is done
}

template <typename TRHIShader>
void FOpenGLDynamicRHI::ApplyStaticUniformBuffers(TRHIShader* Shader)
{
	if (Shader)
	{
		const EShaderFrequency ShaderFrequency = Shader->GetFrequency();

		auto* ProxyShader = ResourceCast(Shader);
		check(ProxyShader);

		UE::RHICore::ApplyStaticUniformBuffers(Shader, GlobalUniformBuffers,
			[this, ShaderFrequency](int32 BufferIndex, FRHIUniformBuffer* Buffer)
			{
				BindUniformBuffer(ShaderFrequency, BufferIndex, Buffer);
			});
	}
}

void FOpenGLDynamicRHI::RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState)
{
	FRHIGraphicsPipelineStateFallBack* FallbackGraphicsState = static_cast<FRHIGraphicsPipelineStateFallBack*>(GraphicsState);

	auto& PsoInit = FallbackGraphicsState->Initializer;

	if (PsoInit.bFromPSOFileCache)
	{
		checkNoEntry();
// 		// If we're from the PSO cache we're just preparing the PSO and do not need to set the state.
		return;
	}

	RHISetBoundShaderState(
		RHICreateBoundShaderState_Internal(
			PsoInit.BoundShaderState.VertexDeclarationRHI,
			PsoInit.BoundShaderState.VertexShaderRHI,
			PsoInit.BoundShaderState.PixelShaderRHI,
			PsoInit.BoundShaderState.GetGeometryShader(),
			PsoInit.bFromPSOFileCache
		).GetReference()
	);

	RHISetDepthStencilState(FallbackGraphicsState->Initializer.DepthStencilState, StencilRef);
	RHISetRasterizerState(FallbackGraphicsState->Initializer.RasterizerState);
	RHISetBlendState(FallbackGraphicsState->Initializer.BlendState, FLinearColor(1.0f, 1.0f, 1.0f));
	if (GSupportsDepthBoundsTest)
	{
		RHIEnableDepthBoundsTest(FallbackGraphicsState->Initializer.bDepthBounds);
	}

	if (bApplyAdditionalState)
	{
		ApplyStaticUniformBuffers(PsoInit.BoundShaderState.VertexShaderRHI);
		ApplyStaticUniformBuffers(PsoInit.BoundShaderState.GetGeometryShader());
		ApplyStaticUniformBuffers(PsoInit.BoundShaderState.PixelShaderRHI);
	}

	// Store the PSO's primitive (after since IRHICommandContext::RHISetGraphicsPipelineState sets the BSS)
	PrimitiveType = PsoInit.PrimitiveType;
}

void FOpenGLDynamicRHI::RHISetComputeShader(FRHIComputeShader* ComputeShaderRHI)
{
	if (OpenGLConsoleVariables::bSkipCompute)
	{
		return;
	}

	PendingState.CurrentComputeShader = ResourceCast(ComputeShaderRHI);

	ApplyStaticUniformBuffers(ComputeShaderRHI);
}

void FOpenGLDynamicRHI::SetupDispatch()
{
	FOpenGLComputeShader* ComputeShader = PendingState.CurrentComputeShader;
	check(ComputeShader);
	LinkComputeProgram(ComputeShader);

	BindPendingComputeShaderState(ComputeShader);
	CommitComputeResourceTables(ComputeShader);
	SetupTexturesForDraw(ComputeShader, FOpenGL::GetMaxComputeTextureImageUnits());
	SetupUAVsForCompute(ComputeShader);
	CommitComputeShaderConstants(ComputeShader);
}

void FOpenGLDynamicRHI::RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ)
{ 
	if (OpenGLConsoleVariables::bSkipCompute)
	{
		return;
	}

	RHI_DISPATCH_CALL_INC();

	VERIFY_GL_SCOPE();

	SetupDispatch();

	FOpenGL::MemoryBarrier(GL_ALL_BARRIER_BITS);
	FOpenGL::DispatchCompute(ThreadGroupCountX, ThreadGroupCountY, ThreadGroupCountZ);
	FOpenGL::MemoryBarrier(GL_ALL_BARRIER_BITS);
}

void FOpenGLDynamicRHI::RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBufferRHI, uint32 ArgumentOffset)
{
	RHI_DISPATCH_CALL_INC();

	VERIFY_GL_SCOPE();

	SetupDispatch();
	
	FOpenGLBuffer* ArgumentBuffer = ResourceCast(ArgumentBufferRHI);
	glBindBuffer( GL_DISPATCH_INDIRECT_BUFFER, ArgumentBuffer->Resource);
	
	FOpenGL::MemoryBarrier(GL_ALL_BARRIER_BITS);
	FOpenGL::DispatchComputeIndirect(ArgumentOffset);
	FOpenGL::MemoryBarrier(GL_ALL_BARRIER_BITS);

	glBindBuffer( GL_DISPATCH_INDIRECT_BUFFER, 0);
}

void FOpenGLDynamicRHI::RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data)
{
	UE_LOGF(LogRHI, Fatal, "OpenGL Render path does not support multiple Viewports!");
}

void FOpenGLDynamicRHI::RHIEnableDepthBoundsTest(bool bEnable)
{
	if (FOpenGL::SupportsDepthBoundsTest())
	{
		if(bEnable)
		{
			glEnable(GL_DEPTH_BOUNDS_TEST_EXT);
		}
		else
		{
			glDisable(GL_DEPTH_BOUNDS_TEST_EXT);
		}
	}
}

void FOpenGLDynamicRHI::RHISetDepthBounds(float MinDepth, float MaxDepth)
{
	if (FOpenGL::SupportsDepthBoundsTest())
	{
		FOpenGL::DepthBounds(MinDepth, MaxDepth);
	}
}

IRHICommandContext* FOpenGLDynamicRHI::RHIGetDefaultContext()
{
	return this;
}

IRHIComputeContext* FOpenGLDynamicRHI::RHIGetCommandContext(ERHIPipeline Pipeline)
{
	UE_LOGF(LogRHI, Fatal, "FOpenGLDynamicRHI::RHIGetCommandContext should never be called. OpenGL RHI does not implement parallel command list execution.");
	return nullptr;
}

FDynamicRHI::FRHIFinalizeContextsResult FOpenGLDynamicRHI::RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args)
{
	checkf(Args.UploadContext == nullptr, TEXT("OpenGL RHI does not implement upload contexts."));

	FlushProfilerStats();

#if DO_CHECK
	for (ERHIPipeline Pipeline : MakeFlagsRange(ERHIPipeline::All))
	{
	    if (IRHIComputeContext* Context = Args.Contexts[Pipeline])
	    {
		    // "Context" will always be the default graphics context, since we don't implement parallel execution.
		    check(Pipeline == ERHIPipeline::Graphics && Context == this);
	    }
    }
#endif

	// Clear some context state
	FMemory::Memset(PendingState.BoundUniformBuffers, 0, sizeof(PendingState.BoundUniformBuffers));
	FMemory::Memset(PendingState.BoundUniformBuffersDynamicOffset, 0u, sizeof(PendingState.BoundUniformBuffersDynamicOffset));

	return {};
}

void FOpenGLDynamicRHI::RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args)
{
	// Periodically poll for completed queries and fences
	FOpenGLRenderQuery::PollQueryResults();
	FOpenGLGPUFence::PollFences();
}

void FOpenGLDynamicRHI::RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes)
{
	VERIFY_GL_SCOPE();
	FOpenGLBuffer* SourceBuffer = ResourceCast(SourceBufferRHI);
	FOpenGLStagingBuffer* DestinationBuffer = ResourceCast(DestinationStagingBufferRHI);

	check(DestinationBuffer->ShadowBuffer != 0);
	glBindBuffer(GL_COPY_WRITE_BUFFER, DestinationBuffer->ShadowBuffer);
	if (DestinationBuffer->ShadowSize < InNumBytes)
	{
		if (FOpenGL::SupportsBufferStorage() && OpenGLConsoleVariables::bUsePersistentMappingStagingBuffer)
		{
			if (DestinationBuffer->Mapping != nullptr)
			{
				FOpenGL::UnmapBuffer(GL_COPY_WRITE_BUFFER);
				glDeleteBuffers(1, &DestinationBuffer->ShadowBuffer);
				glGenBuffers(1, &DestinationBuffer->ShadowBuffer);
				glBindBuffer(GL_COPY_WRITE_BUFFER, DestinationBuffer->ShadowBuffer);
			}

			FOpenGL::BufferStorage(GL_COPY_WRITE_BUFFER, InNumBytes, NULL, GL_MAP_READ_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT);

			DestinationBuffer->Mapping = FOpenGL::MapBufferRange(GL_COPY_WRITE_BUFFER, 0, InNumBytes, FOpenGL::EResourceLockMode::RLM_ReadOnlyPersistent);
		}
		else
		{
			glBufferData(GL_COPY_WRITE_BUFFER, InNumBytes, NULL, GL_STREAM_READ);
		}
		DestinationBuffer->ShadowSize = InNumBytes;
	}

	glBindBuffer(GL_COPY_READ_BUFFER, SourceBuffer->Resource);
	FOpenGL::CopyBufferSubData(GL_COPY_READ_BUFFER, GL_COPY_WRITE_BUFFER, InOffset, 0, InNumBytes);
	glBindBuffer(GL_COPY_READ_BUFFER, 0);
	glBindBuffer(GL_COPY_WRITE_BUFFER, 0);
}

void FOpenGLDynamicRHI::RHIPostExternalCommandsReset()
{
	if ((int)ContextState.Program >= 0)
	{
		FOpenGL::BindProgramPipeline(ContextState.Program);
	}

	glViewport(
		ContextState.Viewport.Min.X,
		ContextState.Viewport.Min.Y,
		ContextState.Viewport.Max.X - ContextState.Viewport.Min.X,
		ContextState.Viewport.Max.Y - ContextState.Viewport.Min.Y
	);

	FOpenGL::DepthRange(ContextState.DepthMinZ, ContextState.DepthMaxZ);

	if (ContextState.bScissorEnabled)
	{
		glEnable(GL_SCISSOR_TEST);
	}
	else
	{
		glDisable(GL_SCISSOR_TEST);
	}

	glScissor(
		ContextState.Scissor.Min.X,
		ContextState.Scissor.Min.Y,
		ContextState.Scissor.Max.X - ContextState.Scissor.Min.X,
		ContextState.Scissor.Max.Y - ContextState.Scissor.Min.Y
	);

	if ((int)ContextState.Framebuffer >= 0)
	{
		glBindFramebuffer(GL_FRAMEBUFFER, ContextState.Framebuffer);
	}
	if ((int)ContextState.ArrayBufferBound >= 0)
	{
		glBindBuffer(GL_ARRAY_BUFFER, ContextState.ArrayBufferBound);
	}
	if ((int)ContextState.ElementArrayBufferBound >= 0)
	{
		glBindBuffer(GL_ELEMENT_ARRAY_BUFFER, ContextState.ElementArrayBufferBound);
	}

	glBindBuffer(GL_PIXEL_UNPACK_BUFFER, ContextState.PixelUnpackBufferBound);
	glBindBuffer(GL_UNIFORM_BUFFER, ContextState.UniformBufferBound);

	ContextState.BlendState = {};

	glActiveTexture(GL_TEXTURE0);
	ContextState.ActiveTexture = 0;
	ContextState.RasterizerState = {};

	UpdateRasterizerStateInOpenGLContext();
	ContextState.ActiveStreamMask = 0;

	for (GLuint UniformBufferIndex = 0; UniformBufferIndex < CrossCompiler::NUM_SHADER_STAGES * OGL_MAX_UNIFORM_BUFFER_BINDINGS; UniformBufferIndex++)
	{
		ContextState.UniformBuffers[UniformBufferIndex] = FOpenGLCachedUniformBuffer_Invalid;	// that'll enforce state update on next cache test
	}

	for (uint32 Index = 0; Index < NUM_OPENGL_VERTEX_STREAMS; Index++)
	{
		if (ContextState.VertexStreams[Index].VertexBufferResource > 0)
		{
			FOpenGL::BindVertexBuffer(Index, ContextState.VertexStreams[Index].VertexBufferResource, ContextState.VertexStreams[Index].Offset, ContextState.VertexStreams[Index].Stride);
		}
		else
		{
			FOpenGL::BindVertexBuffer(Index, 0, 0, 0);
		}
	}

	for (uint32 Index = 0; Index < NUM_OPENGL_VERTEX_STREAMS; Index++)
	{
		if (ContextState.GetVertexAttrEnabled(Index))
		{
			FOpenGLCachedAttr& Attr = ContextState.VertexAttrs[Index];
			if (Attr.StreamIndex < NUM_OPENGL_VERTEX_STREAMS && ContextState.VertexStreams[Attr.StreamIndex].VertexBufferResource > 0)
			{
				if (!Attr.bShouldConvertToFloat)
				{
					FOpenGL::VertexAttribIFormat(Index, Attr.Size, Attr.Type, Attr.StreamOffset);
				}
				else
				{
					FOpenGL::VertexAttribFormat(Index, Attr.Size, Attr.Type, Attr.bNormalized, Attr.StreamOffset);
				}
				FOpenGL::VertexAttribBinding(Index, Attr.StreamIndex);
				glEnableVertexAttribArray(Index);
			}
		}
	}
}

#if PLATFORM_USES_FIXED_RHI_CLASS
#define INTERNAL_DECORATOR(Method) ((FOpenGLDynamicRHI&)CmdList.GetContext()).FOpenGLDynamicRHI::Method
#include "RHICommandListCommandExecutes.inl"
#endif

