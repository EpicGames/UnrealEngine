// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLDevice.cpp: OpenGL device RHI implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Stats/Stats.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Containers/List.h"
#include "RenderResource.h"
#include "ShaderCore.h"
#include "Shader.h"
#include "RenderUtils.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "SceneUtils.h"
#include "GlobalShader.h"

#include "HardwareInfo.h"
#include "OpenGLProgramBinaryFileCache.h"

#if PLATFORM_WINDOWS
#include "HAL/Platform.h"
#endif

#if PLATFORM_ANDROID
#include "Android/AndroidJniBitmapRendererLegacy.h"
extern bool AndroidThunkCpp_IsXRImmersiveHMDApplication();
#endif

#ifndef GL_STEREO
#define GL_STEREO			0x0C33
#endif

extern GLint GMaxOpenGLColorSamples;
extern GLint GMaxOpenGLDepthSamples;
extern GLint GMaxOpenGLIntegerSamples;
extern GLint GMaxOpenGLTextureFilterAnisotropic;
extern GLint GMaxOpenGLDrawBuffers;

/** OpenGL texture format table. */
FOpenGLTextureFormat GOpenGLTextureFormats[PF_MAX];

/** Device is necessary for vertex buffers, so they can reach the global device on destruction, and tell it to reset vertex array caches */
static FOpenGLDynamicRHI* PrivateOpenGLDevicePtr = NULL;

/** true if we're not using UBOs. */
bool GUseEmulatedUniformBuffers;

static TAutoConsoleVariable<int32> CVarAllowRGLHIThread(
	TEXT("r.OpenGL.AllowRHIThread"),
	1,
	TEXT("Toggle OpenGL RHI thread support.\n")
	TEXT("0: GL scene rendering operations are performed on the render thread.\n")
	TEXT("1: GL scene rendering operations are queued onto the RHI thread gaining some parallelism with the render thread. (default, mobile feature levels only)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarGLExtraDeletionLatency(
	TEXT("r.OpenGL.ExtraDeletionLatency"),
	1,
	TEXT("Toggle the engine's deferred deletion queue for RHI resources. (default:1)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarGLDepth24Bit(
	TEXT("r.OpenGL.Depth24Bit"),
	1,
	TEXT("0: Use 32-bit float depth buffer \n1: Use 24-bit fixed point depth buffer (default)\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

// If precaching is active we should not need the file cache.
// however, precaching and filecache are compatible with each other, there maybe some scenarios in which both could be used.
static TAutoConsoleVariable<bool> CVarEnablePSOFileCacheWhenPrecachingActive(
	TEXT("r.OpenGL.EnablePSOFileCacheWhenPrecachingActive"),
	false,
	TEXT("false: If precaching is active (r.PSOPrecaching=1) then disable the PSO filecache. (default)\n")
	TEXT("true: GL RHI Allows both PSO file cache and precaching."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarAllowPSOPrecaching(
	TEXT("r.OpenGL.AllowPSOPrecaching"),
	true,
	TEXT("true: if r.PSOPrecaching=1 GL RHI will use precaching. (default)\n")
	TEXT("false: GL RHI will disable precaching (even if r.PSOPrecaching=1). "),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

extern void BeginFrame_UniformBufferPoolCleanup();
extern void BeginFrame_VertexBufferCleanup();

void FOpenGLDynamicRHI::RHIEndFrame(const FRHIEndFrameArgs& Args)
{
	Profiler.EndFrame(Args);
	KickHint.Reset();

#if PLATFORM_ANDROID //adding #if since not sure if this is required for any other platform.
	PendingState.DepthStencil = 0;
#endif

	BeginFrame_UniformBufferPoolCleanup();
	BeginFrame_VertexBufferCleanup();
}

#if PLATFORM_ANDROID

void JNICALL UE::Jni::FBitmapRendererLegacy::nativeClearCachedAttributeState(JNIEnv* jenv, jobject thiz, jint PositionAttrib, jint TexCoordsAttrib)
{
	FOpenGLDynamicRHI::Get().ClearCachedAttributeState(PositionAttrib, TexCoordsAttrib);
}

#endif

void FOpenGLDynamicRHI::ClearCachedAttributeState(int32 PositionAttrib, int32 TexCoordsAttrib)
{
	// update vertex attributes state
	ContextState.SetVertexAttrEnabled(PositionAttrib, false);
	ContextState.VertexAttrs[PositionAttrib].Size = -1; // will force attribute update

	ContextState.SetVertexAttrEnabled(TexCoordsAttrib, false);
	ContextState.VertexAttrs[TexCoordsAttrib].Size = -1; // will force attribute update

	// make sure the texture is set again
	ContextState.ActiveTexture = 0;
	ContextState.Textures[0].Texture = nullptr;
	ContextState.Textures[0].Target = 0;

	// restore previous program
	FOpenGL::BindProgramPipeline(ContextState.Program);
}

bool GDisableOpenGLDebugOutput = false;

#if defined(GL_ARB_debug_output) || defined(GL_KHR_debug)

/**
 * Map GL_DEBUG_SOURCE_*_ARB to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugSourceStringARB(GLenum Source)
{
	static const TCHAR* SourceStrings[] =
	{
		TEXT("API"),
		TEXT("System"),
		TEXT("ShaderCompiler"),
		TEXT("ThirdParty"),
		TEXT("Application"),
		TEXT("Other")
	};

	if (Source >= GL_DEBUG_SOURCE_API_ARB && Source <= GL_DEBUG_SOURCE_OTHER_ARB)
	{
		return SourceStrings[Source - GL_DEBUG_SOURCE_API_ARB];
	}
	return TEXT("Unknown");
}

/**
 * Map GL_DEBUG_TYPE_*_ARB to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugTypeStringARB(GLenum Type)
{
	static const TCHAR* TypeStrings[] =
	{
		TEXT("Error"),
		TEXT("Deprecated"),
		TEXT("UndefinedBehavior"),
		TEXT("Portability"),
		TEXT("Performance"),
		TEXT("Other")
	};

	if (Type >= GL_DEBUG_TYPE_ERROR_ARB && Type <= GL_DEBUG_TYPE_OTHER_ARB)
	{
		return TypeStrings[Type - GL_DEBUG_TYPE_ERROR_ARB];
	}
#ifdef GL_KHR_debug
	{
		static const TCHAR* DebugTypeStrings[] =
		{
			TEXT("Marker"),
			TEXT("PushGroup"),
			TEXT("PopGroup"),
		};

		if (Type >= GL_DEBUG_TYPE_MARKER && Type <= GL_DEBUG_TYPE_POP_GROUP)
		{
			return DebugTypeStrings[Type - GL_DEBUG_TYPE_MARKER];
		}
	}
#endif
	return TEXT("Unknown");
}

/**
 * Map GL_DEBUG_SEVERITY_*_ARB to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugSeverityStringARB(GLenum Severity)
{
	static const TCHAR* SeverityStrings[] =
	{
		TEXT("High"),
		TEXT("Medium"),
		TEXT("Low")
	};

	if (Severity >= GL_DEBUG_SEVERITY_HIGH_ARB && Severity <= GL_DEBUG_SEVERITY_LOW_ARB)
	{
		return SeverityStrings[Severity - GL_DEBUG_SEVERITY_HIGH_ARB];
	}
#ifdef GL_KHR_debug
	if(Severity == GL_DEBUG_SEVERITY_NOTIFICATION)
		return TEXT("Notification");
#endif
	return TEXT("Unknown");
}

/**
 * OpenGL debug message callback. Conforms to GLDEBUGPROCARB.
 */
#if PLATFORM_ANDROID
	#ifndef GL_APIENTRY
	#define GL_APIENTRY APIENTRY
	#endif
	#ifndef GL_DEBUG_SEVERITY_MEDIUM_ARB
	#define GL_DEBUG_SEVERITY_MEDIUM_ARB GL_DEBUG_SEVERITY_LOW_ARB
	#endif
static void GL_APIENTRY OpenGLDebugMessageCallbackARB(
#else
static void APIENTRY OpenGLDebugMessageCallbackARB(
#endif
	GLenum Source,
	GLenum Type,
	GLuint Id,
	GLenum Severity,
	GLsizei Length,
	const GLchar* Message,
	GLvoid* UserParam)
{
	if (GDisableOpenGLDebugOutput)
		return;
#if !NO_LOGGING
	const TCHAR* SourceStr = GetOpenGLDebugSourceStringARB(Source);
	const TCHAR* TypeStr = GetOpenGLDebugTypeStringARB(Type);
	const TCHAR* SeverityStr = GetOpenGLDebugSeverityStringARB(Severity);

	ELogVerbosity::Type Verbosity = ELogVerbosity::Warning;
	if (Type == GL_DEBUG_TYPE_ERROR_ARB && Severity == GL_DEBUG_SEVERITY_HIGH_ARB)
	{
		Verbosity = ELogVerbosity::Error;
	}

	if ((Verbosity & ELogVerbosity::VerbosityMask) <= FLogCategoryLogRHI::CompileTimeVerbosity)
	{
		if (!LogRHI.IsSuppressed(Verbosity))
		{
			FMsg::Logf(__FILE__, __LINE__, LogRHI.GetCategoryName(), Verbosity,
				TEXT("GL_DBG: [%s][%s][%s][%u] %s"),
				SourceStr,
				TypeStr,
				SeverityStr,
				Id,
				ANSI_TO_TCHAR(Message)
				);
		}

		// this is a debugging code to catch VIDEO->HOST copying
		if (Id == 131186)
		{
			int A = 5;
		}
	}
#endif
}

#endif // GL_ARB_debug_output

#ifdef GL_AMD_debug_output

/**
 * Map GL_DEBUG_CATEGORY_*_AMD to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugCategoryStringAMD(GLenum Category)
{
	static const TCHAR* CategoryStrings[] =
	{
		TEXT("API"),
		TEXT("System"),
		TEXT("Deprecation"),
		TEXT("UndefinedBehavior"),
		TEXT("Performance"),
		TEXT("ShaderCompiler"),
		TEXT("Application"),
		TEXT("Other")
	};

	if (Category >= GL_DEBUG_CATEGORY_API_ERROR_AMD && Category <= GL_DEBUG_CATEGORY_OTHER_AMD)
	{
		return CategoryStrings[Category - GL_DEBUG_CATEGORY_API_ERROR_AMD];
	}
	return TEXT("Unknown");
}

/**
 * Map GL_DEBUG_SEVERITY_*_AMD to a human-readable string.
 */
static const TCHAR* GetOpenGLDebugSeverityStringAMD(GLenum Severity)
{
	static const TCHAR* SeverityStrings[] =
	{
		TEXT("High"),
		TEXT("Medium"),
		TEXT("Low")
	};

	if (Severity >= GL_DEBUG_SEVERITY_HIGH_AMD && Severity <= GL_DEBUG_SEVERITY_LOW_AMD)
	{
		return SeverityStrings[Severity - GL_DEBUG_SEVERITY_HIGH_AMD];
	}
	return TEXT("Unknown");
}

/**
 * OpenGL debug message callback. Conforms to GLDEBUGPROCAMD.
 */
static void APIENTRY OpenGLDebugMessageCallbackAMD(
	GLuint Id,
	GLenum Category,
	GLenum Severity,
	GLsizei Length,
	const GLchar* Message,
	GLvoid* UserParam)
{
#if !NO_LOGGING
	const TCHAR* CategoryStr = GetOpenGLDebugCategoryStringAMD(Category);
	const TCHAR* SeverityStr = GetOpenGLDebugSeverityStringAMD(Severity);

	ELogVerbosity::Type Verbosity = ELogVerbosity::Warning;
	if (Severity == GL_DEBUG_SEVERITY_HIGH_AMD)
	{
		Verbosity = ELogVerbosity::Fatal;
	}

	if ((Verbosity & ELogVerbosity::VerbosityMask) <= FLogCategoryLogRHI::CompileTimeVerbosity)
	{
		if (!LogRHI.IsSuppressed(Verbosity))
		{
			FMsg::Logf(__FILE__, __LINE__, LogRHI.GetCategoryName(), Verbosity,
				TEXT("[%s][%s][%u] %s"),
				CategoryStr,
				SeverityStr,
				Id,
				ANSI_TO_TCHAR(Message)
				);
		}
	}
#endif
}

#endif // GL_AMD_debug_output

#if PLATFORM_WINDOWS
PFNWGLSWAPINTERVALEXTPROC wglSwapIntervalEXT_ProcAddress = NULL;
#endif

// Information from OpenGLES specification 3.2 Table 8.10
// Qualcomm Adreno OpenGLES Developer Guide 4.1.4 section and Table 9-1
enum class EOpenGLFormatCapabilities : uint32
{
	None = 0,
	Texture = 1ull << 0,
	Render = 1ull << 1,
	Filterable = 1ull << 2,
	Image = 1ull << 3,
	DepthStencil = 1ull << 4,
};
ENUM_CLASS_FLAGS(EOpenGLFormatCapabilities);

static EOpenGLFormatCapabilities GetOpenGLFormatCapabilities(const FOpenGLTextureFormat& GLFormat)
{
	EOpenGLFormatCapabilities Capabilities = EOpenGLFormatCapabilities::None;
	
	switch (GLFormat.InternalFormat[0])
	{
	default: checkNoEntry();
	case GL_NONE:
		break;

	case GL_R8:
	case GL_RG8:
	case GL_RGB8:
	case GL_RGBA4:
	case GL_RGB10_A2:
	case GL_SRGB8_ALPHA8:
	case GL_R16F:
	case GL_RG16F:
	case GL_RGBA16:
	case GL_RGB565:
	case GL_RGB5_A1:
	case GL_R11F_G11F_B10F:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Render | EOpenGLFormatCapabilities::Filterable;
		break;

	case GL_RGBA8:
	case GL_RGBA16F:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Render | EOpenGLFormatCapabilities::Filterable | EOpenGLFormatCapabilities::Image;
		break;

	case GL_R16:
	case GL_RGB10_A2UI:
	case GL_RG32F:
	case GL_R8I:
	case GL_R8UI:
	case GL_R16I:
	case GL_R16UI:
	case GL_RG8I:
	case GL_RG8UI:
	case GL_RG16I:
	case GL_RG16UI:
	case GL_RG32I:
	case GL_RG32UI:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Render;
		break;

	case GL_R32I:
	case GL_R32F:
	case GL_R32UI:
	case GL_RGBA8I:
	case GL_RGBA8UI:
	case GL_RGBA16I:
	case GL_RGBA16UI:
	case GL_RGBA32I:
	case GL_RGBA32F:
	case GL_RGBA32UI:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Render | EOpenGLFormatCapabilities::Image;
		break;

	case GL_RGB32F:
	case GL_RGB8I:
	case GL_RGB8UI:
	case GL_RGB16I:
	case GL_RGB16UI:
	case GL_RGB32I:
	case GL_RGB32UI:
		Capabilities |= EOpenGLFormatCapabilities::Texture;
		break;

	case GL_DEPTH24_STENCIL8:
	case GL_DEPTH32F_STENCIL8:
	case GL_DEPTH_COMPONENT16:
	case GL_DEPTH_COMPONENT24:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Render | EOpenGLFormatCapabilities::Filterable | EOpenGLFormatCapabilities::DepthStencil;
		break;

#if PLATFORM_ANDROID
	case GL_COMPRESSED_RGB8_ETC2:
	case GL_COMPRESSED_RGBA8_ETC2_EAC:
	case GL_COMPRESSED_R11_EAC:
	case GL_COMPRESSED_RG11_EAC:
#endif

#if PLATFORM_DESKTOP
	case GL_COMPRESSED_RG_RGTC2:
	case GL_COMPRESSED_RED_RGTC1:
	case GL_RG16:
	case GL_RG16_SNORM:
	case GL_COMPRESSED_RGBA_BPTC_UNORM:
	case GL_COMPRESSED_RGB_BPTC_UNSIGNED_FLOAT:
#endif

	case GL_COMPRESSED_RGBA_S3TC_DXT1_EXT:
	case GL_COMPRESSED_RGBA_S3TC_DXT3_EXT:
	case GL_COMPRESSED_RGBA_S3TC_DXT5_EXT:

	case GL_COMPRESSED_RGBA_ASTC_4x4_KHR:
	case GL_COMPRESSED_RGBA_ASTC_6x6_KHR:
	case GL_COMPRESSED_RGBA_ASTC_8x8_KHR:
	case GL_COMPRESSED_RGBA_ASTC_10x10_KHR:
	case GL_COMPRESSED_RGBA_ASTC_12x12_KHR:

	case GL_R8_SNORM:
	case GL_RG8_SNORM:
	case GL_RGB8_SNORM:
	case GL_RGB9_E5:
	case GL_RGB16F:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Filterable;
		break;

	case GL_RGBA8_SNORM:
		Capabilities |= EOpenGLFormatCapabilities::Texture | EOpenGLFormatCapabilities::Filterable | EOpenGLFormatCapabilities::Image;
		break;
	}

	return Capabilities;
}

static inline void SetupTextureFormat( EPixelFormat Format, const FOpenGLTextureFormat& GLFormat)
{
	GOpenGLTextureFormats[Format] = GLFormat;
	GPixelFormats[Format].Supported = (GLFormat.Format != GL_NONE && (GLFormat.InternalFormat[0] != GL_NONE || GLFormat.InternalFormat[1] != GL_NONE));
	GPixelFormats[Format].PlatformFormat = GLFormat.InternalFormat[0];
	
	if (GPixelFormats[Format].Supported)
	{
		EPixelFormatCapabilities& Capabilities = GPixelFormats[Format].Capabilities;
		EOpenGLFormatCapabilities OpenGLCapabilities = GetOpenGLFormatCapabilities(GLFormat);

		if (EnumHasAllFlags(OpenGLCapabilities, EOpenGLFormatCapabilities::Texture))
		{
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::Texture1D);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::Texture2D);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::Texture3D);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureCube);

			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureMipmaps);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureLoad);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureSample);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureGather);
		}

		if (EnumHasAllFlags(OpenGLCapabilities, EOpenGLFormatCapabilities::Filterable))
		{
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureFilterable);
		}

		if (EnumHasAllFlags(OpenGLCapabilities, EOpenGLFormatCapabilities::Render))
		{
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::RenderTarget);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureBlendable);
		}

		if (EnumHasAllFlags(OpenGLCapabilities, EOpenGLFormatCapabilities::DepthStencil))
		{
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::DepthStencil);
		}

		if (EnumHasAllFlags(OpenGLCapabilities, EOpenGLFormatCapabilities::Image))
		{
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureAtomics);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TextureStore);

			EnumAddFlags(Capabilities, EPixelFormatCapabilities::Buffer);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::VertexBuffer);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::IndexBuffer);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::BufferLoad);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::BufferStore);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::BufferAtomics);

			EnumAddFlags(Capabilities, EPixelFormatCapabilities::UAV);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TypedUAVLoad);
			EnumAddFlags(Capabilities, EPixelFormatCapabilities::TypedUAVStore);
		}
	}
}


void InitDebugContext()
{
	// Set the debug output callback if the driver supports it.
	VERIFY_GL(__FUNCTION__);
	bool bDebugOutputInitialized = false;
	if (!IsOGLDebugOutputEnabled())
	{
		return;
	}

#if ENABLE_DEBUG_OUTPUT
	#if defined(GL_ARB_debug_output)
		if (glDebugMessageCallbackARB)
		{
			// Synchronous output can slow things down, but we'll get better callstack if breaking in or crashing in the callback. This is debug only after all.
			glEnable(GL_DEBUG_OUTPUT_SYNCHRONOUS);
			glDebugMessageCallbackARB(GLDEBUGPROCARB(OpenGLDebugMessageCallbackARB), /*UserParam=*/ NULL);
			bDebugOutputInitialized = (glGetError() == GL_NO_ERROR);
		}
	#elif defined(GL_KHR_debug)
		// OpenGLES names the debug functions differently, but they behave the same
		if (glDebugMessageCallbackKHR)
		{
			glDebugMessageCallbackKHR(GLDEBUGPROCKHR(OpenGLDebugMessageCallbackARB), /*UserParam=*/ NULL);
			bDebugOutputInitialized = (glGetError() == GL_NO_ERROR);
		}
	#endif // GL_ARB_debug_output / GL_KHR_debug
	#if defined(GL_AMD_debug_output)
		if (glDebugMessageCallbackAMD && !bDebugOutputInitialized)
		{
			glDebugMessageCallbackAMD(OpenGLDebugMessageCallbackAMD, /*UserParam=*/ NULL);
			bDebugOutputInitialized = (glGetError() == GL_NO_ERROR);
		}
	#endif // GL_AMD_debug_output
 #endif // ENABLE_DEBUG_OUTPUT

	if (!bDebugOutputInitialized)
	{
		UE_LOGF(LogRHI,Warning,"OpenGL debug output extension not supported!");
	}
	else
	{
		UE_LOGF(LogRHI, Log, "OpenGL debug output extension enabled!");
	}

	// this is to suppress feeding back of the debug markers and groups to the log, since those originate in the app anyways...
#if ENABLE_OPENGL_DEBUG_GROUPS && defined(GL_ARB_debug_output) && GL_ARB_debug_output && GL_KHR_debug
	if(glDebugMessageControlARB && bDebugOutputInitialized)
	{
		glDebugMessageControlARB(GL_DEBUG_SOURCE_APPLICATION_ARB, GL_DEBUG_TYPE_MARKER, GL_DONT_CARE, 0, NULL, GL_FALSE);
		glDebugMessageControlARB(GL_DEBUG_SOURCE_APPLICATION_ARB, GL_DEBUG_TYPE_PUSH_GROUP, GL_DONT_CARE, 0, NULL, GL_FALSE);
		glDebugMessageControlARB(GL_DEBUG_SOURCE_APPLICATION_ARB, GL_DEBUG_TYPE_POP_GROUP, GL_DONT_CARE, 0, NULL, GL_FALSE);
#ifdef GL_KHR_debug
		glDebugMessageControlARB(GL_DEBUG_SOURCE_API_ARB, GL_DEBUG_TYPE_OTHER_ARB, GL_DEBUG_SEVERITY_NOTIFICATION, 0, NULL, GL_FALSE);
#endif
		UE_LOGF(LogRHI,Verbose,"disabling reporting back of debug groups and markers to the OpenGL debug output callback");
	}
#elif ENABLE_OPENGL_DEBUG_GROUPS && !defined(GL_ARB_debug_output) && defined(GL_KHR_debug) && GL_KHR_debug
	if(glDebugMessageControlKHR)
	{
		glDebugMessageControlKHR(GL_DEBUG_SOURCE_APPLICATION_KHR, GL_DEBUG_TYPE_MARKER_KHR, GL_DONT_CARE, 0, NULL, GL_FALSE);
		glDebugMessageControlKHR(GL_DEBUG_SOURCE_APPLICATION_KHR, GL_DEBUG_TYPE_PUSH_GROUP_KHR, GL_DONT_CARE, 0, NULL, GL_FALSE);
		glDebugMessageControlKHR(GL_DEBUG_SOURCE_APPLICATION_KHR, GL_DEBUG_TYPE_POP_GROUP_KHR, GL_DONT_CARE, 0, NULL, GL_FALSE);
		glDebugMessageControlKHR(GL_DEBUG_SOURCE_API_KHR, GL_DEBUG_TYPE_OTHER_KHR, GL_DEBUG_SEVERITY_NOTIFICATION, 0, NULL, GL_FALSE);

		GLenum Severity;
		const bool bAllowPerformanceMessages = GetOGLDebugOutputLevel() >= 5;
		switch (GetOGLDebugOutputLevel())
		{
			case 5: Severity = GL_DONT_CARE; break;
			case 4: Severity = GL_DEBUG_SEVERITY_NOTIFICATION; break;
			case 3: Severity = GL_DEBUG_SEVERITY_LOW_ARB; break;
			case 2: Severity = GL_DEBUG_SEVERITY_MEDIUM_ARB; break;
			case 1:
				[[fallthrough]];
			default: Severity = GL_DEBUG_SEVERITY_HIGH_ARB; break;
		}
		glDebugMessageControlKHR(GL_DONT_CARE, GL_DONT_CARE, Severity, 0, NULL, GL_TRUE);
		if( !bAllowPerformanceMessages)
		{
			glDebugMessageControlKHR(GL_DONT_CARE, GL_DEBUG_TYPE_PERFORMANCE_KHR, GL_DONT_CARE, 0, NULL, GL_FALSE);
		}
		
		UE_LOGF(LogRHI,Verbose,"disabling reporting back of debug groups and markers to the OpenGL debug output callback");
	}
#endif
}

TAutoConsoleVariable<FString> CVarOpenGLStripExtensions(
	TEXT("r.OpenGL.StripExtensions"),
	TEXT(""),
	TEXT("List of comma separated OpenGL extensions to strip from a driver reported extensions string"),
	ECVF_ReadOnly);

TAutoConsoleVariable<FString> CVarOpenGLAddExtensions(
	TEXT("r.OpenGL.AddExtensions"),
	TEXT(""),
	TEXT("List of comma separated OpenGL extensions to add to a driver reported extensions string"),
	ECVF_ReadOnly);

void ApplyExtensionsOverrides(FString& ExtensionsString)
{
	// Strip extensions
	{
		TArray<FString> ExtList;
		FString ExtString = CVarOpenGLStripExtensions.GetValueOnAnyThread();
		ExtString.ParseIntoArray(ExtList, TEXT(","), /*InCullEmpty=*/true);

		for (FString& ExtName : ExtList)
		{
			ExtName.TrimStartAndEndInline();
			if (ExtensionsString.ReplaceInline(*ExtName, TEXT("")) > 0)
			{
				UE_LOGF(LogRHI, Log, "Stripped extension: %ls", *ExtName);
			}
		}
	}

	// Add extensions
	{
		TArray<FString> ExtList;
		FString ExtString = CVarOpenGLAddExtensions.GetValueOnAnyThread();
		ExtString.ParseIntoArray(ExtList, TEXT(","), /*InCullEmpty=*/true);

		for (FString& ExtName : ExtList)
		{
			ExtName.TrimStartAndEndInline();
			if (!ExtensionsString.Contains(ExtName))
			{
				ExtensionsString.Append(TEXT(" ")); // extensions delimiter
				ExtensionsString.Append(ExtName);
				UE_LOGF(LogRHI, Log, "Added extension: %ls", *ExtName);
			}
		}
	}
}


/**
 * Initialize RHI capabilities for the current OpenGL context.
 */
static void InitRHICapabilitiesForGL()
{
	VERIFY_GL_SCOPE();

	GTexturePoolSize = 0;
	GPoolSizeVRAMPercentage = 0;
#if PLATFORM_WINDOWS || PLATFORM_LINUX
	GConfig->GetInt( TEXT( "TextureStreaming" ), TEXT( "PoolSizeVRAMPercentage" ), GPoolSizeVRAMPercentage, GEngineIni );
#endif

	// GL vendor and version information.
#if !defined(__GNUC__) && !defined(__clang__) && !defined(__INTEL_COMPILER) && !(defined(_MSC_VER) && _MSC_VER >= 1900)
	#define LOG_GL_STRING(StringEnum) UE_LOG(LogRHI, Log, TEXT("  ") ## TEXT(#StringEnum) ## TEXT(": %s"), ANSI_TO_TCHAR((const ANSICHAR*)glGetString(StringEnum)))
#else
	#define LOG_GL_STRING(StringEnum) UE_LOG(LogRHI, Log, TEXT("  " #StringEnum ": %s"), ANSI_TO_TCHAR((const ANSICHAR*)glGetString(StringEnum)))
#endif
	UE_LOGF(LogRHI, Log, "Initializing OpenGL RHI");
	LOG_GL_STRING(GL_VENDOR);
	LOG_GL_STRING(GL_RENDERER);
	LOG_GL_STRING(GL_VERSION);
	LOG_GL_STRING(GL_SHADING_LANGUAGE_VERSION);
	#undef LOG_GL_STRING

	GRHIAdapterName = FOpenGL::GetAdapterName();
	GRHIAdapterInternalDriverVersion = ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VERSION));

	// Shader platform & RHI feature level
	GMaxRHIFeatureLevel = ERHIFeatureLevel::ES3_1;
	GMaxRHIShaderPlatform = FOpenGL::GetShaderPlatform();

	GRHIMaximumInFlightQueries = 4000;

	// Log all supported extensions.
#if PLATFORM_WINDOWS
	bool bWindowsSwapControlExtensionPresent = false;
#endif
	{
		extern void GetExtensionsString( FString& ExtensionsString);
		FString ExtensionsString;

		GetExtensionsString(ExtensionsString);

#if PLATFORM_WINDOWS
		if (ExtensionsString.Contains(TEXT("WGL_EXT_swap_control")))
		{
			bWindowsSwapControlExtensionPresent = true;
		}
#endif

		// Log supported GL extensions
		UE_LOGF(LogRHI, Log, "OpenGL Extensions:");
		TArray<FString> GLExtensionArray;
		ExtensionsString.ParseIntoArray(GLExtensionArray, TEXT(" "), true);
		for (int ExtIndex = 0; ExtIndex < GLExtensionArray.Num(); ExtIndex++)
		{
			UE_LOGF(LogRHI, Log, "  %ls", *GLExtensionArray[ExtIndex]);
		}

		ApplyExtensionsOverrides(ExtensionsString);

		FOpenGL::ProcessExtensions(ExtensionsString);
	}

#if PLATFORM_WINDOWS
	PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	if (!bWindowsSwapControlExtensionPresent)
	{
		// Disable warning C4191: 'type cast' : unsafe conversion from 'PROC' to 'XXX' while getting GL entry points.
		PFNWGLGETEXTENSIONSSTRINGEXTPROC wglGetExtensionsStringEXT_ProcAddress =
			(PFNWGLGETEXTENSIONSSTRINGEXTPROC) wglGetProcAddress("wglGetExtensionsStringEXT");

		if (strstr(wglGetExtensionsStringEXT_ProcAddress(), "WGL_EXT_swap_control") != NULL)
		{
			bWindowsSwapControlExtensionPresent = true;
		}
	}

	if (bWindowsSwapControlExtensionPresent)
	{
		wglSwapIntervalEXT_ProcAddress = (PFNWGLSWAPINTERVALEXTPROC) wglGetProcAddress("wglSwapIntervalEXT");
	}
	PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
#endif

	// Set debug flag if context was setup with debugging
	FOpenGL::InitDebugContext();

	// Log and get various limits.
#if !defined(__GNUC__) && !defined(__clang__) && !defined(__INTEL_COMPILER) && !(defined(_MSC_VER) && _MSC_VER >= 1900)
#define LOG_AND_GET_GL_INT_TEMP(IntEnum,Default) GLint Value_##IntEnum = Default; if (IntEnum) {glGetIntegerv(IntEnum, &Value_##IntEnum); glGetError();} else {Value_##IntEnum = Default;} UE_LOG(LogRHI, Log, TEXT("  ") ## TEXT(#IntEnum) ## TEXT(": %d"), Value_##IntEnum)
#else
#define LOG_AND_GET_GL_INT_TEMP(IntEnum,Default) GLint Value_##IntEnum = Default; if (IntEnum) {glGetIntegerv(IntEnum, &Value_##IntEnum); glGetError();} else {Value_##IntEnum = Default;} UE_LOG(LogRHI, Log, TEXT("  " #IntEnum ": %d"), Value_##IntEnum)
#endif
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_TEXTURE_SIZE, 0);
#if defined(GL_MAX_TEXTURE_BUFFER_SIZE)
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_TEXTURE_BUFFER_SIZE, 0);
#endif
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_CUBE_MAP_TEXTURE_SIZE, 0);
#if defined(GL_MAX_ARRAY_TEXTURE_LAYERS) && GL_MAX_ARRAY_TEXTURE_LAYERS
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_ARRAY_TEXTURE_LAYERS, 0);
#endif
#if GL_MAX_3D_TEXTURE_SIZE
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_3D_TEXTURE_SIZE, 0);
#endif
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_RENDERBUFFER_SIZE, 0);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_TEXTURE_IMAGE_UNITS, 0);

	LOG_AND_GET_GL_INT_TEMP(GL_MAX_DRAW_BUFFERS, 1);
	GMaxOpenGLDrawBuffers = FMath::Min(Value_GL_MAX_DRAW_BUFFERS, (GLint)MaxSimultaneousRenderTargets);

	LOG_AND_GET_GL_INT_TEMP(GL_MAX_COLOR_ATTACHMENTS, 1);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_SAMPLES, 1);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_COLOR_TEXTURE_SAMPLES, 1);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_DEPTH_TEXTURE_SAMPLES, 1);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_INTEGER_SAMPLES, 1);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_COMBINED_TEXTURE_IMAGE_UNITS, 0);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_VERTEX_ATTRIBS, 0);

	LOG_AND_GET_GL_INT_TEMP(GL_MAX_COMPUTE_WORK_GROUP_INVOCATIONS, 0);
	LOG_AND_GET_GL_INT_TEMP(GL_MAX_COMPUTE_SHARED_MEMORY_SIZE, 0);
	

	if (FParse::Param(FCommandLine::Get(), TEXT("quad_buffer_stereo")))
	{
		GLboolean Result = GL_FALSE;
		glGetBooleanv(GL_STEREO, &Result);
		// Skip any errors if any were generated
		glGetError();
		GSupportsQuadBufferStereo = (Result == GL_TRUE);
	}

	if (FOpenGL::SupportsTextureFilterAnisotropic())
	{
		LOG_AND_GET_GL_INT_TEMP(GL_MAX_TEXTURE_MAX_ANISOTROPY_EXT, 0);
		GMaxOpenGLTextureFilterAnisotropic = Value_GL_MAX_TEXTURE_MAX_ANISOTROPY_EXT;
	}

	if (FOpenGL::SupportsPixelLocalStorage())
	{
		LOG_AND_GET_GL_INT_TEMP(GL_MAX_SHADER_PIXEL_LOCAL_STORAGE_SIZE_EXT, 0);
	}
#undef LOG_AND_GET_GL_INT_TEMP

	GMaxOpenGLColorSamples = Value_GL_MAX_COLOR_TEXTURE_SAMPLES;
	GMaxOpenGLDepthSamples = Value_GL_MAX_DEPTH_TEXTURE_SAMPLES;
	GMaxOpenGLIntegerSamples = Value_GL_MAX_INTEGER_SAMPLES;

	// Verify some assumptions.
	// Android seems like reports one color attachment even when it supports MRT
#if !PLATFORM_ANDROID
	check(Value_GL_MAX_COLOR_ATTACHMENTS >= MaxSimultaneousRenderTargets);
#endif

	// We don't check for compressed formats right now because vendors have not
	// done a great job reporting what is actually supported:
	//   OSX/NVIDIA doesn't claim to support SRGB DXT formats even though they work correctly.
	//   Windows/AMD sometimes claim to support no compressed formats even though they all work correctly.
#if 0
	// Check compressed texture formats.
	bool bSupportsCompressedTextures = true;
	{
		FString CompressedFormatsString = "  GL_COMPRESSED_TEXTURE_FORMATS:";
		TArray<GLint> CompressedFormats;
		GLint NumCompressedFormats = 0;
		glGetIntegerv(GL_NUM_COMPRESSED_TEXTURE_FORMATS, &NumCompressedFormats);
		CompressedFormats.Empty(NumCompressedFormats);
		CompressedFormats.AddZeroed(NumCompressedFormats);
		glGetIntegerv(GL_COMPRESSED_TEXTURE_FORMATS, CompressedFormats.GetTypedData());

		#define CHECK_COMPRESSED_FORMAT(GLName) if (CompressedFormats.Contains(GLName)) { CompressedFormatsString += TEXT(" ") TEXT(#GLName); } else { bSupportsCompressedTextures = false; }
		CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_RGB_S3TC_DXT1_EXT);
		CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_SRGB_S3TC_DXT1_EXT);
		CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_RGBA_S3TC_DXT3_EXT);
		CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT3_EXT);
		CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_RGBA_S3TC_DXT5_EXT);
		CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT5_EXT);
		//CHECK_COMPRESSED_FORMAT(GL_COMPRESSED_RG_RGTC2);
		#undef CHECK_COMPRESSED_FORMAT

		// ATI does not report that it supports RGTC2, but the 3.2 core profile mandates it.
		// For now assume it is supported and bring it up with ATI if it really isn't(?!)
		CompressedFormatsString += TEXT(" GL_COMPRESSED_RG_RGTC2");

		UE_LOG(LogRHI, Log, *CompressedFormatsString);
	}
	check(bSupportsCompressedTextures);
#endif

	// Set capabilities.
	const GLint MajorVersion = FOpenGL::GetMajorVersion();
	const GLint MinorVersion = FOpenGL::GetMinorVersion();
 
	// Enable the OGL rhi thread if explicitly requested.
	GRHISupportsRHIThread = (GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1 && CVarAllowRGLHIThread.GetValueOnAnyThread());

	GRHISupportsMultithreadedResources = GRHISupportsRHIThread;

	// OpenGL ES does not support glTextureView
	GRHISupportsTextureViews = false;

	// By default use emulated UBs on mobile
	GUseEmulatedUniformBuffers = IsUsingEmulatedUniformBuffers(GMaxRHIShaderPlatform);

	FString FeatureLevelName;
	GetFeatureLevelName(GMaxRHIFeatureLevel, FeatureLevelName);
	FString ShaderPlatformName = LegacyShaderPlatformToShaderFormat(GMaxRHIShaderPlatform).ToString();

	UE_LOGF(LogRHI, Log, "OpenGL MajorVersion = %d, MinorVersion = %d, ShaderPlatform = %ls, FeatureLevel = %ls", MajorVersion, MinorVersion, *ShaderPlatformName, *FeatureLevelName);
#if PLATFORM_ANDROID
	UE_LOGF(LogRHI, Log, "PLATFORM_ANDROID");
#endif

	GMaxTextureSamplers = Value_GL_MAX_TEXTURE_IMAGE_UNITS;
	GMaxTextureMipCount = FMath::CeilLogTwo(Value_GL_MAX_TEXTURE_SIZE) + 1;
	GMaxTextureMipCount = FMath::Min<int32>(MAX_TEXTURE_MIP_COUNT, GMaxTextureMipCount);
	GMaxTextureDimensions = Value_GL_MAX_TEXTURE_SIZE;
	
	GRHIGlobals.MaxViewSizeBytesForNonTypedBuffer = 1 << 27;
#if defined(GL_MAX_TEXTURE_BUFFER_SIZE)
	GRHIGlobals.MaxViewDimensionForTypedBuffer = Value_GL_MAX_TEXTURE_BUFFER_SIZE;
#else
	GRHIGlobals.MaxViewDimensionForTypedBuffer = 1 << 27;
#endif

	GMaxCubeTextureDimensions = Value_GL_MAX_CUBE_MAP_TEXTURE_SIZE;
#if defined(GL_MAX_ARRAY_TEXTURE_LAYERS) && GL_MAX_ARRAY_TEXTURE_LAYERS
	GMaxTextureArrayLayers = Value_GL_MAX_ARRAY_TEXTURE_LAYERS;
#endif

#if defined(GL_MAX_COMPUTE_SHARED_MEMORY_SIZE)
	GMaxComputeSharedMemory = Value_GL_MAX_COMPUTE_SHARED_MEMORY_SIZE;
#endif

	GMaxWorkGroupInvocations = Value_GL_MAX_COMPUTE_WORK_GROUP_INVOCATIONS;

	GSupportsParallelRenderingTasksWithSeparateRHIThread = false; // hopefully this is just temporary
	GRHIThreadNeedsKicking = true; // GL is SLOW
	GRHISupportsExactOcclusionQueries = FOpenGL::SupportsExactOcclusionQueries();

	GSupportsVolumeTextureRendering = FOpenGL::SupportsVolumeTextureRendering();
	GSupportsRenderDepthTargetableShaderResources = true;
	GSupportsRenderTargetFormat_PF_G8 = true;
	GSupportsSeparateRenderTargetBlendState = FOpenGL::SupportsSeparateAlphaBlend();
	GSupportsDepthBoundsTest = FOpenGL::SupportsDepthBoundsTest();

	GSupportsRenderTargetFormat_PF_FloatRGBA = FOpenGL::SupportsColorBufferHalfFloat();

	GSupportsWideMRT = FOpenGL::SupportsWideMRT();
	GSupportsTexture3D = FOpenGL::SupportsTexture3D();
	GSupportsMobileMultiView = FOpenGL::SupportsMobileMultiView();
	GSupportsImageExternal = FOpenGL::SupportsImageExternal();

	GSupportsShaderFramebufferFetch = FOpenGL::SupportsShaderFramebufferFetch();
	GSupportsShaderMRTFramebufferFetch = FOpenGL::SupportsShaderMRTFramebufferFetch();
	GSupportsShaderDepthStencilFetch = FOpenGL::SupportsShaderDepthStencilFetch();
	GSupportsPixelLocalStorage = FOpenGL::SupportsPixelLocalStorage();
	GSupportsShaderFramebufferFetchProgrammableBlending = FOpenGL::SupportsShaderFramebufferFetchProgrammableBlending();

	GMaxShadowDepthBufferSizeX = FMath::Min<int32>(Value_GL_MAX_RENDERBUFFER_SIZE, 4096); // Limit to the D3D11 max.
	GMaxShadowDepthBufferSizeY = FMath::Min<int32>(Value_GL_MAX_RENDERBUFFER_SIZE, 4096);
	GHardwareHiddenSurfaceRemoval = FOpenGL::HasHardwareHiddenSurfaceRemoval();
	GSupportsTimestampRenderQueries = FOpenGL::SupportsTimestampQueries();

	GRHIMaxDispatchThreadGroupsPerDimension.X = MAX_uint16;
	GRHIMaxDispatchThreadGroupsPerDimension.Y = MAX_uint16;
	GRHIMaxDispatchThreadGroupsPerDimension.Z = MAX_uint16;

	// It's not possible to create a framebuffer with the backbuffer as the color attachment and a custom renderbuffer as the depth/stencil surface.
	GRHISupportsBackBufferWithCustomDepthStencil = false;

	GRHISupportsLazyShaderCodeLoading = true;

	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES2_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::ES3_1] = (GMaxRHIFeatureLevel == ERHIFeatureLevel::ES3_1) ? GMaxRHIShaderPlatform : SP_OPENGL_PCES3_1;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM4_REMOVED] = SP_NumPlatforms;
	GShaderPlatformForFeatureLevel[ERHIFeatureLevel::SM5] = SP_NumPlatforms;

	// Not implemented in OpenGL RHI. Supported in GLcore 3.3, not in OpenGL ES, extension GL_EXT_blend_func_extended not widely supported.
	GSupportsDualSrcBlending = false;

	GRHISupportsTextureStreaming = true;

	// Disable texture streaming if forced off by r.OpenGL.DisableTextureStreamingSupport
	static const auto CVarDisableOpenGLTextureStreamingSupport = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.OpenGL.DisableTextureStreamingSupport"));
	if (CVarDisableOpenGLTextureStreamingSupport->GetValueOnAnyThread())
	{
		GRHISupportsTextureStreaming = false;
	}

	for (int32 PF = 0; PF < PF_MAX; ++PF)
	{
		SetupTextureFormat(EPixelFormat(PF), FOpenGLTextureFormat());
	}

	GLenum DepthFormat = FOpenGL::GetDepthFormat();
	GLenum ShadowDepthFormat = FOpenGL::GetShadowDepthFormat();

	// Initialize the platform pixel format map.					InternalFormat				InternalFormatSRGB		Format				Type							bCompressed		bBGRA
	SetupTextureFormat( PF_Unknown,				FOpenGLTextureFormat( ));
	SetupTextureFormat( PF_A32B32G32R32F,		FOpenGLTextureFormat( GL_RGBA32F,				GL_RGBA32F,				GL_RGBA,			GL_FLOAT,						false,			false));
	SetupTextureFormat( PF_UYVY,				FOpenGLTextureFormat( ));
	if (CVarGLDepth24Bit.GetValueOnAnyThread() != 0)
	{
		SetupTextureFormat(PF_DepthStencil, FOpenGLTextureFormat(GL_DEPTH24_STENCIL8, GL_NONE, GL_DEPTH_STENCIL, GL_UNSIGNED_INT_24_8, false, false));
		GPixelFormats[PF_DepthStencil].BlockBytes = 4;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = true;
	}
	else
	{
		SetupTextureFormat(PF_DepthStencil, FOpenGLTextureFormat(GL_DEPTH32F_STENCIL8, GL_NONE, GL_DEPTH_STENCIL, GL_FLOAT_32_UNSIGNED_INT_24_8_REV, false, false));
		GPixelFormats[PF_DepthStencil].BlockBytes = 8;
		GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil = false;
	}
	GPixelFormats[PF_X24_G8].Supported = true;
	SetupTextureFormat( PF_ShadowDepth,			FOpenGLTextureFormat( ShadowDepthFormat,		ShadowDepthFormat,		GL_DEPTH_COMPONENT,	GL_UNSIGNED_INT,				false,			false));
	SetupTextureFormat( PF_D24,					FOpenGLTextureFormat( DepthFormat,				DepthFormat,			GL_DEPTH_COMPONENT,	GL_UNSIGNED_INT,				false,			false));
	SetupTextureFormat( PF_A16B16G16R16,		FOpenGLTextureFormat( GL_RGBA16F,				GL_RGBA16F,				GL_RGBA,			GL_HALF_FLOAT,					false,			false));
	SetupTextureFormat( PF_A1,					FOpenGLTextureFormat( ));
	SetupTextureFormat( PF_R16G16B16A16_UINT,	FOpenGLTextureFormat( GL_RGBA16UI,				GL_RGBA16UI,			GL_RGBA_INTEGER,	GL_UNSIGNED_SHORT,				false,			false));
	SetupTextureFormat( PF_R16G16B16A16_SINT,	FOpenGLTextureFormat( GL_RGBA16I,				GL_RGBA16I,				GL_RGBA_INTEGER,	GL_SHORT,						false,			false));
	SetupTextureFormat( PF_R32G32B32A32_UINT,	FOpenGLTextureFormat( GL_RGBA32UI,				GL_RGBA32UI,			GL_RGBA_INTEGER,	GL_UNSIGNED_INT,				false,			false));
	SetupTextureFormat( PF_R16G16B16A16_UNORM,	FOpenGLTextureFormat( GL_RGBA16,				GL_RGBA16,				GL_RGBA,			GL_UNSIGNED_SHORT,				false,			false));
	SetupTextureFormat( PF_R16G16B16A16_SNORM,	FOpenGLTextureFormat( GL_RGBA16,				GL_RGBA16,				GL_RGBA,			GL_SHORT,						false,			false));
	
	SetupTextureFormat( PF_R16G16_UINT,			FOpenGLTextureFormat( GL_RG16UI,				GL_RG16UI,				GL_RG_INTEGER,		GL_UNSIGNED_SHORT,				false,			false));
	SetupTextureFormat( PF_R16G16_SINT,			FOpenGLTextureFormat( GL_RG16I,					GL_RG16I,				GL_RG_INTEGER,		GL_SHORT,						false,			false));
	SetupTextureFormat( PF_R8,					FOpenGLTextureFormat( GL_R8,					GL_R8,					GL_RED,				GL_UNSIGNED_BYTE,				false,			false));

	SetupTextureFormat( PF_R5G6B5_UNORM,        FOpenGLTextureFormat( GL_RGB565,                GL_RGB565,              GL_RGB,             GL_UNSIGNED_SHORT_5_6_5,        false,          false));

	//GL_UNSIGNED_SHORT_1_5_5_5_REV is not defined in gles
	GLenum GL5551Format = FOpenGL::GetPlatfrom5551Format();
	//the last 5 bits of GL_UNSIGNED_SHORT_1_5_5_5_REV is Red, while the last 5 bits of DXGI_FORMAT_B5G5R5A1_UNORM is Blue
	bool bNeedsToSwizzleRedBlue = GL5551Format != GL_UNSIGNED_SHORT_5_5_5_1;
	SetupTextureFormat( PF_B5G5R5A1_UNORM,		FOpenGLTextureFormat( GL_RGB5_A1,               GL_RGB5_A1,             GL_RGBA,			GL5551Format,					false,          bNeedsToSwizzleRedBlue));
	
#if PLATFORM_ANDROID
	// Unfortunatelly most OpenGLES devices do not support R16Unorm pixel format, fallback to a less precise R16F
	SetupTextureFormat( PF_G16,					FOpenGLTextureFormat( GL_R16F,					GL_R16F,				GL_RED,				GL_HALF_FLOAT,						false,	false));
#else
	SetupTextureFormat( PF_G16,					FOpenGLTextureFormat( GL_R16,					GL_R16,					GL_RED,				GL_UNSIGNED_SHORT,					false,	false));
#endif
	SetupTextureFormat( PF_R32_FLOAT,			FOpenGLTextureFormat( GL_R32F,					GL_R32F,				GL_RED,				GL_FLOAT,							false,	false));
	SetupTextureFormat( PF_G16R16F,				FOpenGLTextureFormat( GL_RG16F,					GL_RG16F,				GL_RG,				GL_HALF_FLOAT,						false,	false));
	SetupTextureFormat( PF_G16R16F_FILTER,		FOpenGLTextureFormat( GL_RG16F,					GL_RG16F,				GL_RG,				GL_HALF_FLOAT,						false,	false));
	SetupTextureFormat( PF_G32R32F,				FOpenGLTextureFormat( GL_RG32F,					GL_RG32F,				GL_RG,				GL_FLOAT,							false,	false));
	SetupTextureFormat( PF_A2B10G10R10,			FOpenGLTextureFormat( GL_RGB10_A2,				GL_RGB10_A2,			GL_RGBA,			GL_UNSIGNED_INT_2_10_10_10_REV,		false,	false));
	SetupTextureFormat( PF_R16F,				FOpenGLTextureFormat( GL_R16F,					GL_R16F,				GL_RED,				GL_HALF_FLOAT,						false,	false));
	SetupTextureFormat( PF_R16F_FILTER,			FOpenGLTextureFormat( GL_R16F,					GL_R16F,				GL_RED,				GL_HALF_FLOAT,						false,	false));
	SetupTextureFormat( PF_A8,					FOpenGLTextureFormat( GL_R8,					GL_R8,					GL_RED,				GL_UNSIGNED_BYTE,					false,	false));
	SetupTextureFormat( PF_R32_UINT,			FOpenGLTextureFormat( GL_R32UI,					GL_R32UI,				GL_RED_INTEGER,		GL_UNSIGNED_INT,					false,	false));
	SetupTextureFormat( PF_R32_SINT,			FOpenGLTextureFormat( GL_R32I,					GL_R32I,				GL_RED_INTEGER,		GL_INT,								false,	false));
	SetupTextureFormat( PF_R16_UINT,			FOpenGLTextureFormat( GL_R16UI,					GL_R16UI,				GL_RED_INTEGER,		GL_UNSIGNED_SHORT,					false,	false));
	SetupTextureFormat( PF_R16_SINT,			FOpenGLTextureFormat( GL_R16I,					GL_R16I,				GL_RED_INTEGER,		GL_SHORT,							false,	false));
	SetupTextureFormat( PF_R8_UINT,				FOpenGLTextureFormat( GL_R8UI,					GL_R8UI,				GL_RED_INTEGER,		GL_UNSIGNED_BYTE,					false,  false));
	SetupTextureFormat( PF_R8G8,				FOpenGLTextureFormat( GL_RG8,					GL_RG8,					GL_RG,				GL_UNSIGNED_BYTE,					false,	false));
	SetupTextureFormat( PF_R32G32_UINT,			FOpenGLTextureFormat( GL_RG32UI,				GL_RG32UI,				GL_RG_INTEGER,		GL_UNSIGNED_INT,					false,	false));
	// Should be GL_RGBA8_SNORM but it doesn't work with glTexBuffer -> mapping to Unorm and unpacking in the shader
	SetupTextureFormat( PF_R8G8B8A8_SNORM,		FOpenGLTextureFormat( GL_RGBA8,					GL_RGBA8,				GL_RGBA,			GL_UNSIGNED_BYTE,					false, false));
	
	SetupTextureFormat( PF_FloatRGB,			FOpenGLTextureFormat( GL_R11F_G11F_B10F,		GL_R11F_G11F_B10F,		GL_RGB,				GL_UNSIGNED_INT_10F_11F_11F_REV,	false,  false));
	SetupTextureFormat( PF_FloatR11G11B10,		FOpenGLTextureFormat( GL_R11F_G11F_B10F,		GL_R11F_G11F_B10F,		GL_RGB,				GL_UNSIGNED_INT_10F_11F_11F_REV,	false,  false));
	SetupTextureFormat( PF_FloatRGBA,			FOpenGLTextureFormat( GL_RGBA16F,				GL_RGBA16F,				GL_RGBA,			GL_HALF_FLOAT,						false,	false));

	SetupTextureFormat( PF_R8G8_UINT,			FOpenGLTextureFormat( GL_RG8UI,					GL_RG8UI,				GL_RG_INTEGER,		GL_UNSIGNED_BYTE,					false, false));
	SetupTextureFormat( PF_R32G32B32_UINT,		FOpenGLTextureFormat( GL_RGB32UI,				GL_RGB32UI,				GL_RGB_INTEGER,		GL_UNSIGNED_INT,					false, false));
	SetupTextureFormat( PF_R32G32B32_SINT,		FOpenGLTextureFormat( GL_RGB32I,				GL_RGB32I,				GL_RGB_INTEGER,		GL_INT,								false, false));
	SetupTextureFormat( PF_R32G32B32F,			FOpenGLTextureFormat( GL_RGB32F,				GL_RGB32F,				GL_RGB,				GL_FLOAT,							false, false));
	SetupTextureFormat( PF_R8_SINT,				FOpenGLTextureFormat( GL_R8I,					GL_R8I,					GL_RED_INTEGER,		GL_BYTE,							false, false));

	SetupTextureFormat( PF_B8G8R8A8,			FOpenGLTextureFormat( GL_RGBA8,					GL_SRGB8_ALPHA8,		GL_RGBA,			GL_UNSIGNED_BYTE,					false, true));
	SetupTextureFormat( PF_R8G8B8A8,			FOpenGLTextureFormat( GL_RGBA8,					GL_SRGB8_ALPHA8,		GL_RGBA,			GL_UNSIGNED_BYTE,					false, false));
	SetupTextureFormat( PF_R8G8B8A8_UINT,		FOpenGLTextureFormat( GL_RGBA8UI,				GL_RGBA8UI,				GL_RGBA_INTEGER,	GL_UNSIGNED_BYTE,					false, false));
	SetupTextureFormat( PF_G8,					FOpenGLTextureFormat( GL_R8,					GL_R8,					GL_RED,				GL_UNSIGNED_BYTE,					false, false));
	SetupTextureFormat( PF_R8G8B8,				FOpenGLTextureFormat( GL_RGB8,					GL_SRGB8,				GL_RGB,				GL_UNSIGNED_BYTE,					false, false));
#if PLATFORM_DESKTOP
	CA_SUPPRESS(6286);
	if (PLATFORM_DESKTOP)
	{
		SetupTextureFormat( PF_V8U8,			FOpenGLTextureFormat( GL_RG8_SNORM,				GL_NONE,				GL_RG,			GL_BYTE,							false, false));
		SetupTextureFormat( PF_BC5,				FOpenGLTextureFormat( GL_COMPRESSED_RG_RGTC2,	GL_COMPRESSED_RG_RGTC2,	GL_RG,			GL_UNSIGNED_BYTE,					true,	false));
		SetupTextureFormat( PF_BC4,				FOpenGLTextureFormat( GL_COMPRESSED_RED_RGTC1,	GL_COMPRESSED_RED_RGTC1,GL_RED,			GL_UNSIGNED_BYTE,					true,	false));
		
		SetupTextureFormat( PF_BC7,				FOpenGLTextureFormat( GL_COMPRESSED_RGBA_BPTC_UNORM, GL_COMPRESSED_SRGB_ALPHA_BPTC_UNORM, GL_RGBA, GL_UNSIGNED_BYTE, true, false));
		SetupTextureFormat( PF_BC6H,			FOpenGLTextureFormat( GL_COMPRESSED_RGB_BPTC_UNSIGNED_FLOAT, GL_COMPRESSED_RGB_BPTC_UNSIGNED_FLOAT, GL_RGB, GL_HALF_FLOAT, true, false));

		SetupTextureFormat( PF_G16R16,			FOpenGLTextureFormat(GL_RG16, GL_RG16, GL_RG, GL_UNSIGNED_SHORT, false, false));
		SetupTextureFormat( PF_G16R16_SNORM,	FOpenGLTextureFormat(GL_RG16_SNORM, GL_RG16_SNORM, GL_RG, GL_SHORT, false, false));
	}
	else
#endif // PLATFORM_DESKTOP
	{
#if !PLATFORM_DESKTOP
		FOpenGL::PE_SetupTextureFormat(&SetupTextureFormat); // platform extension
#endif // !PLATFORM_DESKTOP
	}

	if (FOpenGL::SupportsDXT())
	{
		SetupTextureFormat( PF_DXT1,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_S3TC_DXT1_EXT,	GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT1_EXT,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false));
		SetupTextureFormat( PF_DXT3,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_S3TC_DXT3_EXT,	GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT3_EXT,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false));
		SetupTextureFormat( PF_DXT5,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_S3TC_DXT5_EXT,	GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT5_EXT,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false));
	}
#if PLATFORM_ANDROID
	if ( FOpenGL::SupportsETC2() )
	{
		SetupTextureFormat( PF_ETC2_RGB,		FOpenGLTextureFormat(GL_COMPRESSED_RGB8_ETC2,		GL_COMPRESSED_SRGB8_ETC2,					GL_RGBA,		GL_UNSIGNED_BYTE,	true,		false));
		SetupTextureFormat( PF_ETC2_RGBA,		FOpenGLTextureFormat(GL_COMPRESSED_RGBA8_ETC2_EAC,	GL_COMPRESSED_SRGB8_ALPHA8_ETC2_EAC,		GL_RGBA,		GL_UNSIGNED_BYTE,	true,		false));
		SetupTextureFormat( PF_ETC2_R11_EAC,	FOpenGLTextureFormat(GL_COMPRESSED_R11_EAC,			GL_COMPRESSED_R11_EAC,						GL_RED,			GL_UNSIGNED_BYTE,	true,		false));
		SetupTextureFormat( PF_ETC2_RG11_EAC,	FOpenGLTextureFormat(GL_COMPRESSED_RG11_EAC,		GL_COMPRESSED_RG11_EAC,						GL_RG,			GL_UNSIGNED_BYTE,	true,		false));
	}
#endif
	if (FOpenGL::SupportsASTC())
	{
		SetupTextureFormat( PF_ASTC_4x4,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_4x4_KHR,	GL_COMPRESSED_SRGB8_ALPHA8_ASTC_4x4_KHR,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false) );
		SetupTextureFormat( PF_ASTC_6x6,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_6x6_KHR,	GL_COMPRESSED_SRGB8_ALPHA8_ASTC_6x6_KHR,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false) );
		SetupTextureFormat( PF_ASTC_8x8,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_8x8_KHR,	GL_COMPRESSED_SRGB8_ALPHA8_ASTC_8x8_KHR,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false) );
		SetupTextureFormat( PF_ASTC_10x10,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_10x10_KHR,	GL_COMPRESSED_SRGB8_ALPHA8_ASTC_10x10_KHR,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false) );
		SetupTextureFormat( PF_ASTC_12x12,	FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_12x12_KHR,	GL_COMPRESSED_SRGB8_ALPHA8_ASTC_12x12_KHR,	GL_RGBA,	GL_UNSIGNED_BYTE,	true,	false) );
	}
	if (FOpenGL::SupportsASTCHDR())
	{
		SetupTextureFormat(PF_ASTC_4x4_HDR, FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_4x4_KHR, GL_COMPRESSED_SRGB8_ALPHA8_ASTC_4x4_KHR, GL_RGBA, GL_HALF_FLOAT, true, false));
		SetupTextureFormat(PF_ASTC_6x6_HDR, FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_6x6_KHR, GL_COMPRESSED_SRGB8_ALPHA8_ASTC_6x6_KHR, GL_RGBA, GL_HALF_FLOAT, true, false));
		SetupTextureFormat(PF_ASTC_8x8_HDR, FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_8x8_KHR, GL_COMPRESSED_SRGB8_ALPHA8_ASTC_8x8_KHR, GL_RGBA, GL_HALF_FLOAT, true, false));
		SetupTextureFormat(PF_ASTC_10x10_HDR, FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_10x10_KHR, GL_COMPRESSED_SRGB8_ALPHA8_ASTC_10x10_KHR, GL_RGBA, GL_HALF_FLOAT, true, false));
		SetupTextureFormat(PF_ASTC_12x12_HDR, FOpenGLTextureFormat(GL_COMPRESSED_RGBA_ASTC_12x12_KHR, GL_COMPRESSED_SRGB8_ALPHA8_ASTC_12x12_KHR, GL_RGBA, GL_HALF_FLOAT, true, false));
	}
	// Some formats need to know how large a block is.
	GPixelFormats[ PF_FloatRGB			].BlockBytes	 = 4;
	GPixelFormats[ PF_FloatRGBA			].BlockBytes	 = 8;

	// Temporary fix for nvidia driver issue with non-power-of-two shadowmaps (9/8/2016) UE-35312
	// @TODO revisit this with newer drivers
	GRHINeedsUnatlasedCSMDepthsWorkaround = true;

	GRHISupportsPSOPrecaching = CVarAllowPSOPrecaching.GetValueOnAnyThread();
	GRHISupportsPipelineFileCache = !GRHISupportsPSOPrecaching || CVarEnablePSOFileCacheWhenPrecachingActive.GetValueOnAnyThread();

	GRHIGlobals.NeedsShaderUnbinds = true;

	// OpenGL always uses GLSL (aka GLSL ES or ESSL). This is used to differentiate standard and preview shaders in the DDC.
	GRHIGlobals.PreferredPreviewShaderCodeFormat = TEXT("GLSL");
}

FDynamicRHI* FOpenGLDynamicRHIModule::CreateRHI(ERHIFeatureLevel::Type InRequestedFeatureLevel)
{
	GRequestedFeatureLevel = InRequestedFeatureLevel;
	return new FOpenGLDynamicRHI();
}


FOpenGLDynamicRHI::FOpenGLDynamicRHI()
{
	check(Singleton == nullptr);
	Singleton = this;

	// This should be called once at the start
	check( IsInGameThread() );
	check( !GIsThreadedRendering );

#if PLATFORM_ANDROID
	GRHINeedsExtraDeletionLatency = CVarGLExtraDeletionLatency.GetValueOnAnyThread() == 1;
#endif

	// Ogl must have all programs created on the context owning thread.
	// when GRHISupportsAsyncPipelinePrecompile is true pipelinefilecache will call RHICreateGraphicsPipelineState from any thread and
	// RHISetGraphicsPipelineState will not be called for precompiled programs.
	GRHISupportsAsyncPipelinePrecompile = false;

	PlatformInitOpenGL();
	PlatformDevice = PlatformCreateOpenGLDevice();
	VERIFY_GL_SCOPE();
	InitRHICapabilitiesForGL();

	if (PlatformCanEnableGPUCapture())
	{
		EnableIdealGPUCaptureOptions(true);

		// Disable persistent mapping
		{
			auto* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("OpenGL.UBODirectWrite"));
			if (CVar)
			{
				CVar->Set(false);
			}
		}
		{
			auto* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("OpenGL.UseStagingBuffer"));
			if (CVar)
			{
				CVar->Set(false);
			}
		}
	}

	{
		// Temp disable gpusorting for Opengl because of issues on Adreno and Mali devices
		auto* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("FX.AllowGPUSorting"));
		if (CVar)
		{
#if PLATFORM_ANDROID
			if(!AndroidThunkCpp_IsXRImmersiveHMDApplication())
#endif
			{
				CVar->Set(false);
			}
		}
	}

#if PLATFORM_ANDROID
	// Temp disable gpu particles for OpenGL because of issues on Adreno devices with old driver version
	if (GRHIVendorId == 0x5143)
	{
		auto* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("fx.NiagaraAllowGPUParticles"));
		if (CVar)
		{
			int32 DriverVersion = INT32_MAX;
			FString SubVersionString, DriverVersionString;
			FString VersionString = FString(ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VERSION)));
			
			if (VersionString.Split(TEXT("V@"), nullptr, &SubVersionString) && SubVersionString.Split(TEXT(" "), &DriverVersionString, nullptr))
			{
				DriverVersion = FCString::Atoi(*DriverVersionString);
			}

			if (DriverVersion < 415)
			{
				CVar->Set(false);
				UE_LOGF(LogRHI, Log, "GPU particles are disabled on this device because the driver version %d is less than 415", DriverVersion);
			}
		}
	}
#endif
	
	// Disable SingleRHIThreadStall for GL occlusion queiresn, which should be set for D3D11 only. Enabling it causes RT->RHIT deadlock
	{
		auto* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Occlusion.SingleRHIThreadStall"));
		if (CVar)
		{
			CVar->Set(0);
		}
	}

	PrivateOpenGLDevicePtr = this;
	GlobalUniformBuffers.AddZeroed(FUniformBufferStaticSlotRegistry::Get().GetSlotCount());
}

extern void DestroyShadersAndPrograms();

#if PLATFORM_ANDROID
extern bool GOpenGLShaderHackLastCompileSuccess;

// only used to test for shader compatibility issues
static bool VerifyCompiledShader(GLuint Shader, const ANSICHAR* GlslCode, bool IsFatal)
{
	SCOPE_CYCLE_COUNTER(STAT_OpenGLShaderCompileVerifyTime);

	if (!GOpenGLShaderHackLastCompileSuccess)
	{
#if DEBUG_GL_SHADERS
		if (GlslCode)
		{
			UE_LOGF(LogRHI, Warning, "Shader:\n%ls", ANSI_TO_TCHAR(GlslCode));

			const ANSICHAR *Temp = GlslCode;

			for (int i = 0; i < 30 && (*Temp != '\0'); ++i)
			{
				FString Converted = ANSI_TO_TCHAR(Temp);
				Converted = Converted.LeftChop(256);

				UE_LOGF(LogRHI, Display, "%ls", *Converted);
				Temp += Converted.Len();
			}
		}
#endif
		UE_LOGF(LogRHI, Warning, "Failed to compile shader.");

		return false;
	}
	return true;

}
#endif

void FOpenGLDynamicRHI::Init()
{
	check(!GIsRHIInitialized);
	VERIFY_GL_SCOPE();

	FOpenGLProgramBinaryCache::Initialize();

	InitializeStateResources();

	// Create a default point sampler state for internal use.
	FSamplerStateInitializerRHI PointSamplerStateParams(SF_Point,AM_Clamp,AM_Clamp,AM_Clamp);
	PointSamplerState = this->RHICreateSamplerState(PointSamplerStateParams);

#if PLATFORM_WINDOWS || PLATFORM_LINUX

	extern int64 GOpenGLDedicatedVideoMemory;
	extern int64 GOpenGLTotalGraphicsMemory;

	GOpenGLDedicatedVideoMemory = FOpenGL::GetVideoMemorySize();

	if ( GOpenGLDedicatedVideoMemory != 0)
	{
		GOpenGLTotalGraphicsMemory = GOpenGLDedicatedVideoMemory;

		if ( GPoolSizeVRAMPercentage > 0 )
		{
			float PoolSize = float(GPoolSizeVRAMPercentage) * 0.01f * float(GOpenGLTotalGraphicsMemory);

			// Truncate GTexturePoolSize to MB (but still counted in bytes)
			GTexturePoolSize = int64(FGenericPlatformMath::TruncToInt(PoolSize / 1024.0f / 1024.0f)) * 1024 * 1024;

			UE_LOGF(LogRHI, Log, "Texture pool is %llu MB (%d%% of %llu MB)",
				GTexturePoolSize / 1024 / 1024,
				GPoolSizeVRAMPercentage,
				GOpenGLTotalGraphicsMemory / 1024 / 1024);
		}
	}

#else

	static const auto CVarStreamingTexturePoolSize = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Streaming.PoolSize"));
	GTexturePoolSize = (int64)CVarStreamingTexturePoolSize->GetValueOnAnyThread() * 1024 * 1024;

	UE_LOGF(LogRHI,Log,"Texture pool is %llu MB (of %llu MB total graphics mem)",
			GTexturePoolSize / 1024 / 1024,
			FOpenGL::GetVideoMemorySize());

#endif

	// Flush here since we might be switching to a different context/thread for rendering
	FOpenGL::Flush();

	Profiler.Setup();

	FHardwareInfo::RegisterHardwareInfo( NAME_RHI, TEXT( "OpenGL" ) );

	FRenderResource::InitPreRHIResources();
	GIsRHIInitialized = true;
}

void FOpenGLDynamicRHI::Shutdown()
{
	check(IsInGameThread() && IsInRenderingThread()); // require that the render thread has been shut down

	Profiler.Shutdown();
	Cleanup();

	FOpenGLRenderQuery::Cleanup();

	DestroyShadersAndPrograms();
	PlatformDestroyOpenGLDevice(PlatformDevice);

	PrivateOpenGLDevicePtr = NULL;
}

void FOpenGLDynamicRHI::Cleanup()
{
	if(GIsRHIInitialized)
	{
		FOpenGLProgramBinaryCache::Shutdown();

		// Reset the RHI initialized flag.
		GIsRHIInitialized = false;

		// Ask all initialized FRenderResources to release their RHI resources.
		FRenderResource::ReleaseRHIForAllResources();
	}

	// Release the point sampler state.
	PointSamplerState.SafeRelease();

	extern void EmptyGLSamplerStateCache();
	EmptyGLSamplerStateCache();

	// Release zero-filled dummy uniform buffer, if it exists.
	if (PendingState.ZeroFilledDummyUniformBuffer)
	{
		FOpenGL::DeleteBuffers(1, &PendingState.ZeroFilledDummyUniformBuffer);
		PendingState.ZeroFilledDummyUniformBuffer = 0;
		OpenGLBufferStats::UpdateUniformBufferStats(ZERO_FILLED_DUMMY_UNIFORM_BUFFER_SIZE, false);
	}

	// Release pending shader
	PendingState.BoundShaderState.SafeRelease();
	check(!IsValidRef(PendingState.BoundShaderState));

	VERIFY_GL_SCOPE();
	PendingState.CleanupResources();
	ContextState.CleanupResources();
}

void FOpenGLDynamicRHI::RHIAcquireThreadOwnership()
{
	PlatformRenderingContextSetup(PlatformDevice);
	VERIFY_GL(RHIAcquireThreadOwnership);
}

void FOpenGLDynamicRHI::RHIReleaseThreadOwnership()
{
	VERIFY_GL(RHIReleaseThreadOwnership);
	PlatformNULLContextSetup();
}

void* FOpenGLDynamicRHI::RHIGetNativeDevice()
{
	return PlatformDevice;
}

void FOpenGLDynamicRHI::SetCustomPresent(FRHICustomPresent* InCustomPresent)
{
	FScopeLock lock(&CustomPresentSection);
	CustomPresent = InCustomPresent;
}

uint64 FOpenGLDynamicRHI::RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format)
{
	check(FOpenGL::GetTextureBufferAlignment()>=0);
	return FOpenGL::GetTextureBufferAlignment();
}

bool FOpenGLDynamicRHIModule::IsSupported()
{
	return true;
}

FOpenGLRenderQuery* FOpenGLProfiler::GetQuery()
{
	if (QueryPool.IsEmpty())
	{
		return new FOpenGLRenderQuery(FOpenGLRenderQuery::EType::Profiler);
	}
	else
	{
		return QueryPool.Pop();
	}
}

FOpenGLRenderQuery* FOpenGLProfiler::InsertQuery(uint64* Target)
{
	check(bEnabled);

	// Temporarily store the current frame pointer in the Target.
	// We retrieve this later when resolving the query.
	*Target = reinterpret_cast<uint64>(Current.Get());

	FOpenGLRenderQuery* Query = GetQuery();
	Query->End(Target);
	return Query;
}

FOpenGLProfiler::FFrame::FFrame(FOpenGLProfiler& Profiler)
	: EventStream(UE::RHI::GPUProfiler::FQueue(UE::RHI::GPUProfiler::FQueue::EType::Graphics, 0, 0))
{
	if (Profiler.bEnabled)
	{
		glGetInteger64v(GL_TIMESTAMP_EXT, (GLint64*)&Calibration.GLTimestamp);
		Calibration.CPUTimestamp = FPlatformTime::Cycles64();
		Calibration.CPUFrequency = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64());
	}
}

void FOpenGLProfiler::Setup()
{
	bEnabled = false;

	if (!FOpenGL::SupportsDisjointTimeQueries())
	{
		UE_LOGF(LogOpenGL, Warning, "OpenGL RHI profiler not supported.");
	}
	else
	{
		// Test for the correctness of the GL timestamp functions.
		// On some devices, the values reported by the GPU queries do not tick at the same rate as those reported by the CPU function.
		
		GLuint Query;
		FOpenGL::GenQueries(1, &Query);

		GLuint64 GPU = 0;
		GLint64 CPU = 0;

		for (int32 Attempts = 0; Attempts < 10; ++Attempts)
		{
			GPU = 0;
			CPU = 0;

			// Sample both the GPU and CPU timestamps, as close to simultenously as possible
			FOpenGL::QueryTimestampCounter(Query);
			FOpenGL::GetQueryObject(Query, FOpenGL::QM_Result, &GPU);
			glGetInteger64v(GL_TIMESTAMP_EXT, &CPU);

			GLint64 Delta = FMath::Abs(GLint64(GPU) - CPU);

			// We consider the timestamps reliable if the GPU and CPU clocks are within 100ms of each other.
			// On devices that don't implement this properly, the difference between the clocks can be minutes / hours.
			const GLint64 Threshold = 100 * 1000 * 1000; // 100ms in nanoseconds
			if (!FOpenGL::TimerQueryDisjoint() && GPU != 0 && CPU != 0 && Delta < Threshold)
			{
				bEnabled = true;
				break;
			}
		}

		FOpenGL::DeleteQueries(1, &Query);

		if (bEnabled)
		{
			UE_LOGF(LogOpenGL, Display, "OpenGL RHI profiler detected reliable timestamps. GPU profiler is enabled. GPU clock: 0x%llx, CPU Clock: 0x%llx", GPU, CPU);
		}
		else
		{
			UE_LOGF(LogOpenGL, Warning, "OpenGL RHI profiler detected unreliable timestamps. GPU profiler is disabled. GPU clock: 0x%llx, CPU Clock: 0x%llx", GPU, CPU);
		}
	}

	// Register the single graphics GPU queue we have access to in OpenGL.
	UE::RHI::GPUProfiler::FQueue Queue(UE::RHI::GPUProfiler::FQueue::EType::Graphics, 0, 0);
	UE::RHI::GPUProfiler::InitializeQueues(MakeConstArrayView(&Queue, 1));

	Current = MakeUnique<FFrame>(*this);

	if (bEnabled)
	{
		//
		// Since we can't tell when the GPU is actually executing engine work in OpenGL,
		// just mark the GPU as always busy in the frame. We also push begin/end work
		// markers in RHIEndFrame either side of the frame boundary.
		//
		auto& Event = EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(Current->Calibration.CPUTimestamp);
		InsertQuery(&Event.GPUTimestampTOP);
	}
}

void FOpenGLProfiler::Shutdown()
{
	// Drain the query readback queue
	FOpenGLRenderQuery::PollQueryResults();
}

void FOpenGLProfiler::EndFrame(FRHIEndFrameArgs const& Args)
{
	if (bEnabled)
	{
		auto& EventWork = EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FEndWork>();
		Current->EndWorkQuery = InsertQuery(&EventWork.GPUTimestampBOP);
	}

	if (ExternalGPUTime)
	{
		EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FFrameTime>(*ExternalGPUTime);
	}

	// Flushing after the EndWork event helps improve the accuracy of the timer
	FOpenGL::Flush();

	// We must always emit the frame boundary event, even if the profiler is disabled in the RHI.
	TUniquePtr<FFrame> NewFrame = MakeUnique<FFrame>(*this);

	// Insert frame boundary
	Current->FrameBoundaryEvent = &EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(
		ERHIPipeline::Graphics,
		Args,
		NewFrame->Calibration.CPUTimestamp
	);

	Pending.Enqueue(MoveTemp(Current));
	Current = MoveTemp(NewFrame);

	// Attempt to process historic results
	while (TUniquePtr<FFrame>* PreviousFramePtr = Pending.Peek())
	{
		TUniquePtr<FFrame>& PreviousFrame = *PreviousFramePtr;

		// When the profiler is disabled, EndWorkQuery will be nullptr
		if (PreviousFrame->EndWorkQuery && !FOpenGLRenderQuery::PollQueryResults(PreviousFrame->EndWorkQuery))
		{
			// Frame not yet finished on the GPU
			break;
		}

		// Previous frame has completed and the data is available.

		if (PreviousFrame->bDisjoint)
		{
			// Discard all the events from this frame, except the frame boundary
			UE::RHI::GPUProfiler::FEventStream Stream(PreviousFrame->EventStream.Queue);
			Stream.Emplace<UE::RHI::GPUProfiler::FEvent::FFrameBoundary>(*PreviousFrame->FrameBoundaryEvent);

			UE::RHI::GPUProfiler::ProcessEvents(MakeArrayView(&Stream, 1));
		}
		else
		{
			// This is a valid frame. Publish the profiler events.
			UE::RHI::GPUProfiler::ProcessEvents(MakeArrayView(&PreviousFrame->EventStream, 1));
		}

		Pending.Pop();
	}

	if (bEnabled)
	{
		// Start the next frame's GPU work
		auto& BeginWork = EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FBeginWork>(Current->Calibration.CPUTimestamp);
		InsertQuery(&BeginWork.GPUTimestampTOP);
	}
}

uint64 FOpenGLProfiler::ResolveQuery(uint64 Value, uint64* Target, bool bDisjoint)
{
	check(bEnabled);

	// Read the current frame pointer back out of the target, which was stored here during InsertQuery().
	FFrame* Frame = reinterpret_cast<FFrame*>(*Target);

	Frame->bDisjoint |= bDisjoint;
	if (Frame->bDisjoint)
	{
		return 0;
	}
	else
	{
		// Convert from GPU timestamp to CPU timestamp (relative to FPlatformTime::Cycles64())
		int64 GLDelta = Value - Frame->Calibration.GLTimestamp;
		int64 CPUDelta = (GLDelta * Frame->Calibration.CPUFrequency) / Frame->Calibration.GLFrequency;

		return CPUDelta + Frame->Calibration.CPUTimestamp;
	}
}
