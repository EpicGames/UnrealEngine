// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLDrv.cpp: Unreal OpenGL RHI library implementation.
=============================================================================*/

#include "OpenGLDrv.h"
#include "Modules/ModuleManager.h"
#include "EngineGlobals.h"
#include "StaticBoundShaderState.h"
#include "RHIStaticStates.h"
#include "Engine/Engine.h"
#include "OpenGLDrvPrivate.h"
#include "PipelineStateCache.h"
#include "Engine/GameViewportClient.h"
#include "DataDrivenShaderPlatformInfo.h"


IMPLEMENT_MODULE(FOpenGLDynamicRHIModule, OpenGLDrv);

#include "Shader.h"
#include "OneColorShader.h"
#include "OpenGLShaders.h"

/** OpenGL Logging. */
DEFINE_LOG_CATEGORY(LogOpenGL);

#define LOCTEXT_NAMESPACE "OpenGLDrv"

ERHIFeatureLevel::Type GRequestedFeatureLevel = ERHIFeatureLevel::Num;


int32 FOpenGLDynamicRHI::RHIGetGLMajorVersion() const
{
	return FOpenGL::GetMajorVersion();
}

int32 FOpenGLDynamicRHI::RHIGetGLMinorVersion() const
{
	return FOpenGL::GetMinorVersion();
}

bool FOpenGLDynamicRHI::RHISupportsFramebufferSRGBEnable() const
{
	return FOpenGL::SupportsFramebufferSRGBEnable();
}

GLuint FOpenGLDynamicRHI::RHIGetResource(FRHITexture* InTexture) const
{
	FOpenGLTexture* GLTexture = ResourceCast(InTexture);
	return GLTexture->GetResource();
}

bool FOpenGLDynamicRHI::RHIIsValidTexture(GLuint InTexture) const
{
	return glIsTexture(InTexture) == GL_TRUE;
}

void FOpenGLDynamicRHI::RHISetExternalGPUTime(uint64 InExternalGPUTime)
{
	Profiler.ExternalGPUTime = InExternalGPUTime;
}

#if PLATFORM_ANDROID

EGLDisplay FOpenGLDynamicRHI::RHIGetEGLDisplay() const
{
	return AndroidEGL::GetInstance()->GetDisplay();
}

EGLSurface FOpenGLDynamicRHI::RHIGetEGLSurface() const
{
	return AndroidEGL::GetInstance()->GetSurface();
}

EGLConfig FOpenGLDynamicRHI::RHIGetEGLConfig() const
{
	return AndroidEGL::GetInstance()->GetConfig();
}

EGLContext FOpenGLDynamicRHI::RHIGetEGLContext() const
{
	return AndroidEGL::GetInstance()->GetRenderingContext()->eglContext;
}

ANativeWindow* FOpenGLDynamicRHI::RHIGetEGLNativeWindow() const
{
	return AndroidEGL::GetInstance()->GetNativeWindow();
}

bool FOpenGLDynamicRHI::RHIEGLSupportsNoErrorContext() const
{
	return AndroidEGL::GetInstance()->GetSupportsNoErrorContext();
}

void FOpenGLDynamicRHI::RHIInitEGLInstanceGLES2()
{
	AndroidEGL::GetInstance()->Init(AndroidEGL::AV_OpenGLES, 2, 0);
	AndroidEGL::GetInstance()->InitRenderSurface(false, false, nullptr);
}

void FOpenGLDynamicRHI::RHIInitEGLBackBuffer()
{
	AndroidEGL::GetInstance()->InitBackBuffer(nullptr);
}

void FOpenGLDynamicRHI::RHIEGLSetCurrentRenderingContext()
{
	AndroidEGL::GetInstance()->SetCurrentRenderingContext(nullptr);
}

void FOpenGLDynamicRHI::RHIEGLTerminateContext()
{
	AndroidEGL::GetInstance()->Terminate();
}
#endif

#if WITH_RHI_BREADCRUMBS
	void FOpenGLDynamicRHI::RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		const TCHAR* NameStr = nullptr;
		FRHIBreadcrumb::FBuffer Buffer;
		auto GetNameStr = [&]()
		{
			if (!NameStr)
			{
				NameStr = Breadcrumb->GetTCHAR(Buffer);
			}
			return NameStr;
		};

		if (ShouldEmitBreadcrumbs())
		{
	#if ENABLE_OPENGL_DEBUG_GROUPS
			// @todo-mobile: Fix string conversion ASAP!
			// @todo dev-pr avoid TCHAR -> ANSI conversion
			FOpenGL::PushGroupMarker(TCHAR_TO_ANSI(GetNameStr()));
	#endif
		}

		FlushProfilerStats();

		if (Profiler.bEnabled)
		{
			auto& Event = Profiler.EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FBeginBreadcrumb>(Breadcrumb);
			Profiler.InsertQuery(&Event.GPUTimestampTOP);
		}
	}

	void FOpenGLDynamicRHI::RHIEndBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb)
	{
		FlushProfilerStats();

		if (Profiler.bEnabled)
		{
			auto& Event = Profiler.EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FEndBreadcrumb>(Breadcrumb);
			Profiler.InsertQuery(&Event.GPUTimestampBOP);
		}

		if (ShouldEmitBreadcrumbs())
		{
	#if ENABLE_OPENGL_DEBUG_GROUPS
			FOpenGL::PopGroupMarker();
	#endif
		}
	}
#endif // WITH_RHI_BREADCRUMBS

// only use shader hashes to determine GL PSO hash;
uint64 FOpenGLDynamicRHI::RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	struct FHashKey
	{
		FShaderHash VertexShader;
		FShaderHash PixelShader;
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
		FShaderHash GeometryShader;
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
#if PLATFORM_SUPPORTS_MESH_SHADERS
		FShaderHash MeshShader;
#endif // PLATFORM_SUPPORTS_MESH_SHADERS
	} HashKey;

	FMemory::Memzero(&HashKey, sizeof(FHashKey));

	HashKey.VertexShader = Initializer.BoundShaderState.GetVertexShader() ? Initializer.BoundShaderState.GetVertexShader()->GetHash() : FShaderHash();
	HashKey.PixelShader = Initializer.BoundShaderState.GetPixelShader() ? Initializer.BoundShaderState.GetPixelShader()->GetHash() : FShaderHash();
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	HashKey.GeometryShader = Initializer.BoundShaderState.GetGeometryShader() ? Initializer.BoundShaderState.GetGeometryShader()->GetHash() : FShaderHash();
#endif
#if PLATFORM_SUPPORTS_MESH_SHADERS
	HashKey.MeshShader = Initializer.BoundShaderState.GetMeshShader() ? Initializer.BoundShaderState.GetMeshShader()->GetHash() : FShaderHash();
#endif

	uint64 PrecachePSOHash = CityHash64((const char*)&HashKey, sizeof(FHashKey));
	return PrecachePSOHash;
}

uint64 FOpenGLDynamicRHI::RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer)
{
	uint64 StatePrecachePSOHash = Initializer.StatePrecachePSOHash;
	if (StatePrecachePSOHash == 0)
	{
		StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(Initializer);
	}

	return StatePrecachePSOHash;
}

bool FOpenGLDynamicRHI::RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS)
{
	// check the RHI shaders (pointer check for shaders should be fine)
	if (LHS.BoundShaderState.VertexShaderRHI != RHS.BoundShaderState.VertexShaderRHI ||
		LHS.BoundShaderState.PixelShaderRHI != RHS.BoundShaderState.PixelShaderRHI ||
		LHS.BoundShaderState.GetMeshShader() != RHS.BoundShaderState.GetMeshShader() ||
		LHS.BoundShaderState.GetAmplificationShader() != RHS.BoundShaderState.GetAmplificationShader() ||
		LHS.BoundShaderState.GetGeometryShader() != RHS.BoundShaderState.GetGeometryShader())
	{
		return false;
	}

	return true;
}

FOpenGLDynamicRHI::FGLViewportContainer FOpenGLDynamicRHI::GetViewportContainer()
{
	return FGLViewportContainer(Viewports);
}

void FOpenGLDynamicRHI::InitializeStateResources()
{
	VERIFY_GL_SCOPE();
	ContextState.InitializeResources(FOpenGL::GetMaxCombinedTextureImageUnits(), FOpenGL::GetMaxCombinedUAVUnits());
	PendingState.InitializeResources(FOpenGL::GetMaxCombinedTextureImageUnits(), FOpenGL::GetMaxCombinedUAVUnits());
}

GLint FOpenGLBase::MaxTextureImageUnits = -1;
GLint FOpenGLBase::MaxCombinedTextureImageUnits = -1;
GLint FOpenGLBase::MaxComputeTextureImageUnits = -1;
GLint FOpenGLBase::MaxVertexTextureImageUnits = -1;
GLint FOpenGLBase::MaxGeometryTextureImageUnits = -1;
GLint FOpenGLBase::MaxVaryingVectors = -1;
GLint FOpenGLBase::TextureBufferAlignment = -1;
GLint FOpenGLBase::MaxVertexUniformComponents = -1;
GLint FOpenGLBase::MaxPixelUniformComponents = -1;
GLint FOpenGLBase::MaxGeometryUniformComponents = -1;
bool  FOpenGLBase::bSupportsClipControl = false;
bool  FOpenGLBase::bSupportsASTC = false;
bool  FOpenGLBase::bSupportsASTCHDR = false;
bool  FOpenGLBase::bSupportsSeamlessCubemap = false;
bool  FOpenGLBase::bSupportsVolumeTextureRendering = false;
bool  FOpenGLBase::bSupportsTextureFilterAnisotropic = false;
bool  FOpenGLBase::bSupportsDrawBuffersBlend = false;
bool  FOpenGLBase::bAmdWorkaround = false;

void FOpenGLBase::ProcessQueryGLInt()
{
	LOG_AND_GET_GL_INT(GL_MAX_TEXTURE_IMAGE_UNITS, 0, MaxTextureImageUnits);
	LOG_AND_GET_GL_INT(GL_MAX_VERTEX_TEXTURE_IMAGE_UNITS, 0, MaxVertexTextureImageUnits);
	LOG_AND_GET_GL_INT(GL_MAX_COMPUTE_TEXTURE_IMAGE_UNITS, 0, MaxComputeTextureImageUnits);
	GET_GL_INT(GL_MAX_COMBINED_TEXTURE_IMAGE_UNITS, 0, MaxCombinedTextureImageUnits);
}

void FOpenGLBase::ProcessExtensions( const FString& ExtensionsString )
{
	ProcessQueryGLInt();

	auto CheckAndSetImageUnits = [](GLint& StageImageUnitsINOUT, GLint Limit, const TCHAR* Msg) 
	{
		const bool bUnsupported = StageImageUnitsINOUT < Limit;
		UE_CLOGF(bUnsupported, LogRHI, Error, "GL RHI requires a minimum %ls texture unit count of %d, this device reports %d.", Msg, Limit, StageImageUnitsINOUT);
		check(!bUnsupported);
		StageImageUnitsINOUT = Limit;
	};

	static const GLint GLESMaxImageUnitsPerStage = 16; // gles 3 spec is a minimum of 16 per stage. 
	static const GLint MaxCombinedImageUnits = 48;

	if (IsMobilePlatform(GMaxRHIShaderPlatform))
	{
		// clamp things to the levels that the spec is expecting, check the minimum is supported.
		CheckAndSetImageUnits(MaxTextureImageUnits, GLESMaxImageUnitsPerStage, TEXT("pixel stage"));
		CheckAndSetImageUnits(MaxVertexTextureImageUnits, GLESMaxImageUnitsPerStage, TEXT("vertex stage"));
		CheckAndSetImageUnits(MaxGeometryTextureImageUnits, 0, TEXT("geometry stage")); // gles is not expecting this.
		CheckAndSetImageUnits(MaxComputeTextureImageUnits, GLESMaxImageUnitsPerStage, TEXT("compute stage"));
		CheckAndSetImageUnits(MaxCombinedTextureImageUnits, MaxCombinedImageUnits, TEXT("combined"));
	}
	else
	{
		UE_CLOG(MaxCombinedTextureImageUnits<MaxCombinedImageUnits, LogRHI, Fatal, TEXT("GL RHI requires a minimum combined texture unit count of %d, this device reports %d."), MaxCombinedImageUnits, MaxCombinedTextureImageUnits);
	}

	// Check for support for advanced texture compression (desktop and mobile)
	bSupportsASTC = ExtensionsString.Contains(TEXT("GL_KHR_texture_compression_astc_ldr"));

	bSupportsASTCHDR = bSupportsASTC && ExtensionsString.Contains(TEXT("GL_KHR_texture_compression_astc_hdr"));

	bSupportsSeamlessCubemap = ExtensionsString.Contains(TEXT("GL_ARB_seamless_cube_map"));
	
	bSupportsTextureFilterAnisotropic = ExtensionsString.Contains(TEXT("GL_EXT_texture_filter_anisotropic"));

	bSupportsDrawBuffersBlend = ExtensionsString.Contains(TEXT("GL_ARB_draw_buffers_blend"));

#if PLATFORM_IOS
	GRHIVendorId = 0x1010;
#else
	FString VendorName( ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_VENDOR) ) );
	if (VendorName.Contains(TEXT("ATI ")))
	{
		GRHIVendorId = 0x1002;
#if PLATFORM_WINDOWS || PLATFORM_LINUX
		bAmdWorkaround = true;
#endif
	}
#if PLATFORM_LINUX
	else if (VendorName.Contains(TEXT("X.Org")))
	{
		GRHIVendorId = 0x1002;
		bAmdWorkaround = true;
	}
#endif
	else if (VendorName.Contains(TEXT("Intel ")) || VendorName == TEXT("Intel"))
	{
		GRHIVendorId = 0x8086;
#if PLATFORM_WINDOWS || PLATFORM_LINUX
		bAmdWorkaround = true;
#endif
	}
	else if (VendorName.Contains(TEXT("NVIDIA ")))
	{
		GRHIVendorId = 0x10DE;
	}
	else if (VendorName.Contains(TEXT("ImgTec")) || VendorName.Contains(TEXT("Imagination")))
	{
		GRHIVendorId = 0x1010;
	}
	else if (VendorName.Contains(TEXT("ARM")))
	{
		GRHIVendorId = 0x13B5;
	}
	else if (VendorName.Contains(TEXT("Qualcomm")))
	{
		GRHIVendorId = 0x5143;
	}

#if PLATFORM_LINUX
	if (GRHIVendorId == 0x0)
	{
		// Try harder for Mesa
		const ANSICHAR* AnsiVersion = (const ANSICHAR*)glGetString(GL_VERSION);
		const ANSICHAR* AnsiRenderer = (const ANSICHAR*)glGetString(GL_RENDERER);
		if (AnsiVersion && AnsiRenderer)
		{
			if (FCStringAnsi::Strstr(AnsiVersion, "Mesa"))
			{
				if (FCStringAnsi::Strstr(AnsiRenderer, "AMD") || FCStringAnsi::Strstr(AnsiRenderer, "ATI"))
				{
					// Radeon
					GRHIVendorId = 0x1002;
					bAmdWorkaround = true;
				}
				else if (FCStringAnsi::Strstr(AnsiRenderer, "Intel"))
				{
					GRHIVendorId = 0x8086;
					bAmdWorkaround = true;
				}
			}
		}

		// If still not detected, show a message box to the user (editor build only) and
		// set GRHIVendorId to something to avoid crashing in check()s later
		if (GRHIVendorId == 0x0)
		{
			if (WITH_EDITOR != 0 && !IsRunningCommandlet() && !FApp::IsUnattended())
			{
				FString GlRenderer(ANSI_TO_TCHAR(AnsiRenderer));
				FText ErrorMessage = FText::Format(LOCTEXT("CannotDetermineGraphicsDriversVendor", "Unknown graphics drivers '{0}' by '{1}' are installed on this system. You may experience visual artifacts and other problems."),
					FText::FromString(GlRenderer), FText::FromString(VendorName));
				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage.ToString(),
					*LOCTEXT("CannotDetermineGraphicsDriversVendorTitle", "Cannot determine driver vendor.").ToString());
			}

			GRHIVendorId = 0xFFFF;
			bAmdWorkaround = true;	// be conservative here as well.
		}
	}
#endif // PLATFORM_LINUX

#if PLATFORM_WINDOWS
	auto* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("OpenGL.UseStagingBuffer"));
	if (CVar)
	{
		CVar->Set(false);
	}
#endif
#endif // !PLATFORM_IOS

	// Setup CVars that require the RHI initialized
}

void FOpenGLBase::PE_GetCurrentOpenGLShaderDeviceCapabilities(FOpenGLShaderDeviceCapabilities& Capabilities)
{
	Capabilities.TargetPlatform = EOpenGLShaderTargetPlatform::OGLSTP_Unknown;
}

void GetExtensionsString( FString& ExtensionsString)
{
	GLint ExtensionCount = 0;
	ExtensionsString = TEXT("");
	if ( FOpenGL::SupportsIndexedExtensions() )
	{
		glGetIntegerv(GL_NUM_EXTENSIONS, &ExtensionCount);
		for (int32 ExtensionIndex = 0; ExtensionIndex < ExtensionCount; ++ExtensionIndex)
		{
			const ANSICHAR* ExtensionString = FOpenGL::GetStringIndexed(GL_EXTENSIONS, ExtensionIndex);

			ExtensionsString += TEXT(" ");
			ExtensionsString += ANSI_TO_TCHAR(ExtensionString);
		}
	}
	else
	{
		const ANSICHAR* GlGetStringOutput = (const ANSICHAR*) glGetString( GL_EXTENSIONS );
		if (GlGetStringOutput)
		{
			ExtensionsString += GlGetStringOutput;
			ExtensionsString += TEXT(" ");
		}
	}
}

namespace OpenGLConsoleVariables
{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
	int32 bUseGlClipControlIfAvailable = 1;
#else
	int32 bUseGlClipControlIfAvailable = 0;
#endif
	static FAutoConsoleVariableRef CVarUseGlClipControlIfAvailable(
		TEXT("OpenGL.UseGlClipControlIfAvailable"),
		bUseGlClipControlIfAvailable,
		TEXT("If true, the engine trys to use glClipControl if the driver supports it."),
		ECVF_RenderThreadSafe | ECVF_ReadOnly
	);
}

void InitDefaultGLContextState(void)
{
	// NOTE: This function can be called before capabilities setup, so extensions need to be checked directly
	FString ExtensionsString;
	GetExtensionsString(ExtensionsString);

	// Intel HD4000 under <= 10.8.4 requires GL_DITHER disabled or dithering will occur on any channel < 8bits.
	// No other driver does this but we don't need GL_DITHER on anyway.
	glDisable(GL_DITHER);

	if (FOpenGL::SupportsFramebufferSRGBEnable())
	{
		// Render targets with TexCreate_SRGB should do sRGB conversion like in D3D11
		glEnable(GL_FRAMEBUFFER_SRGB);
	}

	// Engine always expects seamless cubemap, so enable it if available
	if (ExtensionsString.Contains(TEXT("GL_ARB_seamless_cube_map")))
	{
		glEnable(GL_TEXTURE_CUBE_MAP_SEAMLESS);
	}

#if PLATFORM_WINDOWS || PLATFORM_LINUX
	if (OpenGLConsoleVariables::bUseGlClipControlIfAvailable && ExtensionsString.Contains(TEXT("GL_ARB_clip_control")) && !FOpenGL::IsAndroidGLESCompatibilityModeEnabled())
	{
		FOpenGL::EnableSupportsClipControl();
		glClipControl(GL_UPPER_LEFT, GL_ZERO_TO_ONE);
	}
#endif

	// optional per platform setup
	FOpenGL::SetupDefaultGLContextState(ExtensionsString);
}

void FOpenGLDynamicRHI::RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos)
{
	RHICmdList.EnqueueLambda(TEXT("FOpenGLDynamicRHI::RHIReplaceResources"),
		[ReplaceInfos = MoveTemp(ReplaceInfos)](FRHICommandListBase&)
		{
			for (FRHIResourceReplaceInfo const& Info : ReplaceInfos)
			{
				switch (Info.GetType())
				{
				default:
					checkNoEntry();
					break;

				case FRHIResourceReplaceInfo::EType::Buffer:
					{
						FOpenGLBuffer* Dst = ResourceCast(Info.GetBuffer().Dst);
						FOpenGLBuffer* Src = ResourceCast(Info.GetBuffer().Src);

						if (Src)
						{
							// The source buffer should not have any associated views.
							check(!Src->HasLinkedViews());

							Dst->TakeOwnership(*Src);
						}
						else
						{
							Dst->ReleaseOwnership();
						}

						Dst->UpdateLinkedViews();
					}
					break;
				}
			}
		}
	);

	RHICmdList.RHIThreadFence(true);
}

#undef LOCTEXT_NAMESPACE
