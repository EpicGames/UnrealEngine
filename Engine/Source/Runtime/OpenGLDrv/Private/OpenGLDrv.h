// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLDrv.h: Public OpenGL RHI definitions.
=============================================================================*/

#pragma once

// Dependencies.

#include "CoreMinimal.h"
#include "Containers/IndirectArray.h"
#include "IOpenGLDynamicRHI.h"
#include "RHIDefinitions.h"
#include "RHI.h"
#include "GPUProfiler.h"
#include "RenderResource.h"
#include "Templates/EnableIf.h"
#include "BoundShaderStateHistory.h"

#include "OpenGLState.h"
#include "OpenGLPlatform.h"
#include "OpenGLUtil.h"
#include "RenderUtils.h"

#define UE_API OPENGLDRV_API

// Define here so don't have to do platform filtering
#ifndef GL_TEXTURE_EXTERNAL_OES
#define GL_TEXTURE_EXTERNAL_OES 0x8D65
#endif

#define FOpenGLCachedUniformBuffer_Invalid 0xFFFFFFFF

template<class T> struct TOpenGLResourceTraits;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/** The interface which is implemented by the dynamically bound RHI. */
class FOpenGLDynamicRHI final : public IOpenGLDynamicRHI, public IRHICommandContextPSOFallback
{
	static inline FOpenGLDynamicRHI* Singleton = nullptr;

public:
	static inline FOpenGLDynamicRHI& Get() { return *Singleton; }

	friend class FOpenGLViewport;

	/** Initialization constructor. */
	UE_API FOpenGLDynamicRHI();

	/** Destructor */
	~FOpenGLDynamicRHI() = default;

	// IOpenGLDynamicRHI interface.
	UE_API virtual int32 RHIGetGLMajorVersion() const final override;
	UE_API virtual int32 RHIGetGLMinorVersion() const final override;
	UE_API virtual bool RHISupportsFramebufferSRGBEnable() const final override;
	UE_API virtual FTextureRHIRef RHICreateTexture2DFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags Flags) final override;
#if PLATFORM_ANDROID
	UE_API virtual FTextureRHIRef RHICreateTexture2DFromAndroidHardwareBuffer(FRHICommandListBase& RHICmdList, AHardwareBuffer* HardwareBuffer) override;
#endif //PLATFORM_ANDROID

	UE_API virtual FTextureRHIRef RHICreateTexture2DArrayFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags Flags) final override;
	UE_API virtual FTextureRHIRef RHICreateTextureCubeFromResource(EPixelFormat Format, uint32 Size, bool bArray, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags Flags) final override;
	UE_API virtual GLuint RHIGetResource(FRHITexture* InTexture) const final override;
	UE_API virtual bool RHIIsValidTexture(GLuint InTexture) const final override;
	UE_API virtual void RHISetExternalGPUTime(uint64 InExternalGPUTime) final override;

#if PLATFORM_ANDROID
	UE_API virtual EGLDisplay RHIGetEGLDisplay() const final override;
	UE_API virtual EGLSurface RHIGetEGLSurface() const final override;
	UE_API virtual EGLConfig  RHIGetEGLConfig() const final override;
	UE_API virtual EGLContext RHIGetEGLContext() const final override;
	UE_API virtual ANativeWindow* RHIGetEGLNativeWindow() const final override;
	UE_API virtual bool RHIEGLSupportsNoErrorContext() const final override;
	UE_API virtual void RHIInitEGLInstanceGLES2() final override;
	UE_API virtual void RHIInitEGLBackBuffer() final override;
	UE_API virtual void RHIEGLSetCurrentRenderingContext() final override;
	UE_API virtual void RHIEGLTerminateContext() final override;
#endif

	// FDynamicRHI interface.
	UE_API virtual void Init() override;

	UE_API virtual void Shutdown() override;
	virtual const TCHAR* GetName() override { return TEXT("OpenGL"); }

	template<typename TRHIType>
	static auto* ResourceCast(TRHIType* Resource)
	{
		return static_cast<typename TOpenGLResourceTraits<TRHIType>::TConcreteType*>(Resource);
	}

	static FOpenGLTexture* ResourceCast(FRHITexture* TextureRHI)
	{
		if (!TextureRHI)
		{
			return nullptr;
		}
		else
		{
			return static_cast<FOpenGLTexture*>(TextureRHI->GetTextureBaseRHI());
		}
	}

	UE_API void BindUniformBuffer(EShaderFrequency ShaderFrequency, uint32 BufferIndex, FRHIUniformBuffer* BufferRHI);

	UE_API void SetShaderParametersCommon(EShaderFrequency ShaderFrequency, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters);
	UE_API void SetShaderUnbindsCommon(EShaderFrequency ShaderFrequency, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds);

	UE_API virtual FRasterizerStateRHIRef RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer) final override;
	UE_API virtual FDepthStencilStateRHIRef RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer) final override;
	UE_API virtual FBlendStateRHIRef RHICreateBlendState(const FBlendStateInitializerRHI& Initializer) final override;
	UE_API virtual FVertexDeclarationRHIRef RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements) final override;

	UE_API virtual FPixelShaderRHIRef RHICreatePixelShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	UE_API virtual FVertexShaderRHIRef RHICreateVertexShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	UE_API virtual FGeometryShaderRHIRef RHICreateGeometryShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;
	UE_API virtual FComputeShaderRHIRef RHICreateComputeShader(const FRHICreateShaderDesc& CreateShaderDesc) final override;

	// SRV / UAV creation functions
	UE_API virtual FShaderResourceViewRHIRef  RHICreateShaderResourceView (class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;
	UE_API virtual FUnorderedAccessViewRHIRef RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc) final override;

	UE_API virtual FUniformBufferRHIRef RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation) final override;
	UE_API virtual void RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents) final override;

	[[nodiscard]] UE_API virtual FRHIBufferInitializer RHICreateBufferInitializer(FRHICommandListBase& RHICmdList, const FRHIBufferCreateDesc& CreateDesc) final override;

	UE_API virtual void RHIReplaceResources(FRHICommandListBase& RHICmdList, TArray<FRHIResourceReplaceInfo>&& ReplaceInfos) final override;
	UE_API virtual void* LockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer, uint32 Offset, uint32 Size, EResourceLockMode LockMode) final override;
	UE_API virtual void UnlockBuffer_BottomOfPipe(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#if ENABLE_LOW_LEVEL_MEM_TRACKER || UE_MEMORY_TRACE_ENABLED
	UE_API virtual void RHIUpdateAllocationTags(FRHICommandListBase& RHICmdList, FRHIBuffer* Buffer) final override;
#endif
	UE_API virtual FRHICalcTextureSizeResult RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex) final override;
	UE_API virtual void RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats) final override;
	UE_API virtual bool RHIGetTextureMemoryVisualizeData(FColor* TextureData,int32 SizeX,int32 SizeY,int32 Pitch,int32 PixelSize) final override;
	[[nodiscard]] UE_API virtual FRHITextureInitializer RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc) final override;
	UE_API virtual FTextureRHIRef RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent) final override;
	UE_API virtual void RHIGenerateMips(FRHITexture* Texture) final override;
	UE_API virtual uint32 RHIComputeMemorySize(FRHITexture* TextureRHI) final override;

	UE_API virtual FRHILockTextureResult RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;
	UE_API virtual void RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments) final override;

	UE_API virtual void RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData) final override;
	UE_API virtual void RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* Texture, uint32 MipIndex, const struct FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData) final override;
	UE_API virtual void RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* Texture, const TCHAR* Name) final override;
	UE_API virtual void RHIReadSurfaceData(FRHITexture* Texture,FIntRect Rect,TArray<FColor>& OutData,FReadSurfaceDataFlags InFlags) final override;
	UE_API virtual void RHIReadSurfaceData(FRHITexture* Texture, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags) final override;
	UE_API virtual void RHIMapStagingSurface(FRHITexture* Texture, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex = 0) final override;
	UE_API virtual void RHIUnmapStagingSurface(FRHITexture* Texture, uint32 GPUIndex = 0) final override;
	UE_API virtual void RHIReadSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex) final override;
	UE_API virtual void RHIRead3DSurfaceFloatData(FRHITexture* Texture,FIntRect Rect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData) final override;
	UE_API virtual FRenderQueryRHIRef RHICreateRenderQuery(ERenderQueryType QueryType) final override;
	UE_API virtual bool RHIGetRenderQueryResult(FRHIRenderQuery* RenderQuery, uint64& OutResult, bool bWait, uint32 GPUIndex = INDEX_NONE) final override;
	UE_API virtual FTextureRHIRef RHIGetViewportBackBuffer(FRHIViewport* Viewport) final override;
	UE_API virtual void RHIAliasTextureResources(FTextureRHIRef& DestTextureRHI, FTextureRHIRef& SrcTextureRHI) final override;
	UE_API virtual FTextureRHIRef RHICreateAliasedTexture(FTextureRHIRef& SourceTexture) final override;
	UE_API virtual void RHIAcquireThreadOwnership() final override;
	UE_API virtual void RHIReleaseThreadOwnership() final override;
	UE_API virtual FViewportRHIRef RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	UE_API virtual void RHIResizeViewport(FRHIViewport* Viewport, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat) final override;
	UE_API virtual void RHITick(float DeltaTime) final override;
	UE_API virtual void RHIBlockUntilGPUIdle() final override;
	UE_API virtual bool RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate) final override;
	UE_API virtual void RHIGetSupportedResolution(uint32& Width, uint32& Height) final override;
	UE_API virtual void* RHIGetNativeDevice() final override;
	UE_API virtual class IRHICommandContext* RHIGetDefaultContext() final override;
	UE_API virtual IRHIComputeContext* RHIGetCommandContext(ERHIPipeline Pipeline) final override;
	UE_API virtual FRHIFinalizeContextsResult RHIFinalizeContexts(FRHIFinalizeContextsArgs&& Args) final override;
	UE_API virtual void RHISubmitCommandLists(FRHISubmitCommandListsArgs&& Args) final override;

	UE_API virtual void RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName) final override;
	UE_API virtual void RHIEndRenderPass() final override;
	UE_API virtual void RHINextSubpass() final override;
	
	UE_API virtual void RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions) override final;
	UE_API virtual void RHIEndTransitions(TArrayView<const FRHITransition*> Transitions) override final;

	UE_API virtual void RHISetComputeShader(FRHIComputeShader* ComputeShader) final override;
	UE_API virtual void RHIDispatchComputeShader(uint32 ThreadGroupCountX, uint32 ThreadGroupCountY, uint32 ThreadGroupCountZ) final override;
	UE_API virtual void RHIDispatchIndirectComputeShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	UE_API virtual void RHISetMultipleViewports(uint32 Count, const FViewportBounds* Data) final override;
	UE_API virtual void RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values) final override;
	UE_API virtual void RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values) final override;

	UE_API virtual void RHIBeginRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override final;
	UE_API virtual void RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery) override final;
	UE_API virtual void RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery) final override;
	UE_API virtual void RHIEndRenderQuery(FRHIRenderQuery* RenderQuery) final override;

	UE_API virtual void RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* Viewport, const FRHIPresentArgs& PresentArgs) final override;
	UE_API virtual void RHIEndFrame(const FRHIEndFrameArgs& Args) final override;
	UE_API virtual void RHISetStreamSource(uint32 StreamIndex, FRHIBuffer* VertexBuffer, uint32 Offset) final override;
	UE_API virtual void RHISetRasterizerState(FRHIRasterizerState* NewState) final override;
	UE_API virtual void RHISetViewport(float MinX, float MinY, float MinZ, float MaxX, float MaxY, float MaxZ) final override;
	UE_API virtual void RHISetScissorRect(bool bEnable, uint32 MinX, uint32 MinY, uint32 MaxX, uint32 MaxY) final override;
	UE_API virtual void RHISetBoundShaderState(FRHIBoundShaderState* BoundShaderState) final override;
	UE_API virtual void RHISetStaticUniformBuffers(const FUniformBufferStaticBindings& InUniformBuffers) final override;
	UE_API virtual void RHISetStaticUniformBuffer(FUniformBufferStaticSlot Slot, FRHIUniformBuffer* Buffer) final override;
	UE_API virtual void RHISetUniformBufferDynamicOffset(FUniformBufferStaticSlot Slot, uint32 Offset) final override;
	UE_API virtual void RHISetShaderParameters(FRHIGraphicsShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	UE_API virtual void RHISetShaderParameters(FRHIComputeShader* Shader, TConstArrayView<uint8> InParametersData, TConstArrayView<FRHIShaderParameter> InParameters, TConstArrayView<FRHIShaderParameterResource> InResourceParameters, TConstArrayView<FRHIShaderParameterResource> InBindlessParameters) final override;
	UE_API virtual void RHISetShaderUnbinds(FRHIComputeShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override;
	UE_API virtual void RHISetShaderUnbinds(FRHIGraphicsShader* Shader, TConstArrayView<FRHIShaderParameterUnbind> InUnbinds) final override;
	UE_API virtual void RHISetDepthStencilState(FRHIDepthStencilState* NewState, uint32 StencilRef) final override;
	UE_API virtual void RHISetStencilRef(uint32 StencilRef) final override;
	UE_API virtual void RHISetBlendState(FRHIBlendState* NewState, const FLinearColor& BlendFactor) final override;
	virtual void RHISetBlendFactor(const FLinearColor& BlendFactor) final override
	{
		// Currently ignored as well as on RHISetBlendState()...
	}

	UE_API void SetRenderTargets(uint32 NumSimultaneousRenderTargets, const FRHIRenderTargetView* NewRenderTargets, const FRHIDepthRenderTargetView* NewDepthStencilTarget);
	UE_API void SetRenderTargetsAndClear(const FRHISetRenderTargetsInfo& RenderTargetsInfo);

	UE_API virtual void RHIDrawPrimitive(uint32 BaseVertexIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	UE_API virtual void RHIDrawPrimitiveIndirect(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	UE_API virtual void RHIDrawIndexedPrimitive(FRHIBuffer* IndexBuffer, int32 BaseVertexIndex, uint32 FirstInstance, uint32 NumVertices, uint32 StartIndex, uint32 NumPrimitives, uint32 NumInstances) final override;
	UE_API virtual void RHIDrawIndexedPrimitiveIndirect(FRHIBuffer* IndexBuffer, FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset) final override;
	UE_API virtual void RHIEnableDepthBoundsTest(bool bEnable) final override;
	UE_API virtual void RHISetDepthBounds(float MinDepth, float MaxDepth) final override;
#if WITH_RHI_BREADCRUMBS
	UE_API virtual void RHIBeginBreadcrumbGPU(FRHIBreadcrumbNode* Breadcrumb) final override;
	UE_API virtual void RHIEndBreadcrumbGPU  (FRHIBreadcrumbNode* Breadcrumb) final override;
#endif
	void DiscardRenderTargets(bool Depth, bool Stencil, uint32 ColorBitMask);

	// FIXME: Broken on Android for cubemaps
	UE_API virtual void RHICopyTexture(FRHITexture* SourceTexture, FRHITexture* DestTexture, const FRHICopyTextureInfo& CopyInfo) final override;

	UE_API virtual void RHICopyBufferRegion(FRHIBuffer* DestBuffer, uint64 DstOffset, FRHIBuffer* SourceBuffer, uint64 SrcOffset, uint64 NumBytes) final override;

	// Inline copy
	UE_API virtual void RHICopyToStagingBuffer(FRHIBuffer* SourceBufferRHI, FRHIStagingBuffer* DestinationStagingBufferRHI, uint32 InOffset, uint32 InNumBytes) final override;
	UE_API virtual void RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI) final override;
	UE_API virtual void RHIWriteGPUFence(FRHIGPUFence* FenceRHI) final override;
	UE_API virtual void* RHILockStagingBuffer(FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) final override;
	UE_API virtual void RHIUnlockStagingBuffer(FRHIStagingBuffer* StagingBuffer) final override;
	UE_API virtual void* LockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer, FRHIGPUFence* Fence, uint32 Offset, uint32 SizeRHI) final override;
	UE_API virtual void UnlockStagingBuffer_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHIStagingBuffer* StagingBuffer) final override;
	UE_API virtual void RHIMapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 GPUIndex, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight) final override;
	UE_API virtual void RHIUnmapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, uint32 GPUIndex) final override;

	UE_API virtual FStagingBufferRHIRef RHICreateStagingBuffer() final override;
	UE_API virtual FGPUFenceRHIRef RHICreateGPUFence(const FName &Name) final override;

	UE_API virtual uint64 RHIGetMinimumAlignmentForBufferBackedSRV(EPixelFormat Format) final override;

	// Compute the hash of the state components of the PSO initializer for PSO Precaching (only hash data relevant for the RHI specific PSO)
	UE_API virtual uint64 RHIComputeStatePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;

	// Compute the hash of the PSO initializer for PSO Precaching (only hash data relevant for the RHI specific PSO)
	UE_API virtual uint64 RHIComputePrecachePSOHash(const FGraphicsPipelineStateInitializer& Initializer) final override;

	// Check if PSO Initializers are the same used during PSO Precaching (only compare data relevant for the RHI specific PSO)
	UE_API virtual bool RHIMatchPrecachePSOInitializers(const FGraphicsPipelineStateInitializer& LHS, const FGraphicsPipelineStateInitializer& RHS) final override;

	UE_API void Cleanup();

	UE_API void PurgeFramebufferFromCaches(GLuint Framebuffer);
	UE_API void OnBufferDeletion(GLenum Type, GLuint BufferResource);
	UE_API void OnProgramDeletion(GLint ProgramResource);
	UE_API void InvalidateTextureResourceInCache(GLuint Resource);
	UE_API void InvalidateUAVResourceInCache(GLuint Resource);

	/** Set a resource on texture target of a specific real OpenGL stage. Goes through cache to eliminate redundant calls. */
	FORCEINLINE void CachedSetupTextureStage(GLint TextureIndex, GLenum Target, GLuint Resource, GLint BaseMip, GLint NumMips)
	{
		FTextureStage& TextureState = ContextState.Textures[TextureIndex];
		const bool bSameTarget = (TextureState.Target == Target);
		const bool bSameResource = (TextureState.Resource == Resource);
		const bool bSameMipLimits = TextureState.LimitMip == BaseMip && TextureState.NumMips == NumMips;

		// if there's no currently bound texture target there can be no mip properties to change.
		if (bSameTarget && bSameResource && (Target == GL_NONE || bSameMipLimits))
		{
			// Nothing changed, no need to update
			return;
		}

		CachedSetupTextureStageInner(TextureIndex, Target, Resource, BaseMip, NumMips);
	}

	UE_API void CachedSetupTextureStageInner(GLint TextureIndex, GLenum Target, GLuint Resource, GLint BaseMip, GLint NumMips);
	UE_API void CachedSetupUAVStage(GLint UAVIndex, GLenum Format, GLuint Resource, bool bLayered, GLint Layer, GLenum Access, GLint Level);
	UE_API void UpdateSRV(FOpenGLShaderResourceView* SRV);

	void CachedBindUniformBuffer(GLuint Buffer)
	{
		check(FOpenGL::SupportsUniformBuffers());

		VERIFY_GL_SCOPE();
		if (ContextState.UniformBufferBound != Buffer)
		{
			glBindBuffer(GL_UNIFORM_BUFFER, Buffer);
			ContextState.UniformBufferBound = Buffer;
		}
	}

	void CachedBindBuffer(GLenum Type, GLuint Buffer)
	{
		VERIFY_GL_SCOPE();
		switch (Type)
		{
		case GL_ARRAY_BUFFER:
			if (ContextState.ArrayBufferBound != Buffer)
			{
				glBindBuffer(GL_ARRAY_BUFFER, Buffer);
				ContextState.ArrayBufferBound = Buffer;
			}
			break;

		case GL_ELEMENT_ARRAY_BUFFER:
			if (ContextState.ElementArrayBufferBound != Buffer)
			{
				glBindBuffer(GL_ELEMENT_ARRAY_BUFFER, Buffer);
				ContextState.ElementArrayBufferBound = Buffer;
			}
			break;

		case GL_SHADER_STORAGE_BUFFER:
			if (ContextState.StorageBufferBound != Buffer)
			{
				glBindBuffer(GL_SHADER_STORAGE_BUFFER, Buffer);
				ContextState.StorageBufferBound = Buffer;
			}
			break;

		case GL_PIXEL_UNPACK_BUFFER:
			if (ContextState.PixelUnpackBufferBound != Buffer)
			{
				glBindBuffer(GL_PIXEL_UNPACK_BUFFER, Buffer);
				ContextState.PixelUnpackBufferBound = Buffer;
			}
			break;

		default:
			checkNoEntry();
			break;
		}
	}

	FOpenGLSamplerState* GetPointSamplerState() const { return (FOpenGLSamplerState*)PointSamplerState.GetReference(); }

	UE_API void InitializeGLTexture(FOpenGLTexture* Texture, const void* BulkDataPtr, uint64 BulkDataSize);

	UE_API void SetCustomPresent(class FRHICustomPresent* InCustomPresent);

#define RHITHREAD_GLTRACE 1
#if RHITHREAD_GLTRACE 
	#define RHITHREAD_GLTRACE_BLOCKING QUICK_SCOPE_CYCLE_COUNTER(STAT_OGLRHIThread_Flush);
//#define RHITHREAD_GLTRACE_BLOCKING FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FLUSHING %s!\n"), ANSI_TO_TCHAR(__FUNCTION__))
//#define RHITHREAD_GLTRACE_BLOCKING UE_LOGF(LogRHI, Warning,"FLUSHING %ls!\n", ANSI_TO_TCHAR(__FUNCTION__));
#else
	#define RHITHREAD_GLTRACE_BLOCKING 
#endif

	struct FTextureLockTracker
	{
		struct FLockParams
		{
			void* RHIBuffer;
			void* Buffer;
			uint32 MipIndex;
			uint32 ArrayIndex;
			uint32 BufferSize;
			uint32 Stride;
			EResourceLockMode LockMode;

			FORCEINLINE_DEBUGGABLE FLockParams(void* InRHIBuffer, void* InBuffer, uint32 InArrayIndex, uint32 InMipIndex, uint32 InStride, uint32 InBufferSize, EResourceLockMode InLockMode)
				: RHIBuffer(InRHIBuffer)
				, Buffer(InBuffer)
				, MipIndex(InMipIndex)
				, ArrayIndex(InArrayIndex)
				, BufferSize(InBufferSize)
				, Stride(InStride)
				, LockMode(InLockMode)
			{
			}
		};
		TArray<FLockParams, TInlineAllocator<16> > OutstandingLocks;

		FTextureLockTracker()
		{}

		FORCEINLINE_DEBUGGABLE void Lock(void* RHIBuffer, void* Buffer, uint32 ArrayIndex, uint32 MipIndex, uint32 Stride, uint32 SizeRHI, EResourceLockMode LockMode)
		{
//#if DO_CHECK
			for (auto& Parms : OutstandingLocks)
			{
				check(Parms.RHIBuffer != RHIBuffer || Parms.MipIndex != MipIndex || Parms.ArrayIndex != ArrayIndex);
			}
//#endif
			OutstandingLocks.Add(FLockParams(RHIBuffer, Buffer, ArrayIndex, MipIndex, Stride, SizeRHI, LockMode));
		}

		FORCEINLINE_DEBUGGABLE FLockParams Unlock(void* RHIBuffer, uint32 ArrayIndex, uint32 MipIndex)
		{
			for (int32 Index = 0; Index < OutstandingLocks.Num(); Index++)
			{
				FLockParams& CurrentLock = OutstandingLocks[Index];
				if (CurrentLock.RHIBuffer == RHIBuffer && CurrentLock.MipIndex == MipIndex && CurrentLock.ArrayIndex == ArrayIndex)
				{
					FLockParams Result = OutstandingLocks[Index];
					OutstandingLocks.RemoveAtSwap(Index, EAllowShrinking::No);
					return Result;
				}
			}
			check(!"Mismatched RHI buffer locks.");
			return FLockParams(nullptr, nullptr, 0, 0, 0, 0, RLM_WriteOnly);
		}
	};

	UE_API virtual FSamplerStateRHIRef RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer) final override;

	virtual FGraphicsPipelineStateRHIRef RHICreateGraphicsPipelineState(const FGraphicsPipelineStateInitializer& Initializer) override
	{
		PrepareGFXBoundShaderState(Initializer);

		return new FRHIGraphicsPipelineStateFallBack(Initializer);
	}

	UE_API virtual void RHISetGraphicsPipelineState(FRHIGraphicsPipelineState* GraphicsState, uint32 StencilRef, bool bApplyAdditionalState) final override;

	virtual FBoundShaderStateRHIRef RHICreateBoundShaderState(
		FRHIVertexDeclaration* VertexDeclarationRHI,
		FRHIVertexShader* VertexShaderRHI,
		FRHIPixelShader* PixelShaderRHI,
		FRHIGeometryShader* GeometryShaderRHI
	) final override
	{
		checkNoEntry();
		return nullptr;
	}

	UE_API void LinkComputeProgram(FRHIComputeShader* ComputeShaderRHI);


	UE_API virtual void RHIPostExternalCommandsReset() final override;

	UE_API GLuint GetOpenGLFramebuffer(uint32 NumSimultaneousRenderTargets, FOpenGLTexture** RenderTargets, const uint32* ArrayIndices, const uint32* MipmapLevels, FOpenGLTexture* DepthStencilTarget);
	UE_API GLuint GetOpenGLFramebuffer(uint32 NumSimultaneousRenderTargets, FOpenGLTexture** RenderTargets, const uint32* ArrayIndices, const uint32* MipmapLevels, FOpenGLTexture* DepthStencilTarget, FExclusiveDepthStencil DepthStencilAccess, int32 NumRenderingSamples);
	
	UE_API void ResolveTexture(FOpenGLTexture* Texture, uint32 MipIndex, uint32 ArrayIndex);

	class FGLViewportContainer;
 	FGLViewportContainer GetViewportContainer();

	class FGLViewportContainer
	{
		friend FGLViewportContainer FOpenGLDynamicRHI::GetViewportContainer();

		static inline UE::FMutex ViewportMutex;
		UE::TUniqueLock<UE::FMutex> ScopeLock;
		TArray<FOpenGLViewport*>& Viewports;

		FGLViewportContainer(TArray<FOpenGLViewport*>& ViewportsIn) : ScopeLock(ViewportMutex), Viewports(ViewportsIn)
		{
		}

	public:
		~FGLViewportContainer()
		{
		}

		TArray<FOpenGLViewport*>& Get()
		{
			return Viewports;
		}
	};

private:
	UE_API FBoundShaderStateRHIRef RHICreateBoundShaderState_Internal(FRHIVertexDeclaration* VertexDeclaration, FRHIVertexShader* VertexShader, FRHIPixelShader* PixelShader, FRHIGeometryShader* GeometryShader, bool FromPSOFileCache);

	UE_API void PrepareGFXBoundShaderState(const FGraphicsPipelineStateInitializer& Initializer);

	/** called once per frame, used for resource processing */
	UE_API void EndFrameTick();

	/** RHI device state, independent of underlying OpenGL context used */
	FOpenGLRHIState						PendingState;
	FSamplerStateRHIRef					PointSamplerState;

	/** A list of all viewport RHIs that have been created. */
	TArray<FOpenGLViewport*> Viewports;
	bool								bRevertToSharedContextAfterDrawingViewport = false;

	EPrimitiveType						PrimitiveType = PT_Num;

	/** A history of the most recently used bound shader states, used to keep transient bound shader states from being recreated for each use. */
	TGlobalResource< TBoundShaderStateHistory<10000> > BoundShaderStateHistory;

	FOpenGLContextState	ContextState;

	template <typename TRHIShader>
	void ApplyStaticUniformBuffers(TRHIShader* Shader);

	TArray<FRHIUniformBuffer*> GlobalUniformBuffers;

	/** Cached mip-limits for textures when ARB_texture_view is unavailable */
	TMap<GLuint, TPair<GLenum, GLenum>> TextureMipLimits;

	/** Underlying platform-specific data */
	struct FPlatformOpenGLDevice* PlatformDevice = nullptr;

	friend class FOpenGLRenderQuery;

	FOpenGLProfiler Profiler;
	
	void FlushProfilerStats()
	{
		// Flush accumulated draw stats
		if (Profiler.bEnabled && StatEvent)
		{
			Profiler.EmplaceEvent<UE::RHI::GPUProfiler::FEvent::FStats>() = StatEvent;
			StatEvent = {};
		}
	}

	FCriticalSection CustomPresentSection;
	TRefCountPtr<class FRHICustomPresent> CustomPresent;

	UE_API void InitializeStateResources();

	UE_API void SetupVertexArrays(uint32 BaseVertexIndex, FOpenGLStream* Streams, uint32 NumStreams, uint32 MaxVertices);

	UE_API void SetupDraw(FOpenGLBuffer* IndexBuffer, uint32 BaseVertexIndex, uint32 MaxVertices);
	UE_API void SetupDispatch();

	/** needs to be called before each draw call */
	UE_API void BindPendingFramebuffer();
	UE_API void BindPendingShaderState();
	UE_API void BindPendingComputeShaderState(FOpenGLComputeShader* ComputeShader);
	UE_API void UpdateRasterizerStateInOpenGLContext();
	UE_API void UpdateDepthStencilStateInOpenGLContext();
	UE_API void UpdateScissorRectInOpenGLContext();
	UE_API void UpdateViewportInOpenGLContext();
	
	template <class ShaderType> void SetResourcesFromTables(ShaderType* Shader);
	FORCEINLINE void CommitGraphicsResourceTables()
	{
		if (PendingState.bAnyDirtyGraphicsUniformBuffers)
		{
			CommitGraphicsResourceTablesInner();
		}
	}
	UE_API void CommitGraphicsResourceTablesInner();
	UE_API void CommitComputeResourceTables(FOpenGLComputeShader* ComputeShader);
	UE_API void CommitNonComputeShaderConstants();
	UE_API void CommitComputeShaderConstants(FOpenGLComputeShader* ComputeShader);
	UE_API void SetPendingBlendStateForActiveRenderTargets();
	
	template <typename StateType>
	void SetupTexturesForDraw(const StateType& ShaderState, int32 MaxTexturesNeeded);
	UE_API void SetupTexturesForDraw();

	UE_API void SetupUAVsForDraw();
	UE_API void SetupUAVsForCompute(const FOpenGLComputeShader* ComputeShader);
	UE_API void SetupUAVsForProgram(const TBitArray<>& NeededBits, int32 MaxUAVUnitUsed);

	UE_API void RHIClearMRT(const bool* bClearColorArray, int32 NumClearColors, const FLinearColor* ColorArray, bool bClearDepth, float Depth, bool bClearStencil, uint32 Stencil);

public:
	/** Remember what RHI user wants set on a specific OpenGL texture stage, translating from Stage and TextureIndex for stage pair. */
	UE_API void InternalSetShaderTexture(FOpenGLTexture* Texture, FOpenGLShaderResourceView* SRV, GLint TextureIndex, GLenum Target, GLuint Resource, int NumMips, int LimitMip);
	UE_API void InternalSetShaderImageUAV(GLint UAVIndex, GLenum Format, GLuint Resource, bool bLayered, GLint Layer, GLenum Access, GLint Level);
	UE_API void InternalSetShaderBufferUAV(GLint UAVIndex, GLuint Resource);
	UE_API void InternalSetSamplerStates(GLint TextureIndex, FOpenGLSamplerState* SamplerState);
	UE_API void InitializeGLTextureInternal(FOpenGLTexture* Texture, void const* BulkDataPtr, uint64 BulkDataSize);

	UE_API void ClearCachedAttributeState(int32 PositionAttrib, int32 TexCoordsAttrib);

private:

	UE_API void ApplyTextureStage(GLint TextureIndex, const FTextureStage& TextureStage, FOpenGLSamplerState* SamplerState);

	UE_API void ReadSurfaceDataRaw(FRHITexture* TextureRHI, FIntRect Rect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags);

	UE_API void BindUniformBufferBase(int32 NumUniformBuffers, FRHIUniformBuffer** BoundUniformBuffers, uint32* DynamicOffsets, uint32 FirstUniformBuffer, bool ForceUpdate);

	UE_API void ClearCurrentFramebufferWithCurrentScissor(int8 ClearType, int32 NumClearColors, const bool* bClearColorArray, const FLinearColor* ClearColorArray, float Depth, uint32 Stencil);

	FTextureLockTracker GLLockTracker;

	class FOpenGLFenceKick
	{
	public:
		FOpenGLFenceKick(FOpenGLDynamicRHI* RHI)
			: RHI(*RHI)
		{}
		~FOpenGLFenceKick();

		void OnDrawCall();
		void Reset();

	private:
		void InsertKick();

		TArray<UGLsync> Syncs;
		int32 DrawCounter = 0;
		GLuint LastSeenFramebuffer = 0;

		FOpenGLDynamicRHI& RHI;
	} KickHint { this };
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Implements the OpenGLDrv module as a dynamic RHI providing module. */
class FOpenGLDynamicRHIModule : public IDynamicRHIModule
{
public:
	
	// IModuleInterface
	virtual bool SupportsDynamicReloading() override { return false; }

	// IDynamicRHIModule
	virtual bool IsSupported() override;

	virtual FDynamicRHI* CreateRHI(ERHIFeatureLevel::Type RequestedFeatureLevel = ERHIFeatureLevel::Num) override;
};

extern ERHIFeatureLevel::Type GRequestedFeatureLevel;

#undef UE_API
