// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLDrvPrivate.h: Private OpenGL RHI definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "RenderResource.h"
#include "OpenGLDrv.h"

#define SUBALLOCATED_CONSTANT_BUFFER 0

#define GL_CHECK(x)		x; do { GLint Err = glGetError(); if (Err != 0) {FPlatformMisc::LowLevelOutputDebugStringf(TEXT("(%s:%d) GL_CHECK Failed '%s'! %d (%x)\n"), ANSI_TO_TCHAR(__FILE__), __LINE__, ANSI_TO_TCHAR( #x ), Err, Err); check(!Err);}} while (0)

// 
// #if !defined(__GNUC__) && !defined(__clang__) && !defined(__INTEL_COMPILER) && !(defined(_MSC_VER) && _MSC_VER >= 1900)
// #define LOG_AND_GET_GL_INT_TEMP(IntEnum,Default) 
// GLint Value_##IntEnum = Default; if (IntEnum) {glGetIntegerv(IntEnum, &Value_##IntEnum); glGetError();} else {Value_##IntEnum = Default;} 
// UE_LOG(LogRHI, Log, TEXT("  ") ## TEXT(#IntEnum) ## TEXT(": %d"), Value_##IntEnum)
// #else
// #define LOG_AND_GET_GL_INT_TEMP(IntEnum,Default) GLint Value_##IntEnum = Default; if (IntEnum) {glGetIntegerv(IntEnum, &Value_##IntEnum); glGetError();} else {Value_##IntEnum = Default;} 
// UE_LOG(LogRHI, Log, TEXT("  " #IntEnum ": %d"), Value_##IntEnum)
// #endif

#if !defined(__GNUC__) && !defined(__clang__) && !defined(__INTEL_COMPILER) && !(defined(_MSC_VER) && _MSC_VER >= 1900)
#define LOG_AND_GET_GL_INT(IntEnum, Default, Dest) \
		do \
		{ \
			Dest = Default; \
			extern bool GDisableOpenGLDebugOutput; \
			GDisableOpenGLDebugOutput = true; \
			glGetIntegerv(IntEnum, &Dest); \
			GDisableOpenGLDebugOutput = false; \
			\
			UE_LOG(LogRHI, Log, TEXT("  ") ## TEXT(#IntEnum) ## TEXT(": %d"), Dest); \
		} \
		while (0)
#else
	#define LOG_AND_GET_GL_INT(IntEnum, Default, Dest) \
		do \
		{ \
			Dest = Default; \
			extern bool GDisableOpenGLDebugOutput; \
			GDisableOpenGLDebugOutput = true; \
			glGetIntegerv(IntEnum, &Dest); \
			GDisableOpenGLDebugOutput = false; \
			UE_LOG(LogRHI, Log, TEXT("  " #IntEnum ": %d"), Dest); \
		} \
		while (0)
#endif

#define GET_GL_INT(IntEnum, Default, Dest) \
	do \
	{ \
		Dest = Default; \
		extern bool GDisableOpenGLDebugOutput; \
		GDisableOpenGLDebugOutput = true; \
		glGetIntegerv(IntEnum, &Dest); \
		GDisableOpenGLDebugOutput = false; \
		} \
	while (0)

/**
 * The OpenGL RHI stats.
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("Present time"),STAT_OpenGLPresentTime,STATGROUP_OpenGLRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Triangles drawn"),STAT_OpenGLTriangles,STATGROUP_OpenGLRHI, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Lines drawn"),STAT_OpenGLLines,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateTexture time"),STAT_OpenGLCreateTextureTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("LockTexture time"),STAT_OpenGLLockTextureTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UnlockTexture time"),STAT_OpenGLUnlockTextureTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CopyTexture time"),STAT_OpenGLCopyTextureTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CopyMipToMipAsync time"),STAT_OpenGLCopyMipToMipAsyncTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("UploadTextureMip time"),STAT_OpenGLUploadTextureMipTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("CreateBoundShaderState time"),STAT_OpenGLCreateBoundShaderStateTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Constant buffer update time"),STAT_OpenGLConstantBufferUpdateTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Uniform commit time"),STAT_OpenGLUniformCommitTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Shader compile time"),STAT_OpenGLShaderCompileTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Shader compile verify time"),STAT_OpenGLShaderCompileVerifyTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Shader link time"),STAT_OpenGLShaderLinkTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Shader link verify time"),STAT_OpenGLShaderLinkVerifyTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Shader bind param time"),STAT_OpenGLShaderBindParameterTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Uniform buffer pool cleanup time"),STAT_OpenGLUniformBufferCleanupTime,STATGROUP_OpenGLRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Uniform buffer pool memory"),STAT_OpenGLFreeUniformBufferMemory,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Emulated Uniform buffer time"), STAT_OpenGLEmulatedUniformBufferTime,STATGROUP_OpenGLRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Uniform buffer pool num free"),STAT_OpenGLNumFreeUniformBuffers,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Time for first draw of shader programs"), STAT_OpenGLShaderFirstDrawTime,STATGROUP_OpenGLRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Active Program binary memory (estimate driver use)"), STAT_OpenGLProgramBinaryMemory, STATGROUP_OpenGLRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("GL Program count"), STAT_OpenGLProgramCount, STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Program get from cache time"),STAT_OpenGLUseCachedProgramTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Program create from binary time"),STAT_OpenGLCreateProgramFromBinaryTime,STATGROUP_OpenGLRHI, );

DECLARE_CYCLE_STAT_EXTERN(TEXT("Program LRU cache eviction time"), STAT_OpenGLShaderLRUEvictTime, STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Program LRU cache miss time"), STAT_OpenGLShaderLRUMissTime, STATGROUP_OpenGLRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Program LRU count"), STAT_OpenGLShaderLRUProgramCount, STATGROUP_OpenGLRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Program LRU evicted count"), STAT_OpenGLShaderLRUEvictedProgramCount, STATGROUP_OpenGLRHI, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Program LRU miss count"), STAT_OpenGLShaderLRUMissCount, STATGROUP_OpenGLRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Program LRU memory (evicted, heap)"), STAT_OpenGLShaderLRUProgramMemory, STATGROUP_OpenGLRHI, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("Program LRU mem mapped (evicted, filemapped)"), STAT_OpenGLShaderLRUProgramMemoryMapped, STATGROUP_OpenGLRHI, );

#if OPENGLRHI_DETAILED_STATS
DECLARE_CYCLE_STAT_EXTERN(TEXT("DrawPrimitive Time"),STAT_OpenGLDrawPrimitiveTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("DrawPrimitive Driver Time"),STAT_OpenGLDrawPrimitiveDriverTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("DrawPrimitiveUP Time"),STAT_OpenGLDrawPrimitiveUPTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Shader bind time"),STAT_OpenGLShaderBindTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Texture bind time"),STAT_OpenGLTextureBindTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Uniform bind time"),STAT_OpenGLUniformBindTime,STATGROUP_OpenGLRHI, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("VBO setup time"),STAT_OpenGLVBOSetupTime,STATGROUP_OpenGLRHI, );
#endif

/*------------------------------------------------------------------------------
	All platforms using OpenGL must implement the following API.
------------------------------------------------------------------------------*/

/** Platform specific OpenGL context. */
struct FPlatformOpenGLContext;

/** Platform specific OpenGL device. */
struct FPlatformOpenGLDevice;

/**
 * Initialize OpenGL on this platform. This must be called once before device
 * contexts can be created.
 * @returns true if initialization was successful.
 */
bool PlatformInitOpenGL();

/**
 * Just a glGetError() call. Added to make it possible to compile GL_VERIFY macros in
 * other projects than OpenGLDrv.
 */
int32 PlatformGlGetError();

/**
 * Returns true when the calling thread owns the OpenGL rendering context.
 */
bool PlatformOpenGLThreadHasRenderingContext();

/**
 * Create the OpenGL device, encompassing all data needed to handle set of OpenGL contexts
 * for a single OpenGL RHI. This contains shared context (to be used for resource loading),
 * rendering context (to be used for rendering), and viewport contexts (to be used for
 * blitting for various platform-specific viewports). On different platforms implementation
 * details differ, by a lot.
 * This should be called when creating the OpenGL RHI.
 */
FPlatformOpenGLDevice* PlatformCreateOpenGLDevice();

/**
* Returns true if the platform supports a GPU capture tool (eg RenderDoc)
*/
bool PlatformCanEnableGPUCapture();

/**
 * Destroy the OpenGL device for single OpenGL RHI. This should happen when destroying the RHI.
 */
void PlatformDestroyOpenGLDevice(FPlatformOpenGLDevice* Device);

/**
 * Create an OpenGL context connected to some window, or fullscreen. This will be used to
 * transfer the rendering results to screen by rendering thread, inside RHIEndDrawingViewport().
 */
FPlatformOpenGLContext* PlatformCreateOpenGLContext(FPlatformOpenGLDevice* Device, void* InWindowHandle);

/**
 * Destroy a viewport OpenGL context.
 */
void PlatformDestroyOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context);

/**
 * Create an FPlatformOpenGLSurfaceContext that contains properties required for an individual output surface
 * attached to the FPlatformOpenGLContext.
 */
struct FPlatformOpenGLSurfaceContext* PlatformCreateOpenGLSurfaceContext(FPlatformOpenGLContext* SourceContext, void* InWindowHandle);

/**
 * destroy the FPlatformOpenGLSurfaceContext
 */
void PlatformDestroyOpenGLSurfaceContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context, FPlatformOpenGLSurfaceContext* SurfaceContext);

/**
 * Set up rendering OpenGL context on current thread. This is the only context that can be used for rendering,
 * and not only loading resources and framebuffer blitting (due to limitations of Windows OpenGL).
 */
void PlatformRenderingContextSetup(FPlatformOpenGLDevice* Device);

/**
 * This is used to detach an OpenGL context from current thread.
 */
void PlatformNULLContextSetup();

// Creates a platform-specific back buffer
class FOpenGLTexture* PlatformCreateBuiltinBackBuffer(FOpenGLDynamicRHI* OpenGLRHI, struct FPlatformOpenGLSurfaceContext* SurfaceContext, uint32 SizeX, uint32 SizeY);

/**
 * Main function for transferring data to on-screen buffers.
 * On Windows it temporarily switches OpenGL context, on Mac only context's output view.
 * Should return true if frame was presented and it is necessary to finish frame rendering.
 */
bool PlatformBlitToViewport(IRHICommandContext& RHICmdContext, FPlatformOpenGLDevice* Device, FOpenGLViewport& Viewport, uint32 BackbufferSizeX, uint32 BackbufferSizeY, bool bPresent,bool bLockToVsync);

/**
 * Resize the GL context for platform.
 */
void PlatformResizeGLContext( FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context, struct FPlatformOpenGLSurfaceContext* SurfaceContext, uint32 SizeX, uint32 SizeY, bool bFullscreen, bool bWasFullscreen, GLenum BackBufferTarget, GLuint BackBufferResource);


/**
 * Returns a supported screen resolution that most closely matches input.
 */
void PlatformGetSupportedResolution(uint32 &Width, uint32 &Height);

/**
 * Retrieve available screen resolutions.
 */
bool PlatformGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate);

/**
 * Restore the original display mode
 */
void PlatformRestoreDesktopDisplayMode();

// Returns native window handle.
void* PlatformGetWindow(FPlatformOpenGLContext* Context, void** AddParam);

/*------------------------------------------------------------------------------
	OpenGL texture format table.
------------------------------------------------------------------------------*/

struct FOpenGLTextureFormat
{
	// [0]: without sRGB, [1]: with sRGB
	GLenum InternalFormat[2] = { GL_NONE, GL_NONE };
	GLenum Format = GL_NONE;
	GLenum Type = GL_NONE;
	bool bCompressed = false;
	// Reorder to B and R elements using texture swizzle
	bool bBGRA = false;

	FOpenGLTextureFormat() = default;

	FOpenGLTextureFormat(
		GLenum InInternalFormat, GLenum InInternalFormatSRGB, GLenum InFormat,
		GLenum InType, bool bInCompressed, bool bInBGRA)
	{
		InternalFormat[0] = InInternalFormat;
		InternalFormat[1] = InInternalFormatSRGB;
		Format = InFormat;
		Type = InType;
		bCompressed = bInCompressed;
		bBGRA = bInBGRA;
	}
};

extern FOpenGLTextureFormat OPENGLDRV_API GOpenGLTextureFormats[PF_MAX];

inline uint32 FindMaxMipmapLevel(uint32 Size)
{
	uint32 MipCount = 1;
	while( Size >>= 1 )
	{
		MipCount++;
	}
	return MipCount;
}

inline uint32 FindMaxMipmapLevel(uint32 Width, uint32 Height)
{
	return FindMaxMipmapLevel((Width > Height) ? Width : Height);
}

inline uint32 FindMaxMipmapLevel(uint32 Width, uint32 Height, uint32 Depth)
{
	return FindMaxMipmapLevel((Width > Height) ? Width : Height, Depth);
}

inline void FindPrimitiveType(uint32 InPrimitiveType, uint32 InNumPrimitives, GLenum &DrawMode, GLsizei &NumElements)
{
	DrawMode = GL_TRIANGLES;
	NumElements = InNumPrimitives;

	switch (InPrimitiveType)
	{
	case PT_TriangleList:
		DrawMode = GL_TRIANGLES;
		NumElements = InNumPrimitives * 3;
		break;
	case PT_TriangleStrip:
		DrawMode = GL_TRIANGLE_STRIP;
		NumElements = InNumPrimitives + 2;
		break;
	case PT_LineList:
		DrawMode = GL_LINES;
		NumElements = InNumPrimitives * 2;
		break;
	case PT_PointList:
		DrawMode = GL_POINTS;
		NumElements = InNumPrimitives;
		break;
	default:
		UE_LOGF(LogRHI, Fatal,"Unsupported primitive type %u", InPrimitiveType);
		break;
	}
}

inline uint32 FindUniformElementSize(GLenum UniformType)
{
	switch (UniformType)
	{
	case GL_FLOAT:
		return sizeof(float);
	case GL_FLOAT_VEC2:
		return sizeof(float) * 2;
	case GL_FLOAT_VEC3:
		return sizeof(float) * 3;
	case GL_FLOAT_VEC4:
		return sizeof(float) * 4;

	case GL_INT:
	case GL_BOOL:
		return sizeof(uint32);
	case GL_INT_VEC2:
	case GL_BOOL_VEC2:
		return sizeof(uint32) * 2;
	case GL_INT_VEC3:
	case GL_BOOL_VEC3:
		return sizeof(uint32) * 3;
	case GL_INT_VEC4:
	case GL_BOOL_VEC4:
		return sizeof(uint32) * 4;

	case GL_FLOAT_MAT2:
		return sizeof(float) * 4;
	case GL_FLOAT_MAT3:
		return sizeof(float) * 9;
	case GL_FLOAT_MAT4:
		return sizeof(float) * 16;
	case GL_FLOAT_MAT2x3:
		return sizeof(float) * 6;
	case GL_FLOAT_MAT2x4:
		return sizeof(float) * 8;
	case GL_FLOAT_MAT3x2:
		return sizeof(float) * 6;
	case GL_FLOAT_MAT3x4:
		return sizeof(float) * 12;
	case GL_FLOAT_MAT4x2:
		return sizeof(float) * 8;
	case GL_FLOAT_MAT4x3:
		return sizeof(float) * 12;

	case GL_SAMPLER_1D:
	case GL_SAMPLER_2D:
	case GL_SAMPLER_3D:
	case GL_SAMPLER_CUBE:
	case GL_SAMPLER_1D_SHADOW:
	case GL_SAMPLER_2D_SHADOW:
	default:
		return sizeof(uint32);
	}
}

/**
 * Calculate the dynamic buffer size needed for a given allocation.
 */
inline uint32 CalcDynamicBufferSize(uint32 Size)
{
	// Allocate dynamic buffers in MB increments.
	return Align(Size, (1 << 20));
}

/**
 * Call after creating a context to initialise default state values to correct values for UE.
 */
void InitDefaultGLContextState(void);

extern bool GUseEmulatedUniformBuffers;
