// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLES.cpp: OpenGL ES implementation.
=============================================================================*/

#include "OpenGLES.h"

#if UGL_PLATFORM_SUPPORTS_GLES

#include "OpenGLDrvPrivate.h"

/** GL_EXT_disjoint_timer_query */
bool FOpenGLES::bSupportsDisjointTimeQueries = false;

static TAutoConsoleVariable<int32> CVarDisjointTimerQueries(
	TEXT("r.DisjointTimerQueries"),
	0,
	TEXT("If set to 1, allows GPU time to be measured (e.g. STAT UNIT). It defaults to 0 because some devices supports it but very slowly."),
	ECVF_ReadOnly);

/** Some timer query implementations are never disjoint */
bool FOpenGLES::bTimerQueryCanBeDisjoint = true;

/** GL_APPLE_texture_format_BGRA8888 */
bool FOpenGLES::bSupportsBGRA8888 = false;

/** GL_EXT_color_buffer_half_float */
bool FOpenGLES::bSupportsColorBufferHalfFloat = false;

/** GL_EXT_color_buffer_float */
bool FOpenGLES::bSupportsColorBufferFloat = false;

/** GL_EXT_shader_framebuffer_fetch */
bool FOpenGLES::bSupportsShaderFramebufferFetch = false;

/** GL_EXT_shader_framebuffer_fetch (MRT's) */
bool FOpenGLES::bSupportsShaderMRTFramebufferFetch = false;


/** GL_ARM_shader_framebuffer_fetch_depth_stencil */
bool FOpenGLES::bSupportsShaderDepthStencilFetch = false;

/** GL_EXT_multisampled_render_to_texture */
bool FOpenGLES::bSupportsMultisampledRenderToTexture = false;

/** OpenGL ES 3.0 profile */
bool FOpenGLES::bSupportsETC2 = false;

/** GL_EXT_shader_pixel_local_storage */
bool FOpenGLES::bSupportsPixelLocalStorage = false;

/** GL_FRAGMENT_SHADER, GL_LOW_FLOAT */
int FOpenGLES::ShaderLowPrecision = 0;

/** GL_FRAGMENT_SHADER, GL_MEDIUM_FLOAT */
int FOpenGLES::ShaderMediumPrecision = 0;

/** GL_FRAGMENT_SHADER, GL_HIGH_FLOAT */
int FOpenGLES::ShaderHighPrecision = 0;

/** GL_NV_framebuffer_blit */
bool FOpenGLES::bSupportsNVFrameBufferBlit = false;

/* This indicates failure when attempting to retrieve driver's binary representation of the hack program  */
bool FOpenGLES::bBinaryProgramRetrievalFailed = false;

/* Some Mali devices do not work correctly with early_fragment_test enabled */
bool FOpenGLES::bRequiresDisabledEarlyFragmentTests = false;

/* This is a workaround for a Mali bug where read-only buffers do not work when passed to functions*/
bool FOpenGLES::bRequiresReadOnlyBuffersWorkaround = false;

/* This is a workaround for Samsung xclipse that fails to compile shaders that use the 'precise' qualifier*/
bool FOpenGLES::bRequiresPreciseQualifierWorkaround = false;

/* This is to avoid a bug in Adreno drivers that define GL_ARM_shader_framebuffer_fetch_depth_stencil even when device does not support this extension  */
bool FOpenGLES::bRequiresARMShaderFramebufferFetchDepthStencilUndef = false;

/** Framebuffer fetch can be used to do programmable blending without running into driver issues */
bool FOpenGLES::bSupportsShaderFramebufferFetchProgrammableBlending = false;

/** GL_EXT_buffer_storage */
bool FOpenGLES::bSupportsBufferStorage = false;

/** GL_EXT_depth_clamp */
bool FOpenGLES::bSupportsDepthClamp = false;

bool FOpenGLES::bHasHardwareHiddenSurfaceRemoval = false;
bool FOpenGLES::bSupportsMobileMultiView = false;
GLint FOpenGLES::MaxMSAASamplesTileMem = 1;

GLint FOpenGLES::MaxComputeUniformComponents = -1;

GLint FOpenGLES::MaxComputeUAVUnits = -1;
GLint FOpenGLES::MaxPixelUAVUnits = -1;
GLint FOpenGLES::MaxCombinedUAVUnits = 0;

/** GL_EXT_texture_compression_astc_decode_mode */
bool FOpenGLES::bSupportsASTCDecodeMode = false;

// GL_OES_get_program_binary
bool FOpenGLES::bSupportsProgramBinary = false;

FOpenGLES::EFeatureLevelSupport FOpenGLES::CurrentFeatureLevelSupport = FOpenGLES::EFeatureLevelSupport::ES31;

bool FOpenGLES::SupportsDisjointTimeQueries()
{
	bool bAllowDisjointTimerQueries = false;
	bAllowDisjointTimerQueries = (CVarDisjointTimerQueries.GetValueOnRenderThread() == 1);
	return bSupportsDisjointTimeQueries && bAllowDisjointTimerQueries;
}

void FOpenGLES::ProcessQueryGLInt()
{
	GLint MaxVertexAttribs;
	LOG_AND_GET_GL_INT(GL_MAX_VERTEX_ATTRIBS, 0, MaxVertexAttribs);
	if (MaxVertexAttribs < 16)
	{
		UE_LOGF(LogRHI, Error,
			"Device reports support for %d vertex attributes, UnrealEditor requires 16. Rendering artifacts may occur.",
			MaxVertexAttribs
		);
	}

	LOG_AND_GET_GL_INT(GL_MAX_VARYING_VECTORS, 0, MaxVaryingVectors);
	LOG_AND_GET_GL_INT(GL_MAX_VERTEX_UNIFORM_VECTORS, 0, MaxVertexUniformComponents);
	LOG_AND_GET_GL_INT(GL_MAX_FRAGMENT_UNIFORM_VECTORS, 0, MaxPixelUniformComponents);
	LOG_AND_GET_GL_INT(GL_TEXTURE_BUFFER_OFFSET_ALIGNMENT, 0, TextureBufferAlignment);
	
	LOG_AND_GET_GL_INT(GL_MAX_COMPUTE_UNIFORM_COMPONENTS, 0, MaxComputeUniformComponents);
	LOG_AND_GET_GL_INT(GL_MAX_COMBINED_IMAGE_UNIFORMS, 0, MaxCombinedUAVUnits);
	LOG_AND_GET_GL_INT(GL_MAX_COMPUTE_IMAGE_UNIFORMS, 0, MaxComputeUAVUnits);
	LOG_AND_GET_GL_INT(GL_MAX_FRAGMENT_IMAGE_UNIFORMS, 0, MaxPixelUAVUnits);

	GLint MaxCombinedSSBOUnits = 0;
	GET_GL_INT(GL_MAX_COMBINED_SHADER_STORAGE_BLOCKS, 0, MaxCombinedSSBOUnits);
	// UAVs slots in UE are shared between Images and SSBO, so this should be max(GL_MAX_COMBINED_IMAGE_UNIFORMS, GL_MAX_COMBINED_SHADER_STORAGE_BLOCKS)
	MaxCombinedUAVUnits = FMath::Max(MaxCombinedUAVUnits, MaxCombinedSSBOUnits);

	// clamp UAV units to a sensible limit
	MaxCombinedUAVUnits = FMath::Min(MaxCombinedUAVUnits, 16);
	MaxComputeUAVUnits = FMath::Min(MaxComputeUAVUnits, 16);
	// this is split between VS and PS, 4 to each stage
	MaxPixelUAVUnits = FMath::Min(MaxPixelUAVUnits, 4);

	const GLint RequiredMaxVertexUniformComponents = 256;
	if (MaxVertexUniformComponents < RequiredMaxVertexUniformComponents)
	{
		UE_LOGF(LogRHI, Warning,
			"Device reports support for %d vertex uniform vectors, UnrealEditor requires %d. Rendering artifacts may occur, especially with skeletal meshes. Some drivers, e.g. iOS, report a smaller number than is actually supported.",
			MaxVertexUniformComponents,
			RequiredMaxVertexUniformComponents
		);
	}
	MaxVertexUniformComponents = FMath::Max<GLint>(MaxVertexUniformComponents, RequiredMaxVertexUniformComponents);
	MaxGeometryUniformComponents = 0;
	MaxGeometryTextureImageUnits = 0;

	// Set lowest possible limits for texture units, to avoid extra work in GL RHI
	MaxTextureImageUnits = FMath::Min(MaxTextureImageUnits, 16);
	MaxVertexTextureImageUnits = FMath::Min(MaxVertexTextureImageUnits, 16);
	MaxCombinedTextureImageUnits = FMath::Min(MaxCombinedTextureImageUnits, 32);
}

void FOpenGLES::ProcessExtensions(const FString& ExtensionsString)
{
	ProcessQueryGLInt();
	FOpenGLBase::ProcessExtensions(ExtensionsString);

	bSupportsDisjointTimeQueries = ExtensionsString.Contains(TEXT("GL_EXT_disjoint_timer_query"));
	if (bSupportsDisjointTimeQueries)
	{
		glQueryCounterEXT = (PFNGLQUERYCOUNTEREXTPROC)((void*)eglGetProcAddress("glQueryCounterEXT"));;
		glGetQueryObjectui64vEXT = (PFNGLGETQUERYOBJECTUI64VEXTPROC)((void*)eglGetProcAddress("glGetQueryObjectui64vEXT"));
	}

	bTimerQueryCanBeDisjoint = !ExtensionsString.Contains(TEXT("GL_NV_timer_query"));
	bSupportsBGRA8888 = ExtensionsString.Contains(TEXT("GL_APPLE_texture_format_BGRA8888")) || ExtensionsString.Contains(TEXT("GL_IMG_texture_format_BGRA8888")) || ExtensionsString.Contains(TEXT("GL_EXT_texture_format_BGRA8888"));
	bSupportsColorBufferFloat = ExtensionsString.Contains(TEXT("GL_EXT_color_buffer_float"));
	bSupportsColorBufferHalfFloat = ExtensionsString.Contains(TEXT("GL_EXT_color_buffer_half_float"));
	bSupportsMultisampledRenderToTexture = ExtensionsString.Contains(TEXT("GL_EXT_multisampled_render_to_texture"));
	bSupportsNVFrameBufferBlit = ExtensionsString.Contains(TEXT("GL_NV_framebuffer_blit"));
	bSupportsBufferStorage = ExtensionsString.Contains(TEXT("GL_EXT_buffer_storage"));
	bSupportsDepthClamp = ExtensionsString.Contains(TEXT("GL_EXT_depth_clamp"));
	bSupportsASTCDecodeMode = ExtensionsString.Contains(TEXT("GL_EXT_texture_compression_astc_decode_mode"));
	
	bSupportsShaderFramebufferFetch = ExtensionsString.Contains(TEXT("GL_EXT_shader_framebuffer_fetch")) || ExtensionsString.Contains(TEXT("GL_NV_shader_framebuffer_fetch")) || ExtensionsString.Contains(TEXT("GL_ARM_shader_framebuffer_fetch ")); // has space at the end to exclude GL_ARM_shader_framebuffer_fetch_depth_stencil match
	bSupportsShaderMRTFramebufferFetch = ExtensionsString.Contains(TEXT("GL_EXT_shader_framebuffer_fetch")) || ExtensionsString.Contains(TEXT("GL_NV_shader_framebuffer_fetch"));
	bSupportsPixelLocalStorage = ExtensionsString.Contains(TEXT("GL_EXT_shader_pixel_local_storage"));
	bSupportsShaderDepthStencilFetch = ExtensionsString.Contains(TEXT("GL_ARM_shader_framebuffer_fetch_depth_stencil"));
	bSupportsShaderFramebufferFetchProgrammableBlending = bSupportsShaderFramebufferFetch;

	// Report shader precision
	int Range[2];
	glGetShaderPrecisionFormat(GL_FRAGMENT_SHADER, GL_LOW_FLOAT, Range, &ShaderLowPrecision);
	glGetShaderPrecisionFormat(GL_FRAGMENT_SHADER, GL_MEDIUM_FLOAT, Range, &ShaderMediumPrecision);
	glGetShaderPrecisionFormat(GL_FRAGMENT_SHADER, GL_HIGH_FLOAT, Range, &ShaderHighPrecision);
	UE_LOGF(LogRHI, Log, "Fragment shader lowp precision: %d", ShaderLowPrecision);
	UE_LOGF(LogRHI, Log, "Fragment shader mediump precision: %d", ShaderMediumPrecision);
	UE_LOGF(LogRHI, Log, "Fragment shader highp precision: %d", ShaderHighPrecision);

	if (FPlatformMisc::IsDebuggerPresent() && UE_BUILD_DEBUG)
	{
		// Enable GL debug markers if we're running in Xcode
		extern int32 GEmitMeshDrawEvent;
		GEmitMeshDrawEvent = 1;
		SetEmitDrawEvents(true);
	}

	glPushGroupMarkerEXT = (PFNGLPUSHGROUPMARKEREXTPROC)((void*)eglGetProcAddress("glPushGroupMarkerEXT"));
	glPopGroupMarkerEXT = (PFNGLPOPGROUPMARKEREXTPROC)((void*)eglGetProcAddress("glPopGroupMarkerEXT"));

	if (ExtensionsString.Contains(TEXT("GL_EXT_DEBUG_LABEL")))
	{
		glLabelObjectEXT = (PFNGLLABELOBJECTEXTPROC)((void*)eglGetProcAddress("glLabelObjectEXT"));
		glGetObjectLabelEXT = (PFNGLGETOBJECTLABELEXTPROC)((void*)eglGetProcAddress("glGetObjectLabelEXT"));
	}

	if (bSupportsBufferStorage)
	{
		glBufferStorageEXT = (PFNGLBUFFERSTORAGEEXTPROC)((void*)eglGetProcAddress("glBufferStorageEXT"));
	}

	if (ExtensionsString.Contains(TEXT("GL_EXT_multisampled_render_to_texture2")))
	{
		glFramebufferTexture2DMultisampleEXT = (PFNGLFRAMEBUFFERTEXTURE2DMULTISAMPLEEXTPROC)((void*)eglGetProcAddress("glFramebufferTexture2DMultisampleEXT"));
		glRenderbufferStorageMultisampleEXT = (PFNGLRENDERBUFFERSTORAGEMULTISAMPLEEXTPROC)((void*)eglGetProcAddress("glRenderbufferStorageMultisampleEXT"));
		glGetIntegerv(GL_MAX_SAMPLES_EXT, &MaxMSAASamplesTileMem);
		MaxMSAASamplesTileMem = FMath::Max<GLint>(MaxMSAASamplesTileMem, 1);
		UE_LOGF(LogRHI, Log, "Support for %dx MSAA detected", MaxMSAASamplesTileMem);
	}
	else
	{
		// indicates RHI supports on-chip MSAA but this device does not.
		MaxMSAASamplesTileMem = 1;
	}

	bSupportsProgramBinary = ExtensionsString.Contains(TEXT("GL_OES_get_program_binary"));

	bSupportsETC2 = true;
	// According to https://www.khronos.org/registry/gles/extensions/EXT/EXT_color_buffer_float.txt
	bSupportsColorBufferHalfFloat = (bSupportsColorBufferHalfFloat || bSupportsColorBufferFloat);
		
	// Mobile multi-view setup
	const bool bMultiViewSupport = ExtensionsString.Contains(TEXT("GL_OVR_multiview"));
	const bool bMultiView2Support = ExtensionsString.Contains(TEXT("GL_OVR_multiview2"));
	const bool bMultiViewMultiSampleSupport = ExtensionsString.Contains(TEXT("GL_OVR_multiview_multisampled_render_to_texture"));
	if (bMultiViewSupport && bMultiView2Support && bMultiViewMultiSampleSupport)
	{
		glFramebufferTextureMultiviewOVR = (PFNGLFRAMEBUFFERTEXTUREMULTIVIEWOVRPROC)((void*)eglGetProcAddress("glFramebufferTextureMultiviewOVR"));
		glFramebufferTextureMultisampleMultiviewOVR = (PFNGLFRAMEBUFFERTEXTUREMULTISAMPLEMULTIVIEWOVRPROC)((void*)eglGetProcAddress("glFramebufferTextureMultisampleMultiviewOVR"));

		bSupportsMobileMultiView = (glFramebufferTextureMultiviewOVR != NULL) && (glFramebufferTextureMultisampleMultiviewOVR != NULL);

		// Just because the driver declares multi-view support and hands us valid function pointers doesn't actually guarantee the feature works...
		if (bSupportsMobileMultiView)
		{
			UE_LOGF(LogRHI, Log, "Device supports mobile multi-view.");
		}
	}
	
	if (IsES32Usable())
	{
		glTexBufferEXT = (PFNGLTEXBUFFEREXTPROC)((void*)eglGetProcAddress("glTexBuffer"));
		glTexBufferRangeEXT = (PFNGLTEXBUFFERRANGEEXTPROC)((void*)eglGetProcAddress("glTexBufferRange"));
		glCopyImageSubData = (PFNGLCOPYIMAGESUBDATAEXTPROC)((void*)eglGetProcAddress("glCopyImageSubData"));
		glEnableiEXT = (PFNGLENABLEIEXTPROC)((void*)eglGetProcAddress("glEnablei"));
		glDisableiEXT = (PFNGLDISABLEIEXTPROC)((void*)eglGetProcAddress("glDisablei"));
		glBlendEquationiEXT = (PFNGLBLENDEQUATIONIEXTPROC)((void*)eglGetProcAddress("glBlendEquationi"));
		glBlendEquationSeparateiEXT = (PFNGLBLENDEQUATIONSEPARATEIEXTPROC)((void*)eglGetProcAddress("glBlendEquationSeparatei"));
		glBlendFunciEXT = (PFNGLBLENDFUNCIEXTPROC)((void*)eglGetProcAddress("glBlendFunci"));
		glBlendFuncSeparateiEXT = (PFNGLBLENDFUNCSEPARATEIEXTPROC)((void*)eglGetProcAddress("glBlendFuncSeparatei"));
		glColorMaskiEXT = (PFNGLCOLORMASKIEXTPROC)((void*)eglGetProcAddress("glColorMaski"));
		glFramebufferTexture = (PFNGLFRAMEBUFFERTEXTUREPROC)((void*)eglGetProcAddress("glFramebufferTexture"));
	}
	
	if (!glEnableiEXT && ExtensionsString.Contains(TEXT("GL_EXT_draw_buffers_indexed")))
	{
		// GL_EXT_draw_buffers_indexed
		glEnableiEXT = (PFNGLENABLEIEXTPROC)((void*)eglGetProcAddress("glEnableiEXT"));
		glDisableiEXT = (PFNGLDISABLEIEXTPROC)((void*)eglGetProcAddress("glDisableiEXT"));
		glBlendEquationiEXT = (PFNGLBLENDEQUATIONIEXTPROC)((void*)eglGetProcAddress("glBlendEquationiEXT"));
		glBlendEquationSeparateiEXT = (PFNGLBLENDEQUATIONSEPARATEIEXTPROC)((void*)eglGetProcAddress("glBlendEquationSeparateiEXT"));
		glBlendFunciEXT = (PFNGLBLENDFUNCIEXTPROC)((void*)eglGetProcAddress("glBlendFunciEXT"));
		glBlendFuncSeparateiEXT = (PFNGLBLENDFUNCSEPARATEIEXTPROC)((void*)eglGetProcAddress("glBlendFuncSeparateiEXT"));
		glColorMaskiEXT = (PFNGLCOLORMASKIEXTPROC)((void*)eglGetProcAddress("glColorMaskiEXT"));
	}
	bSupportsDrawBuffersBlend = (glEnableiEXT != nullptr);
		
	if (!glTexBufferEXT && ExtensionsString.Contains(TEXT("GL_EXT_texture_buffer")))
	{
		// GL_EXT_texture_buffer
		glTexBufferEXT = (PFNGLTEXBUFFEREXTPROC)((void*)eglGetProcAddress("glTexBufferEXT"));
		glTexBufferRangeEXT = (PFNGLTEXBUFFERRANGEEXTPROC)((void*)eglGetProcAddress("glTexBufferRangeEXT"));
	}
}

#endif //UGL_PLATFORM_SUPPORTS_GLES
