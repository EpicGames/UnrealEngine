// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLES2.h: Public OpenGL ES 2.0 definitions for non-common functionality
=============================================================================*/

#pragma once

#include "OpenGLPlatform.h"

#if UGL_PLATFORM_SUPPORTS_GLES

#include "OpenGLBase.h"
#include "OpenGLUtil.h"		// for VERIFY_GL

namespace OpenGLConsoleVariables
{
	extern int32 bUseUnsynchronizedBufferMapping;
}

struct FOpenGLES : public FOpenGLBase
{
	static FORCEINLINE bool IsES31Usable()
	{
		check(CurrentFeatureLevelSupport != EFeatureLevelSupport::Invalid);
		return CurrentFeatureLevelSupport >= EFeatureLevelSupport::ES31;
	}

	static FORCEINLINE bool IsES32Usable()
	{
		check(CurrentFeatureLevelSupport != EFeatureLevelSupport::Invalid);
		return CurrentFeatureLevelSupport == EFeatureLevelSupport::ES32;
	}

	static void		ProcessQueryGLInt();
	static void		ProcessExtensions(const FString& ExtensionsString);

	static FORCEINLINE bool SupportsUniformBuffers() { return true; }
	static FORCEINLINE bool SupportsStructuredBuffers() { return true; }
	static FORCEINLINE bool SupportsExactOcclusionQueries() { return false; }
	// MLCHANGES BEGIN -- changed to use bSupportsDisjointTimeQueries
	static FORCEINLINE bool SupportsTimestampQueries() { return bSupportsDisjointTimeQueries; }
	// MLCHANGES END
	static bool SupportsDisjointTimeQueries();
	static FORCEINLINE bool SupportsDepthStencilRead() { return false; }
	static FORCEINLINE bool SupportsFloatReadSurface() { return SupportsColorBufferHalfFloat(); }
	static FORCEINLINE bool SupportsWideMRT() { return true; }
	static FORCEINLINE bool SupportsPolygonMode() { return false; }
	static FORCEINLINE bool SupportsTexture3D() { return true; }
	static FORCEINLINE bool SupportsMobileMultiView() { return bSupportsMobileMultiView; }
	static FORCEINLINE bool SupportsImageExternal() { return false; }
	static FORCEINLINE bool SupportsTextureLODBias() { return false; }
	static FORCEINLINE bool SupportsTextureCompare() { return false; }
	static FORCEINLINE bool SupportsDrawIndexOffset() { return false; }
	static FORCEINLINE bool SupportsDiscardFrameBuffer() { return true; }
	static FORCEINLINE bool SupportsIndexedExtensions() { return false; }
	static FORCEINLINE bool SupportsColorBufferFloat() { return bSupportsColorBufferFloat; }
	static FORCEINLINE bool SupportsColorBufferHalfFloat() { return bSupportsColorBufferHalfFloat; }
	static FORCEINLINE bool SupportsShaderFramebufferFetch() { return bSupportsShaderFramebufferFetch; }
	static FORCEINLINE bool SupportsShaderMRTFramebufferFetch() { return bSupportsShaderMRTFramebufferFetch; }
	static FORCEINLINE bool SupportsShaderFramebufferFetchProgrammableBlending() { return bSupportsShaderFramebufferFetchProgrammableBlending; }
	static FORCEINLINE bool SupportsShaderDepthStencilFetch() { return bSupportsShaderDepthStencilFetch; }
	static FORCEINLINE bool SupportsPixelLocalStorage() { return bSupportsPixelLocalStorage; }
	static FORCEINLINE bool SupportsMultisampledRenderToTexture() { return bSupportsMultisampledRenderToTexture; }
	static FORCEINLINE bool SupportsVertexArrayBGRA() { return false; }
	static FORCEINLINE bool SupportsBGRA8888() { return bSupportsBGRA8888; }
	static FORCEINLINE bool SupportsDXT() { return false; }
	static FORCEINLINE bool SupportsETC2() { return bSupportsETC2; }
	static FORCEINLINE GLenum GetDepthFormat() { return GL_DEPTH_COMPONENT24; }
	static FORCEINLINE GLenum GetShadowDepthFormat() { return GL_DEPTH_COMPONENT16; }
	static FORCEINLINE bool SupportsFramebufferSRGBEnable() { return false; }
	static FORCEINLINE bool SupportsRGB10A2() { return bSupportsRGB10A2; }
	static FORCEINLINE bool SupportsDrawIndirect() { return true; }
	static FORCEINLINE bool SupportsBufferStorage() { return bSupportsBufferStorage; }
	static FORCEINLINE bool SupportsDepthClamp() { return bSupportsDepthClamp; }
	

	static FORCEINLINE bool HasBinaryProgramRetrievalFailed() { return bBinaryProgramRetrievalFailed; }
	static FORCEINLINE bool RequiresDisabledEarlyFragmentTests() { return bRequiresDisabledEarlyFragmentTests; }
	static FORCEINLINE bool RequiresReadOnlyBuffersWorkaround() { return bRequiresReadOnlyBuffersWorkaround; }
	static FORCEINLINE bool RequiresPreciseQualifierWorkaround() { return bRequiresPreciseQualifierWorkaround; }
	static FORCEINLINE bool RequiresARMShaderFramebufferFetchDepthStencilUndef() { return bRequiresARMShaderFramebufferFetchDepthStencilUndef; }

	// Adreno doesn't support HALF_FLOAT
	static FORCEINLINE int32 GetReadHalfFloatPixelsEnum() { return GL_FLOAT; }
	static FORCEINLINE GLint GetMaxMSAASamplesTileMem() { return MaxMSAASamplesTileMem; }

	// On iOS both glMapBufferOES() and glBufferSubData() for immediate vertex and index data
	// is the slow path (they both hit GPU sync and data cache flush in driver according to profiling in driver symbols).
	// Turning this to false reverts back to not using vertex and index buffers
	// for glDrawArrays() and glDrawElements() on dynamic data.
	static FORCEINLINE bool SupportsFastBufferData() { return false; }

	static FORCEINLINE bool SupportsASTCDecodeMode() { return bSupportsASTCDecodeMode; }

	// Optional
	static FORCEINLINE void BeginQuery(GLenum QueryType, GLuint QueryId)
	{
		glBeginQuery(QueryType, QueryId);
	}

	static FORCEINLINE void EndQuery(GLenum QueryType)
	{
		glEndQuery(QueryType);
	}

	static FORCEINLINE void GenQueries(GLsizei NumQueries, GLuint* QueryIDs)
	{
		glGenQueries(NumQueries, QueryIDs);
	}

	static FORCEINLINE void DeleteQueries(GLsizei NumQueries, const GLuint* QueryIDs)
	{
		glDeleteQueries(NumQueries, QueryIDs);
	}

	static FORCEINLINE void QueryTimestampCounter(GLuint QueryID)
	{
		glQueryCounterEXT(QueryID, GL_TIMESTAMP_EXT);
	}

	using FOpenGLBase::GetQueryObject;

	static FORCEINLINE void GetQueryObject(GLuint QueryId, EQueryMode QueryMode, GLuint* OutResult)
	{
		GLenum QueryName = (QueryMode == QM_Result) ? GL_QUERY_RESULT : GL_QUERY_RESULT_AVAILABLE;
		glGetQueryObjectuiv(QueryId, QueryName, OutResult);
	}

	static FORCEINLINE void GetQueryObject(GLuint QueryId, EQueryMode QueryMode, GLuint64* OutResult)
	{
		GLenum QueryName = (QueryMode == QM_Result) ? GL_QUERY_RESULT : GL_QUERY_RESULT_AVAILABLE;
		glGetQueryObjectui64vEXT(QueryId, QueryName, OutResult);
	}
		
	static FORCEINLINE void LabelObject(GLenum Type, GLuint Object, const ANSICHAR* Name)
	{
		if (glLabelObjectEXT != nullptr)
		{
			glLabelObjectEXT(Type, Object, 0, Name);
		}
	}

	static FORCEINLINE GLsizei GetLabelObject(GLenum Type, GLuint Object, GLsizei BufferSize, ANSICHAR* OutName)
	{
		GLsizei Length = 0;
		if (glGetObjectLabelEXT != nullptr)
		{
			glGetObjectLabelEXT(Type, Object, BufferSize, &Length, OutName);
		}
		return Length;
	}

	static FORCEINLINE void PushGroupMarker(const ANSICHAR* Name)
	{
		if (glPushGroupMarkerEXT != nullptr)
		{
			glPushGroupMarkerEXT(0, Name);
		}
	}

	static FORCEINLINE void PopGroupMarker()
	{
		if (glPopGroupMarkerEXT != nullptr)
		{
			glPopGroupMarkerEXT();
		}
	}

	static FORCEINLINE void* MapBufferRange(GLenum Type, uint32 InOffset, uint32 InSize, EResourceLockMode LockMode)
	{
		GLenum Access;
		switch (LockMode)
		{
		case EResourceLockMode::RLM_ReadOnly:
			Access = GL_MAP_READ_BIT;
			break;
		case EResourceLockMode::RLM_ReadOnlyPersistent:
			Access = (GL_MAP_READ_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT);
			break;
		case EResourceLockMode::RLM_WriteOnly:
			Access = OpenGLConsoleVariables::bUseUnsynchronizedBufferMapping ? (GL_MAP_INVALIDATE_BUFFER_BIT | GL_MAP_WRITE_BIT | GL_MAP_UNSYNCHRONIZED_BIT) : (GL_MAP_INVALIDATE_BUFFER_BIT | GL_MAP_WRITE_BIT);
			break;
		case EResourceLockMode::RLM_WriteOnlyUnsynchronized:
			Access = (GL_MAP_WRITE_BIT | GL_MAP_UNSYNCHRONIZED_BIT);
			break;
		case EResourceLockMode::RLM_WriteOnlyPersistent:
			Access = (GL_MAP_WRITE_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT);
			break;
		case EResourceLockMode::RLM_ReadWrite:
		default:
			Access = (GL_MAP_READ_BIT | GL_MAP_WRITE_BIT);
	}
		return glMapBufferRange(Type, InOffset, InSize, Access);
	}

	static FORCEINLINE void UnmapBuffer(GLenum Type)
	{
		glUnmapBuffer(Type);
	}

	static FORCEINLINE void UnmapBufferRange(GLenum Type, uint32 InOffset, uint32 InSize)
	{
		UnmapBuffer(Type);
	}

	static FORCEINLINE void TexImage3D(GLenum Target, GLint Level, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLint Border, GLenum Format, GLenum Type, const GLvoid* PixelData)
	{
		glTexImage3D(Target, Level, InternalFormat, Width, Height, Depth, Border, Format, Type, PixelData);
	}

	static FORCEINLINE void CompressedTexImage3D(GLenum Target, GLint Level, GLenum InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLint Border, GLsizei ImageSize, const GLvoid* PixelData)
	{
		glCompressedTexImage3D(Target, Level, InternalFormat, Width, Height, Depth, Border, ImageSize, PixelData);
	}

	static FORCEINLINE void CompressedTexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLsizei ImageSize, const GLvoid* PixelData)
	{
		glCompressedTexSubImage3D( Target, Level, XOffset, YOffset, ZOffset, Width, Height, Depth, Format, ImageSize, PixelData);
	}

	static FORCEINLINE void TexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type, const GLvoid* PixelData)
	{
		glTexSubImage3D(Target, Level, XOffset, YOffset, ZOffset, Width, Height, Depth, Format, Type, PixelData);
	}

	static FORCEINLINE void	CopyTexSubImage1D(GLenum Target, GLint Level, GLint XOffset, GLint X, GLint Y, GLsizei Width)
	{
	}

	static FORCEINLINE void	CopyTexSubImage2D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint X, GLint Y, GLsizei Width, GLsizei Height)
	{
		glCopyTexSubImage2D(Target, Level, XOffset, YOffset, X, Y, Width, Height);
	}

	static FORCEINLINE void	CopyTexSubImage3D(GLenum Target, GLint Level, GLint XOffset, GLint YOffset, GLint ZOffset, GLint X, GLint Y, GLsizei Width, GLsizei Height)
	{
		glCopyTexSubImage3D(Target, Level, XOffset, YOffset, ZOffset, X, Y, Width, Height);
	}

	static FORCEINLINE void CopyImageSubData(GLuint SrcName, GLenum SrcTarget, GLint SrcLevel, GLint SrcX, GLint SrcY, GLint SrcZ, GLuint DstName, GLenum DstTarget, GLint DstLevel, GLint DstX, GLint DstY, GLint DstZ, GLsizei Width, GLsizei Height, GLsizei Depth)
	{
		glCopyImageSubData(SrcName, SrcTarget, SrcLevel, SrcX, SrcY, SrcZ, DstName, DstTarget, DstLevel, DstX, DstY, DstZ, Width, Height, Depth);
	}

	static FORCEINLINE bool TexStorage2DMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height, GLboolean FixedSampleLocations)
	{
		glTexStorage2DMultisample(Target, Samples, InternalFormat, Width, Height, FixedSampleLocations);
		return true;
	}

	static FORCEINLINE void RenderbufferStorageMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height)
	{
		check(glRenderbufferStorageMultisampleEXT);
		glRenderbufferStorageMultisampleEXT(Target, Samples, InternalFormat, Width, Height);
	}

	static FORCEINLINE void ClearBufferfv(GLenum Buffer, GLint DrawBufferIndex, const GLfloat* Value)
	{
		glClearBufferfv(Buffer, DrawBufferIndex, Value);
	}

	static FORCEINLINE void ClearBufferfi(GLenum Buffer, GLint DrawBufferIndex, GLfloat Depth, GLint Stencil)
	{
		glClearBufferfi(Buffer, DrawBufferIndex, Depth, Stencil);
	}

	static FORCEINLINE void ClearBufferiv(GLenum Buffer, GLint DrawBufferIndex, const GLint* Value)
	{
		glClearBufferiv(Buffer, DrawBufferIndex, Value);
	}

	static FORCEINLINE void DrawBuffers(GLsizei NumBuffers, const GLenum* Buffers)
	{
		glDrawBuffers(NumBuffers, Buffers);
	}

	static FORCEINLINE void ReadBuffer(GLenum Mode)
	{
		glReadBuffer(Mode);
	}

	static FORCEINLINE void DrawBuffer(GLenum Mode)
	{
		DrawBuffers(1, &Mode);
	}

	static FORCEINLINE void EnableIndexed(GLenum Parameter, GLuint Index)
	{
		if (bSupportsDrawBuffersBlend)
		{
			// ES 3.2 or extension
			glEnableiEXT(Parameter, Index);
		}
		else
		{
			check(Index == 0);
			glEnable(Parameter);
		}
	}
	static FORCEINLINE void DisableIndexed(GLenum Parameter, GLuint Index)
	{
		if (bSupportsDrawBuffersBlend)
		{
			// ES 3.2 or extension
			glDisableiEXT(Parameter, Index);
		}
		else
		{
			check(Index == 0);
			glDisable(Parameter);
		}
	}
	static FORCEINLINE void ColorMaskIndexed(GLuint Index, GLboolean Red, GLboolean Green, GLboolean Blue, GLboolean Alpha)
	{
		if (bSupportsDrawBuffersBlend)
		{
			// ES 3.2 or extension
			glColorMaskiEXT(Index, Red, Green, Blue, Alpha);
		}
		else
		{
			check(Index == 0);
			glColorMask(Red, Green, Blue, Alpha);
		}
	}
	static FORCEINLINE void BlendEquationi(GLuint Buf, GLenum Mode)
	{
		// ES 3.2 or extension
		glBlendEquationiEXT(Buf, Mode);
	}
	static FORCEINLINE void BlendEquationSeparatei(GLuint Buf, GLenum ModeRGB, GLenum ModeAlpha)
	{
		// ES 3.2 or extension
		glBlendEquationSeparateiEXT(Buf, ModeRGB, ModeAlpha);
	}
	static FORCEINLINE void BlendFunci(GLuint Buf, GLenum Src, GLenum Dst)
	{
		// ES 3.2 or extension
		glBlendFunciEXT(Buf, Src, Dst);
	}
	static FORCEINLINE void BlendFuncSeparatei(GLuint Buf, GLenum SrcRGB, GLenum DstRGB, GLenum SrcAlpha, GLenum DstAlpha)
	{
		// ES 3.2 or extension
		glBlendFuncSeparateiEXT(Buf, SrcRGB, DstRGB, SrcAlpha, DstAlpha);
	}

	static FORCEINLINE void TexBuffer(GLenum Target, GLenum InternalFormat, GLuint Buffer)
	{
		// ES 3.2 or extension
		glTexBufferEXT(Target, InternalFormat, Buffer);
	}

	static FORCEINLINE void TexBufferRange(GLenum Target, GLenum InternalFormat, GLuint Buffer, GLintptr Offset, GLsizeiptr Size)
	{
		// ES 3.2 or extension
		glTexBufferRangeEXT(Target, InternalFormat, Buffer, Offset, Size);
	}

	static FORCEINLINE void ProgramUniform4uiv(GLuint Program, GLint Location, GLsizei Count, const GLuint* Value)
	{
		glUniform4uiv(Location, Count, Value);
	}

	static FORCEINLINE bool SupportsProgramBinary() { return bSupportsProgramBinary; }

	static FORCEINLINE void GetProgramBinary(GLuint Program, GLsizei BufSize, GLsizei* Length, GLenum* BinaryFormat, void* Binary)
	{
		glGetProgramBinary(Program, BufSize, Length, BinaryFormat, Binary);
	}

	static FORCEINLINE void ProgramBinary(GLuint Program, GLenum BinaryFormat, const void* Binary, GLsizei Length)
	{
		glProgramBinary(Program, BinaryFormat, Binary, Length);
	}

	static FORCEINLINE void ProgramParameter(GLuint Program, GLenum PName, GLint Value)
	{
		glProgramParameteri(Program, PName, Value);
	}

	static FORCEINLINE void BindBufferBase(GLenum Target, GLuint Index, GLuint Buffer)
	{
		glBindBufferBase(Target, Index, Buffer);
	}

	static FORCEINLINE void BindBufferRange(GLenum Target, GLuint Index, GLuint Buffer, GLintptr Offset, GLsizeiptr Size)
	{
		glBindBufferRange(Target, Index, Buffer, Offset, Size);
	}

	static FORCEINLINE GLuint GetUniformBlockIndex(GLuint Program, const GLchar* UniformBlockName)
	{
		return glGetUniformBlockIndex(Program, UniformBlockName);
	}

	static FORCEINLINE void UniformBlockBinding(GLuint Program, GLuint UniformBlockIndex, GLuint UniformBlockBinding)
	{
		glUniformBlockBinding(Program, UniformBlockIndex, UniformBlockBinding);
	}

	static FORCEINLINE void BufferSubData(GLenum Target, GLintptr Offset, GLsizeiptr Size, const GLvoid* Data)
	{
		glBufferSubData(Target, Offset, Size, Data);
	}

	static FORCEINLINE void VertexAttribIPointer(GLuint Index, GLint Size, GLenum Type, GLsizei Stride, const GLvoid* Pointer)
	{
		glVertexAttribIPointer(Index, Size, Type, Stride, Pointer);
	}

	static FORCEINLINE void GenSamplers(GLsizei Count, GLuint* Samplers)
	{
		glGenSamplers(Count, Samplers);
	}

	static FORCEINLINE void DeleteSamplers(GLsizei Count, GLuint* Samplers)
	{
		glDeleteSamplers(Count, Samplers);
	}

	static FORCEINLINE void SetSamplerParameter(GLuint Sampler, GLenum Parameter, GLint Value)
	{
		glSamplerParameteri(Sampler, Parameter, Value);
	}

	static FORCEINLINE void BindSampler(GLuint Unit, GLuint Sampler)
	{
		glBindSampler(Unit, Sampler);
	}

	static FORCEINLINE void MemoryBarrier(GLbitfield Barriers)
	{
		glMemoryBarrier(Barriers);
	}

	static FORCEINLINE void DispatchCompute(GLuint NumGroupsX, GLuint NumGroupsY, GLuint NumGroupsZ)
	{
		glDispatchCompute(NumGroupsX, NumGroupsY, NumGroupsZ);
	}

	static FORCEINLINE void DispatchComputeIndirect(GLintptr Offset)
	{
		glDispatchComputeIndirect(Offset);
	}

	static FORCEINLINE void BindImageTexture(GLuint Unit, GLuint Texture, GLint Level, GLboolean Layered, GLint Layer, GLenum Access, GLenum Format)
	{
		glBindImageTexture(Unit, Texture, Level, Layered, Layer, Access, Format);
	}
	
	static FORCEINLINE void DepthRange(GLdouble Near, GLdouble Far)
	{
		glDepthRangef(Near, Far);
	}

	static FORCEINLINE void VertexAttribPointer(GLuint Index, GLint Size, GLenum Type, GLboolean Normalized, GLsizei Stride, const GLvoid* Pointer)
	{
		Size = (Size == GL_BGRA) ? 4 : Size;
		glVertexAttribPointer(Index, Size, Type, Normalized, Stride, Pointer);
	}

	static FORCEINLINE void ClearDepth(GLdouble Depth)
	{
		glClearDepthf(Depth);
	}

	static FORCEINLINE void GenerateMipmap( GLenum Target )
	{
		glGenerateMipmap( Target);
	}

	static FORCEINLINE GLuint GetMajorVersion()
	{
		return 3;
	}

	static FORCEINLINE GLuint GetMinorVersion()
	{
		return 1;
	}

	static FORCEINLINE EShaderPlatform GetShaderPlatform()
	{
		return SP_OPENGL_ES3_1_ANDROID;
	}

	static FORCEINLINE ERHIFeatureLevel::Type GetFeatureLevel()
	{
		return ERHIFeatureLevel::ES3_1;
	}

	static FORCEINLINE FString GetAdapterName()
	{
		return (TCHAR*)ANSI_TO_TCHAR((const ANSICHAR*)glGetString(GL_RENDERER));
	}

	static FORCEINLINE void TexParameter(GLenum Target, GLenum Parameter, GLint Value)
	{
		glTexParameteri(Target, Parameter, Value);
	}

	static FORCEINLINE void FramebufferTexture(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level)
	{
		// ES 3.2
		glFramebufferTexture(Target, Attachment, Texture, Level);
	}

	static FORCEINLINE void FramebufferTexture3D(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level, GLint ZOffset)
	{
		// glFramebufferTexture3D is not supported on GLES
		glFramebufferTextureLayer(Target, Attachment, Texture, Level, ZOffset);
	}

	static FORCEINLINE void FramebufferTextureLayer(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLint Layer)
	{
		glFramebufferTextureLayer(Target, Attachment, Texture, Level, Layer);
	}

	static FORCEINLINE void FramebufferTexture2D(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level)
	{
		check(Attachment == GL_COLOR_ATTACHMENT0 ||
				Attachment == GL_DEPTH_ATTACHMENT || 
				Attachment == GL_STENCIL_ATTACHMENT ||
				Attachment == GL_DEPTH_STENCIL_ATTACHMENT ||
				(Attachment >= GL_COLOR_ATTACHMENT0 && Attachment <= GL_COLOR_ATTACHMENT7));

		glFramebufferTexture2D(Target, Attachment, TexTarget, Texture, Level);
		VERIFY_GL(FramebufferTexture_2D);
	}

	static FORCEINLINE void FramebufferTexture2DMultisample(GLenum Target, GLenum Attachment, GLenum TexTarget, GLuint Texture, GLint Level, GLint NumSamples)
	{
		check(glFramebufferTexture2DMultisampleEXT != nullptr);
		glFramebufferTexture2DMultisampleEXT(Target, Attachment, TexTarget, Texture, Level, NumSamples);
	}

	static FORCEINLINE void FramebufferTextureMultiviewOVR(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLint BaseViewIndex, GLsizei NumViews)
	{
		check(glFramebufferTextureMultiviewOVR);
		glFramebufferTextureMultiviewOVR(Target, Attachment, Texture, Level, BaseViewIndex, NumViews);
	}
	
	static FORCEINLINE void FramebufferTextureMultisampleMultiviewOVR(GLenum Target, GLenum Attachment, GLuint Texture, GLint Level, GLsizei NumSamples, GLint BaseViewIndex, GLsizei NumViews)
	{
		check(glFramebufferTextureMultisampleMultiviewOVR);
		glFramebufferTextureMultisampleMultiviewOVR(Target, Attachment, Texture, Level, NumSamples, BaseViewIndex, NumViews);
	}

	static FORCEINLINE void BlitFramebuffer(GLint SrcX0, GLint SrcY0, GLint SrcX1, GLint SrcY1, GLint DstX0, GLint DstY0, GLint DstX1, GLint DstY1, GLbitfield Mask, GLenum Filter)
	{
		glBlitFramebuffer(SrcX0, SrcY0, SrcX1, SrcY1, DstX0, DstY0, DstX1, DstY1, Mask, Filter);
	}

	static FORCEINLINE bool TexStorage2D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLenum Format, GLenum Type, ETextureCreateFlags Flags)
	{
		glTexStorage2D(Target, Levels, InternalFormat, Width, Height);
		VERIFY_GL(glTexStorage2D);
		return true;
	}

	static FORCEINLINE void DrawArraysInstanced(GLenum Mode, GLint First, GLsizei Count, GLsizei InstanceCount)
	{
		glDrawArraysInstanced(Mode, First, Count, InstanceCount);
	}

	static FORCEINLINE void DrawElementsInstanced(GLenum Mode, GLsizei Count, GLenum Type, const GLvoid* Indices, GLsizei InstanceCount)
	{
		glDrawElementsInstanced(Mode, Count, Type, Indices, InstanceCount);
	}

	static FORCEINLINE void CopyBufferSubData(GLenum ReadTarget, GLenum WriteTarget, GLintptr ReadOffset, GLintptr WriteOffset, GLsizeiptr Size)
	{
		glCopyBufferSubData(ReadTarget, WriteTarget, ReadOffset, WriteOffset, Size);
	}

	static FORCEINLINE void DrawArraysIndirect(GLenum Mode, const void* Offset)
	{
		glDrawArraysIndirect(Mode, Offset);
	}

	static FORCEINLINE void DrawElementsIndirect(GLenum Mode, GLenum Type, const void* Offset)
	{
		glDrawElementsIndirect(Mode, Type, Offset);
	}

	static FORCEINLINE void VertexAttribDivisor(GLuint Index, GLuint Divisor)
	{
		glVertexAttribDivisor(Index, Divisor);
	}

	static FORCEINLINE void TexStorage3D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type)
	{
		glTexStorage3D(Target, Levels, InternalFormat, Width, Height, Depth);
	}

	static FPlatformOpenGLDevice* CreateDevice()	UGL_REQUIRED(NULL)
	static FPlatformOpenGLContext* CreateContext(FPlatformOpenGLDevice* Device, void* WindowHandle)	UGL_REQUIRED(NULL)

	static FORCEINLINE void InvalidateFramebuffer(GLenum Target, GLsizei NumAttachments, const GLenum* Attachments)
	{
		glInvalidateFramebuffer(Target, NumAttachments, Attachments);
	}

	static FORCEINLINE void GenBuffers(GLsizei n, GLuint* buffers)
	{
		glGenBuffers(n, buffers);
	}

	static FORCEINLINE void GenTextures(GLsizei n, GLuint* textures)
	{
		glGenTextures(n, textures);
	}

	static FORCEINLINE bool TimerQueryDisjoint()
	{
		bool Disjoint = false;

		if (bTimerQueryCanBeDisjoint)
		{
			GLint WasDisjoint = 0;
			glGetIntegerv(GL_GPU_DISJOINT_EXT, &WasDisjoint);
			Disjoint = (WasDisjoint != 0);
		}

		return Disjoint;
	}

	static FORCEINLINE void BindVertexBuffer(GLuint BindingIndex, GLuint Buffer, GLintptr Offset, GLsizei Stride)
	{
		glBindVertexBuffer(BindingIndex, Buffer, Offset, Stride);
	}
	static FORCEINLINE void VertexAttribFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLboolean Normalized, GLuint RelativeOffset)
	{
		glVertexAttribFormat(AttribIndex, Size, Type, Normalized, RelativeOffset);
	}
	static FORCEINLINE void VertexAttribIFormat(GLuint AttribIndex, GLint Size, GLenum Type, GLuint RelativeOffset)
	{
		glVertexAttribIFormat(AttribIndex, Size, Type, RelativeOffset);
	}
	static FORCEINLINE void VertexAttribBinding(GLuint AttribIndex, GLuint BindingIndex)
	{
		glVertexAttribBinding(AttribIndex, BindingIndex);
	}
	static FORCEINLINE void VertexBindingDivisor(GLuint BindingIndex, GLuint Divisor)
	{
		glVertexBindingDivisor(BindingIndex, Divisor);
	}

	static FORCEINLINE void BufferStorage(GLenum Target, GLsizeiptr Size, const void* Data, GLbitfield Flags)
	{
		glBufferStorageEXT(Target, Size, Data, Flags);
	}
protected:
	/** GL_EXT_disjoint_timer_query */
	static bool bSupportsDisjointTimeQueries;

	/** Some timer query implementations are never disjoint */
	static bool bTimerQueryCanBeDisjoint;

	/** GL_APPLE_texture_format_BGRA8888 */
	static bool bSupportsBGRA8888;

	/** OpenGL ES 3.0 profile */
	static bool bSupportsETC2;

	/** GL_EXT_color_buffer_float */
	static bool bSupportsColorBufferFloat;

	/** GL_EXT_color_buffer_half_float */
	static bool bSupportsColorBufferHalfFloat;

	/** GL_EXT_shader_framebuffer_fetch */
	static bool bSupportsShaderFramebufferFetch;

	/** GL_EXT_shader_framebuffer_fetch (MRT's) */
	static bool bSupportsShaderMRTFramebufferFetch;


	/** GL_ARM_shader_framebuffer_fetch_depth_stencil */
	static bool bSupportsShaderDepthStencilFetch;

	/** GL_EXT_MULTISAMPLED_RENDER_TO_TEXTURE */
	static bool bSupportsMultisampledRenderToTexture;

	/** workaround for GL_EXT_shader_pixel_local_storage */
	static bool bSupportsPixelLocalStorage;

	/** GL_FRAGMENT_SHADER, GL_LOW_FLOAT */
	static int ShaderLowPrecision;

	/** GL_FRAGMENT_SHADER, GL_MEDIUM_FLOAT */
	static int ShaderMediumPrecision;

	/** GL_FRAGMENT_SHADER, GL_HIGH_FLOAT */
	static int ShaderHighPrecision;

	/** GL_NV_framebuffer_blit */
	static bool bSupportsNVFrameBufferBlit;

	/** GL_OES_standard_derivations */
	static bool bSupportsStandardDerivativesExtension;

	/** Maximum number of MSAA samples supported on chip in tile memory, or 1 if not available */
	static GLint MaxMSAASamplesTileMem;

	/** GL_EXT_texture_compression_astc_decode_mode */
	static bool bSupportsASTCDecodeMode;

public:
	/* This indicates failure when attempting to retrieve driver's binary representation of the hack program  */
	static bool bBinaryProgramRetrievalFailed;

	/* Some Mali devices do not work correctly with early_fragment_test enabled */
	static bool bRequiresDisabledEarlyFragmentTests;
		
	/* This is a workaround for a Mali bug where read-only buffers do not work when passed to functions*/
	static bool bRequiresReadOnlyBuffersWorkaround;

	/* This is a workaround for Samsung xclipse that fails to compile shaders that use the 'precise' qualifier*/
	static bool bRequiresPreciseQualifierWorkaround;

	/* This is to avoid a bug in Adreno drivers that define GL_ARM_shader_framebuffer_fetch_depth_stencil even when device does not support this extension  */
	static bool bRequiresARMShaderFramebufferFetchDepthStencilUndef;

	/** Framebuffer fetch can be used to do programmable blending without running into driver issues */
	static bool bSupportsShaderFramebufferFetchProgrammableBlending;

	/** GL_OES_vertex_type_10_10_10_2 */
	static bool bSupportsRGB10A2;

	/** GL_OES_get_program_binary */
	static bool bSupportsProgramBinary;

	/** GL_EXT_buffer_storage */
	static bool bSupportsBufferStorage;

	/** GL_EXT_depth_clamp */
	static bool bSupportsDepthClamp;

	enum class EFeatureLevelSupport : uint8
	{
		Invalid,	// no feature level has yet been determined
		ES2,
		ES31,
		ES32
	};

	/** Describes which feature level is currently being supported */
	static EFeatureLevelSupport CurrentFeatureLevelSupport;

	/** Whether device supports Hidden Surface Removal */
	static bool bHasHardwareHiddenSurfaceRemoval;

	/** Whether device supports mobile multi-view */
	static bool bSupportsMobileMultiView;

	static GLint MaxComputeUniformComponents;

	static GLint MaxCombinedUAVUnits;
	static GLint MaxComputeUAVUnits;
	static GLint MaxPixelUAVUnits;
};

#endif // UGL_PLATFORM_SUPPORTS_GLES
