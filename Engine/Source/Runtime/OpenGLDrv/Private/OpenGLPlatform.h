// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PreprocessorHelpers.h"

#include COMPILED_PLATFORM_HEADER(OpenGLPlatform.h)

using UGLsync = GLsync;

/** Official OpenGL definitions */
#ifndef GL_HALF_FLOAT
#define GL_HALF_FLOAT 0x140B
#endif

#ifndef GL_RGBA16F
#define GL_RGBA16F    0x881A
#endif

/** OpenGL extensions */

// http://www.opengl.org/registry/specs/EXT/texture_compression_s3tc.txt
#if !defined(GL_EXT_texture_compression_s3tc)
#define GL_COMPRESSED_RGB_S3TC_DXT1_EXT			0x83F0
#define GL_COMPRESSED_RGBA_S3TC_DXT1_EXT		0x83F1
#define GL_COMPRESSED_RGBA_S3TC_DXT3_EXT		0x83F2
#define GL_COMPRESSED_RGBA_S3TC_DXT5_EXT		0x83F3
#endif

// http://www.opengl.org/registry/specs/EXT/texture_sRGB.txt
#if !defined(GL_EXT_texture_sRGB)
#define GL_SRGB_EXT                       0x8C40
#define GL_SRGB8_EXT                      0x8C41
#define GL_SRGB_ALPHA_EXT                 0x8C42
#define GL_SRGB8_ALPHA8_EXT               0x8C43
#define GL_SLUMINANCE_ALPHA_EXT           0x8C44
#define GL_SLUMINANCE8_ALPHA8_EXT         0x8C45
#define GL_SLUMINANCE_EXT                 0x8C46
#define GL_SLUMINANCE8_EXT                0x8C47
#define GL_COMPRESSED_SRGB_EXT            0x8C48
#define GL_COMPRESSED_SRGB_ALPHA_EXT      0x8C49
#define GL_COMPRESSED_SLUMINANCE_EXT      0x8C4A
#define GL_COMPRESSED_SLUMINANCE_ALPHA_EXT 0x8C4B
#define GL_COMPRESSED_SRGB_S3TC_DXT1_EXT  0x8C4C
#define GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT1_EXT 0x8C4D
#define GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT3_EXT 0x8C4E
#define GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT5_EXT 0x8C4F
#endif

// http://www.opengl.org/registry/specs/ARB/texture_compression_rgtc.txt
#if !defined(GL_ARB_texture_compression_rgtc)
#define GL_COMPRESSED_RED_RGTC1           0x8DBB
#define GL_COMPRESSED_SIGNED_RED_RGTC1    0x8DBC
#define GL_COMPRESSED_RG_RGTC2            0x8DBD
#define GL_COMPRESSED_SIGNED_RG_RGTC2     0x8DBE
#endif

/* http://www.khronos.org/registry/gles/extensions/NV/NV_sRGB_formats.txt */
#if !defined(GL_NV_sRGB_formats)
#define GL_SLUMINANCE_NV                                        0x8C46
#define GL_SLUMINANCE_ALPHA_NV                                  0x8C44
#define GL_SRGB8_NV                                             0x8C41
#define GL_SLUMINANCE8_NV                                       0x8C47
#define GL_SLUMINANCE8_ALPHA8_NV                                0x8C45
#define GL_COMPRESSED_SRGB_S3TC_DXT1_NV                         0x8C4C
#define GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT1_NV                   0x8C4D
#define GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT3_NV                   0x8C4E
#define GL_COMPRESSED_SRGB_ALPHA_S3TC_DXT5_NV                   0x8C4F
#endif

// http://www.opengl.org/registry/specs/KHR/texture_compression_astc_ldr.txt
#if !defined(GL_KHR_texture_compression_astc_ldr)
#define GL_COMPRESSED_RGBA_ASTC_4x4_KHR            0x93B0
#define GL_COMPRESSED_RGBA_ASTC_5x4_KHR            0x93B1
#define GL_COMPRESSED_RGBA_ASTC_5x5_KHR            0x93B2
#define GL_COMPRESSED_RGBA_ASTC_6x5_KHR            0x93B3
#define GL_COMPRESSED_RGBA_ASTC_6x6_KHR            0x93B4
#define GL_COMPRESSED_RGBA_ASTC_8x5_KHR            0x93B5
#define GL_COMPRESSED_RGBA_ASTC_8x6_KHR            0x93B6
#define GL_COMPRESSED_RGBA_ASTC_8x8_KHR            0x93B7
#define GL_COMPRESSED_RGBA_ASTC_10x5_KHR           0x93B8
#define GL_COMPRESSED_RGBA_ASTC_10x6_KHR           0x93B9
#define GL_COMPRESSED_RGBA_ASTC_10x8_KHR           0x93BA
#define GL_COMPRESSED_RGBA_ASTC_10x10_KHR          0x93BB
#define GL_COMPRESSED_RGBA_ASTC_12x10_KHR          0x93BC
#define GL_COMPRESSED_RGBA_ASTC_12x12_KHR          0x93BD

#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_4x4_KHR    0x93D0
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_5x4_KHR    0x93D1
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_5x5_KHR    0x93D2
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_6x5_KHR    0x93D3
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_6x6_KHR    0x93D4
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_8x5_KHR    0x93D5
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_8x6_KHR    0x93D6
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_8x8_KHR    0x93D7
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_10x5_KHR   0x93D8
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_10x6_KHR   0x93D9
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_10x8_KHR   0x93DA
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_10x10_KHR  0x93DB
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_12x10_KHR  0x93DC
#define GL_COMPRESSED_SRGB8_ALPHA8_ASTC_12x12_KHR  0x93DD
#endif

#if !defined(GL_TESS_EVALUATION_SHADER)
#define GL_TESS_EVALUATION_SHADER					0x8E87
#endif
#if !defined(GL_TESS_CONTROL_SHADER)
#define GL_TESS_CONTROL_SHADER						0x8E88
#endif
#if !defined(GL_PATCHES)
#define GL_PATCHES									0x000E
#endif
#if !defined(GL_PATCH_VERTICES)
#define GL_PATCH_VERTICES							0x8E72
#endif
#if !defined(GL_MAX_GEOMETRY_TEXTURE_IMAGE_UNITS)
#define GL_MAX_GEOMETRY_TEXTURE_IMAGE_UNITS			0x8C29
#endif
#if !defined(GL_MAX_TESS_CONTROL_TEXTURE_IMAGE_UNITS)
#define GL_MAX_TESS_CONTROL_TEXTURE_IMAGE_UNITS		0x8E81
#endif
#if !defined(GL_MAX_TESS_EVALUATION_TEXTURE_IMAGE_UNITS)
#define GL_MAX_TESS_EVALUATION_TEXTURE_IMAGE_UNITS	0x8E82
#endif
#if !defined(GL_READ_WRITE)
#define GL_READ_WRITE								0x88BA
#endif
#if !defined(GL_ALL_BARRIER_BITS)
#define GL_ALL_BARRIER_BITS							0xFFFFFFFF
#endif
#if !defined(GL_TEXTURE_CUBE_MAP_ARRAY)
#define GL_TEXTURE_CUBE_MAP_ARRAY					0x9009
#endif
#if !defined(GL_UNIFORM_BLOCK_REFERENCED_BY_TESS_CONTROL_SHADER)
#define GL_UNIFORM_BLOCK_REFERENCED_BY_TESS_CONTROL_SHADER		0x84F0
#endif
#if !defined(GL_UNIFORM_BLOCK_REFERENCED_BY_TESS_EVALUATION_SHADER)
#define GL_UNIFORM_BLOCK_REFERENCED_BY_TESS_EVALUATION_SHADER	0x84F1
#endif
#if !defined(GL_UNIFORM_BLOCK_REFERENCED_BY_COMPUTE_SHADER)
#define GL_UNIFORM_BLOCK_REFERENCED_BY_COMPUTE_SHADER			0x90EC
#endif
#ifndef GL_ARB_seamless_cube_map
#define GL_TEXTURE_CUBE_MAP_SEAMLESS				0x884F
#endif
#ifndef GL_TIME_ELAPSED_EXT
#define GL_TIME_ELAPSED_EXT							0x88BF
#endif
#ifndef GL_TIMESTAMP_EXT
#define GL_TIMESTAMP_EXT							0x8E28
#endif
#ifndef GL_DISPATCH_INDIRECT_BUFFER
#define GL_DISPATCH_INDIRECT_BUFFER					0x90EE
#endif
#ifndef GL_DRAW_INDIRECT_BUFFER
#define GL_DRAW_INDIRECT_BUFFER						0x8F3F
#endif
#ifndef GL_MAP_WRITE_BIT
#define GL_MAP_WRITE_BIT							0x0002
#endif
#ifndef GL_MAP_PERSISTENT_BIT
#define GL_MAP_PERSISTENT_BIT						0x0040
#endif
#ifndef GL_MAP_COHERENT_BIT
#define GL_MAP_COHERENT_BIT							0x0080
#endif
#ifndef GL_DEPTH_BOUNDS_TEST_EXT
#define GL_DEPTH_BOUNDS_TEST_EXT					0x8890
#endif
#ifndef GL_DEPTH_STENCIL_TEXTURE_MODE
#define GL_DEPTH_STENCIL_TEXTURE_MODE				0x90EA
#endif
#ifndef GL_TEXTURE_SWIZZLE_R
#define GL_TEXTURE_SWIZZLE_R						0x8E42
#endif
#ifndef GL_TEXTURE_SWIZZLE_G
#define GL_TEXTURE_SWIZZLE_G						0x8E43
#endif
#ifndef GL_TEXTURE_SWIZZLE_B
#define GL_TEXTURE_SWIZZLE_B						0x8E44
#endif
#ifndef GL_TEXTURE_SWIZZLE_A
#define GL_TEXTURE_SWIZZLE_A						0x8E45
#endif
#ifndef GL_RED
#define GL_RED										0x1903
#endif
#ifndef GL_BLUE
#define GL_BLUE										0x1905
#endif
#ifndef GL_STENCIL_INDEX
#define GL_STENCIL_INDEX							0x1901
#endif
#ifndef GL_RGBA_INTEGER
#define GL_RGBA_INTEGER								0x8D99
#endif

#ifndef GL_ARB_compute_shader
#define GL_COMPUTE_SHADER                 0x91B9
#define GL_MAX_COMPUTE_UNIFORM_BLOCKS     0x91BB
#define GL_MAX_COMPUTE_TEXTURE_IMAGE_UNITS 0x91BC
#define GL_MAX_COMPUTE_IMAGE_UNIFORMS     0x91BD
#define GL_MAX_COMPUTE_SHARED_MEMORY_SIZE 0x8262
#define GL_MAX_COMPUTE_UNIFORM_COMPONENTS 0x8263
#define GL_MAX_COMPUTE_ATOMIC_COUNTER_BUFFERS 0x8264
#define GL_MAX_COMPUTE_ATOMIC_COUNTERS    0x8265
#define GL_MAX_COMBINED_COMPUTE_UNIFORM_COMPONENTS 0x8266
#define GL_MAX_COMPUTE_LOCAL_INVOCATIONS  0x90EB
#define GL_MAX_COMPUTE_WORK_GROUP_COUNT   0x91BE
#define GL_MAX_COMPUTE_WORK_GROUP_SIZE    0x91BF
#define GL_COMPUTE_LOCAL_WORK_SIZE        0x8267
#define GL_UNIFORM_BLOCK_REFERENCED_BY_COMPUTE_SHADER 0x90EC
#define GL_ATOMIC_COUNTER_BUFFER_REFERENCED_BY_COMPUTE_SHADER 0x90ED
#define GL_DISPATCH_INDIRECT_BUFFER       0x90EE
#define GL_DISPATCH_INDIRECT_BUFFER_BINDING 0x90EF
#define GL_COMPUTE_SHADER_BIT             0x00000020
#endif

#ifndef GL_GPU_DISJOINT_EXT
#define GL_GPU_DISJOINT_EXT							0x8FBB
#endif

#ifndef GL_APPLE_client_storage
#define GL_UNPACK_CLIENT_STORAGE_APPLE				0x85B2
#endif

#ifndef GL_APPLE_texture_range
#define GL_TEXTURE_RANGE_LENGTH_APPLE     0x85B7
#define GL_TEXTURE_RANGE_POINTER_APPLE    0x85B8
#define GL_TEXTURE_STORAGE_HINT_APPLE     0x85BC
#define GL_TEXTURE_MINIMIZE_STORAGE_APPLE 0x85B6
#define GL_STORAGE_PRIVATE_APPLE          0x85BD
#define GL_STORAGE_CACHED_APPLE           0x85BE
#define GL_STORAGE_SHARED_APPLE           0x85BF
#endif

#ifndef TEXTURE_ASTC_DECODE_PRECISION_EXT
#define TEXTURE_ASTC_DECODE_PRECISION_EXT 0x8F69
#endif

#ifndef GL_EXT_shader_pixel_local_storage
#define GL_MAX_SHADER_PIXEL_LOCAL_STORAGE_FAST_SIZE_EXT 0x8F63
#define GL_MAX_SHADER_PIXEL_LOCAL_STORAGE_SIZE_EXT 0x8F67
#define GL_SHADER_PIXEL_LOCAL_STORAGE_EXT 0x8F64
#endif
