// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLProgramBinaryFileCache.h: OpenGL program binary file cache stores/loads a set of binary ogl programs.
=============================================================================*/

#pragma once

#include "OpenGLShaderResources.h"
#include "ShaderPipelineCache.h"

class FOpenGLLinkedProgram;
class FOpenGLProgramBinary;
class FOpenGLProgramBinaryMapping;
struct FGLProgramBinaryFileCacheEntry;

class FOpenGLProgramBinaryCache
{
public:
	static void Initialize();
	static void Shutdown();

	static bool IsEnabled();

	/** has the program already been encountered */
	static bool RequiresCaching(const FOpenGLProgramKey& ProgramKey);

	/** Take an existing program binary and store on disk. Only when ProgramBinaryCache is enabled
	*/
	static void CacheProgramBinary(const FOpenGLProgramKey& ProgramKey, TUniqueObj<FOpenGLProgramBinary>&& ProgramBinary);

	/** Create any pending GL programs that have come from shader library requests */
	static void TickBinaryCache();

	/** Create any single GL program that have come from shader library requests */
	static bool CheckSinglePendingGLProgramCreateRequest(const FOpenGLProgramKey& ProgramKey);

	/** true if the program binary cache is currently in cache build mode */
	static bool IsBuildingCache();

	/** Add a binary program to the GL container queue, it is moved at a later point to the GL program container */
	static void EnqueueBinaryForGLProgramContainer(const FOpenGLProgramKey& ProgramKey, TUniqueObj<FOpenGLProgramBinary>&& ProgramBinary);

private:
	FOpenGLProgramBinaryCache(const FString& InCachePath);
	~FOpenGLProgramBinaryCache();

	FString GetProgramBinaryCacheFilePath() const;

	struct FPendingShaderCode
	{
		TArray<ANSICHAR> GlslCode;
		int32 UncompressedSize;
		bool bCompressed;
	};

private:
	static TAutoConsoleVariable<int32> CVarPBCEnable;
	static TAutoConsoleVariable<int32> CVarRestartAndroidAfterPrecompile;
	static FOpenGLProgramBinaryCache* CachePtr;

	// Binary program file layout is <osfilelocation>/CachePathRoot/CacheSubDir/CurrentShaderPipelineCacheName

	/*  Path to root directory where the binary program subdir will be created */
	FString CachePathRoot;
	/* Subdir is a hash of the device's version info, anything within CachePathRoot that does not == CacheSubDir is deleted. */
	FString CacheSubDir;

	struct FCurrentShaderPipelineProperties
	{
		// Guid of the PSO cache being procesed.
		FGuid CacheVersionGuid;
		// String name of the current cache.
		FString PipelineCacheName;

		// contains the count of programs when the write cache was last flushed.
		int32 NumProgramsFlushed = 0;
		// Last mmapped position
		int64 LastMappedPosition = 0;
		// Last number of programs mapped
		int32 MappedPrograms = 0;
	};
	FCurrentShaderPipelineProperties CurrentShaderPipelineProperties;

	/**
	* Shaders that were requested for compilation
	* They will be compiled just before linking a program only in case when there is no saved binary program
	*/
	TMap<GLuint, FPendingShaderCode> ShadersPendingCompilation;

	void Reset();

	void ScanProgramCacheFile();

	bool OpenCacheWriteHandle(const FString& ProgramCacheFilenameToWrite, bool bAppendToExisting);

	// set the file header to the current write position and program count, then flush the cache to storage.
	bool MarkValidContent(int32 NumPrograms);

	// finalize the current cache file, returns true if a valid file was created.
	bool CloseCacheWriteHandle(int32 NumProgramsAdded); 

	bool RequiresCaching_Internal(const FOpenGLProgramKey& ProgramKey);

	void AddProgramBinaryDataToBinaryCache(const FOpenGLProgramKey& ProgramKey, const FOpenGLProgramBinary& BinaryProgramData);

	void CheckPendingGLProgramCreateRequests_internal();
	bool CheckSinglePendingGLProgramCreateRequest_internal(const FOpenGLProgramKey& ProgramKey);

	/** Delegate handlers to track the ShaderPipelineCache precompile. */
	void OnShaderPipelineCacheOpened(FString const& Name, EShaderPlatform Platform, uint32 Count, const FGuid& VersionGuid, FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext);
	void OnShaderPipelineCachePrecompilationComplete(uint32 Count, double Seconds, const FShaderPipelineCache::FShaderCachePrecompileContext& ShaderCachePrecompileContext);

	void InitPrecaching();
	void UpdatePrecacheMapping();

private:
	TRefCountPtr<FOpenGLProgramBinaryMapping> GetOrAddFileMapping(const FString& ProgramCacheFilename, int32 ProgramCount, int64 Offset, int64 Size);

	FString GetProgramBinaryCacheDir() const { return (CachePathRoot / CacheSubDir);	}
	bool ReadProgramFile_Internal(uint32 ProgramsToRead, int64 EndOffset, const FString& ProgramCacheFilename, FArchive& Ar);

	FDelegateHandle OnShaderPipelineCacheOpenedDelegate;
	FDelegateHandle OnShaderPipelineCachePrecompilationCompleteDelegate;

	TSet<FOpenGLProgramKey> ProgramsInCurrentCache; // Only populated during PSO precompile delegates.

	// All of the binary programs encountered during PSO cache processing.
	// The contents are moved to OpenGL's program cache during RHI's end of frame tick.
	TMap<FOpenGLProgramKey, TUniqueObj<FOpenGLProgramBinary>> PendingGLContainerPrograms;

	FArchive* BinaryCacheWriteFileHandle;
	bool bShownLoadingScreen;

	enum class EBinaryFileState : uint8
	{
		Uninitialized,					// No binary file is yet established and we should not read or write to it.
		BuildingCacheFile,				// We are precompiling shaders from the PSO and storing them in a new binary cache. Do not attempt to read.
		ValidCacheFile,					// We have a valid cache file we can use for reading.
	};

	bool IsBuildingCache_internal() const	{ return CurrentBinaryFileState == EBinaryFileState::BuildingCacheFile; }

	EBinaryFileState CurrentBinaryFileState;

	// Container of all currently open program cache files.
	TMap<FGuid, TRefCountPtr<FOpenGLProgramBinaryMapping>> MappedCacheFiles;
};
