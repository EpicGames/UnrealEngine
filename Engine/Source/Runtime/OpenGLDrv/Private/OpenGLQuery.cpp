// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLQuery.cpp: OpenGL query RHI implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "RenderCore.h"

FOpenGLRenderQuery::FActiveQueries FOpenGLRenderQuery::ActiveQueries;
FOpenGLRenderQuery::FQueryPool FOpenGLRenderQuery::PooledQueries;

FOpenGLRenderQuery::~FOpenGLRenderQuery()
{
	VERIFY_GL_SCOPE();
	ReleaseGlQuery();
}

void FOpenGLRenderQuery::Link()
{
	// The renderer might re-use a query without reading its results back first.
	// Ensure this query is unlinked, so it can be re-linked at the end of the list.
	Unlink();

	if (!ActiveQueries.First)
	{
		check(!ActiveQueries.Last);
		check(Next == nullptr);

		ActiveQueries.First = this;
		Prev = &ActiveQueries.First;
	}
	else
	{
		check(ActiveQueries.Last);
		check(ActiveQueries.Last->Next == nullptr);

		ActiveQueries.Last->Next = this;
		Prev = &ActiveQueries.Last->Next;
	}

	ActiveQueries.Last = this;
}

void FOpenGLRenderQuery::Unlink()
{
	if (!IsLinked())
		return;

	if (ActiveQueries.Last == this)
	{
		// This is the last node in the list, so the "ActiveQueries.Last" pointer needs fixing up.
		if (Prev == &ActiveQueries.First)
		{
			// This is also the first node in the list, meaning there's only 1 node total.
			// Just clear the "ActiveQueries.Last" pointer.
			ActiveQueries.Last = nullptr;
		}
		else
		{
			//
			// There's at least one real node before us.
			// 
			// "Prev" points to the "Next" member field of the previous node.
			// Subtract the "Next" field offset to get the actual previous node address.
			//
			ActiveQueries.Last = reinterpret_cast<FOpenGLRenderQuery*>(reinterpret_cast<uintptr_t>(Prev) - offsetof(FOpenGLRenderQuery, Next));
		}
	}

	if (Next) { Next->Prev = Prev; }
	if (Prev) { *Prev = Next; }

	Next = nullptr;
	Prev = nullptr;
}

void FOpenGLRenderQuery::AcquireGlQuery()
{
	VERIFY_GL_SCOPE();

	if (Resource != 0)
	{
		// Already acquired
		return;
	}

	while (ActiveQueries.First && ActiveQueries.Count >= GRHIMaximumInFlightQueries)
	{
		// We can't start another query until more become available, due to the query count limit.
		// Block for results on the oldest in-flight queries.
		ActiveQueries.First->CacheResult(true);
	}

	ActiveQueries.Count++;

	if (PooledQueries[Type].Num())
	{
		Resource = PooledQueries[Type].Pop();
	}
	else
	{
		FOpenGL::GenQueries(1, &Resource);
	}
}

void FOpenGLRenderQuery::ReleaseGlQuery()
{
	VERIFY_GL_SCOPE();

	if (Resource == 0)
	{
		// Already released
		check(!IsLinked());
		return;
	}

	check(ActiveQueries.Count > 0);
	ActiveQueries.Count--;

	PooledQueries[Type].Add(Resource);

	Resource = 0;

	Unlink();
}

void FOpenGLRenderQuery::Begin()
{
	VERIFY_GL_SCOPE();

	check(!IsLinked());
	AcquireGlQuery();

	check(Resource);

	switch(Type)
	{
	default:
		checkNoEntry();
		break;

	case EType::Occlusion:
		FOpenGL::BeginQuery(
			FOpenGL::SupportsExactOcclusionQueries()
				? UGL_SAMPLES_PASSED
				: UGL_ANY_SAMPLES_PASSED
			, Resource
		);
		break;
	};
}

void FOpenGLRenderQuery::End(uint64* InTarget)
{
	VERIFY_GL_SCOPE();
	AcquireGlQuery();
	
	check(Resource);

	switch (Type)
	{
	case EType::Occlusion:
		check(Resource);
		FOpenGL::EndQuery(FOpenGL::SupportsExactOcclusionQueries()
			? UGL_SAMPLES_PASSED
			: UGL_ANY_SAMPLES_PASSED
		);
		break;

	case EType::Timestamp:
		FOpenGL::QueryTimestampCounter(Resource);
		break;

	case EType::Profiler:
		FOpenGL::QueryTimestampCounter(Resource);
		break;
	}

	BOPCounter++;

	Target = InTarget;

	Link();
}

bool FOpenGLRenderQuery::IsCached()
{
	return BOPCounter == LastCachedBOPCounter.load(std::memory_order_relaxed);
}

bool FOpenGLRenderQuery::CacheResult(bool bWait)
{
	VERIFY_GL_SCOPE();

	if (IsCached())
	{
		// Value has been cached and no newer query operation has started.
		check(!IsLinked());
		return true;
	}

	check(Resource);

	if (!bWait)
	{
		// If we don't want to wait, we need to check if the result is available first.
		GLuint IsAvailable = GL_FALSE;
		FOpenGL::GetQueryObject(Resource, FOpenGL::QM_ResultAvailable, &IsAvailable);

		if (IsAvailable == GL_FALSE)
		{
			// Not ready yet.
			return false;
		}
	}

	// Read the result back (and block if its not ready)
	switch (Type)
	{
	default:
		checkNoEntry();
		break;
	
	case EType::Occlusion:
		{
			GLuint Result32 = 0;
			FOpenGL::GetQueryObject(Resource, FOpenGL::QM_Result, &Result32);
			SetResult(Result32 * (FOpenGL::SupportsExactOcclusionQueries() ? 1 : 500000)); // half a mega pixel display
		}
		break;

	case EType::Timestamp:
		{
			GLuint64 Value = 0;
			FOpenGL::GetQueryObject(Resource, FOpenGL::QM_Result, &Value);

			// Convert to microseconds (GL queries are in nanoseconds)
			SetResult(Value / 1000);
		}
		break;

	case EType::Profiler:
		{
			FOpenGLDynamicRHI& RHI = FOpenGLDynamicRHI::Get();

			// TimerQueryDisjoint is a one-shot state in the driver, it is not pipelined.
			// If it returns true, all timers we've submitted after this timer but haven't
			// yet resolved should be discarded for having invalid data.
			if (FOpenGL::TimerQueryDisjoint())
			{
				for (FOpenGLRenderQuery* Other = this; Other; Other = Other->Next)
				{
					if (Other->Type == EType::Profiler)
					{
						Other->SetResult(RHI.Profiler.ResolveQuery(0, Other->Target, true));

						// Return the query to the pool
						RHI.Profiler.QueryPool.Push(Other);
					}
				}
			}
			else
			{
				GLuint64 Value;
				FOpenGL::GetQueryObject(Resource, FOpenGL::QM_Result, &Value);
				SetResult(RHI.Profiler.ResolveQuery(Value, Target, false));

				// Return the query to the pool
				RHI.Profiler.QueryPool.Push(this);
			}
		}
		break;
	}

	return true;
}

void FOpenGLRenderQuery::SetResult(uint64 Value)
{
	if (Target)
	{
		*Target = Value;
		Target = nullptr;
	}

	Result = Value;
	ReleaseGlQuery();

	LastCachedBOPCounter.store(BOPCounter, std::memory_order_release);
}

bool FOpenGLRenderQuery_RHI::GetResult(bool bWait, uint64& OutResult)
{
	if (TOPCounter == LastCachedBOPCounter.load(std::memory_order_acquire))
	{
		// Early return for queries we already have the result for.
		check(!IsLinked());
		OutResult = FOpenGLRenderQuery::GetResult();
		return true;
	}

	if (!bWait)
	{
		//
		// The query has not yet completed, and we don't want to wait for the query result.
		// Return. The RHI thread will poll for results later.
		//
		OutResult = 0;
		return false;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	//
	// The query has not yet completed, and we want to wait for results.
	// Append an RHI thread command that will force a readback of the GL query, then flush the RHI thread.
	//	
	RHICmdList.EnqueueLambda([this](FRHICommandListImmediate&)
	{
		// Fetch the query result if it hasn't arrived yet...
		CacheResult(true);
	});

	// Wait for the above lambda to execute
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

	checkf(TOPCounter == LastCachedBOPCounter, TEXT("Attempting to get data from an RHI render query which was never issued."));
	check(!IsLinked());

	OutResult = FOpenGLRenderQuery::GetResult();
	return true;
}

bool FOpenGLRenderQuery::PollQueryResults(FOpenGLRenderQuery* TargetQuery)
{
	if (!PlatformOpenGLThreadHasRenderingContext())
	{
		// Don't poll queries if this thread doesn't own the GL context.
		return false;
	}

	if (TargetQuery && TargetQuery->IsCached())
	{
		return true;
	}

	if (ActiveQueries.First)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PollQueryResults);

		do
		{
			FOpenGLRenderQuery* Current = ActiveQueries.First;
			if (!Current->CacheResult(/*bWait = */ false))
			{
				// Not complete yet
				return false;
			}

			if (Current == TargetQuery)
			{
				return true;
			}
		}
		while (ActiveQueries.First);
	}

	return TargetQuery == nullptr;
}

void FOpenGLRenderQuery::Cleanup()
{
	VERIFY_GL_SCOPE();
	check(ActiveQueries.Count == 0);

	for (auto& Array : PooledQueries)
	{
		for (GLuint Resource : Array)
		{
			FOpenGL::DeleteQueries(1, &Resource);
		}

		Array.Reset();
	}
}

FRenderQueryRHIRef FOpenGLDynamicRHI::RHICreateRenderQuery(ERenderQueryType QueryType)
{
	check(QueryType == RQT_Occlusion || QueryType == RQT_AbsoluteTime);
	if (QueryType == RQT_AbsoluteTime && FOpenGL::SupportsTimestampQueries() == false)
	{
		return nullptr;
	}

	return new FOpenGLRenderQuery_RHI(QueryType);
}

void FOpenGLDynamicRHI::RHIBeginRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	if (!RenderQuery)
		return;

	FDynamicRHI::RHIBeginRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FOpenGLDynamicRHI::RHIEndRenderQuery_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIRenderQuery* RenderQuery)
{
	if (!RenderQuery)
		return;

	ResourceCast(RenderQuery)->End_TopOfPipe();
	FDynamicRHI::RHIEndRenderQuery_TopOfPipe(RHICmdList, RenderQuery);
}

void FOpenGLDynamicRHI::RHIBeginRenderQuery(FRHIRenderQuery* RenderQuery)
{
	ResourceCast(RenderQuery)->Begin();
}

void FOpenGLDynamicRHI::RHIEndRenderQuery(FRHIRenderQuery* RenderQuery)
{
	FOpenGLRenderQuery_RHI* Query = ResourceCast(RenderQuery);
	Query->End();
}

bool FOpenGLDynamicRHI::RHIGetRenderQueryResult(FRHIRenderQuery* QueryRHI, uint64& OutResult, bool bWait, uint32 GPUIndex)
{
	if (!QueryRHI)
	{
		OutResult = 0;
		return true;
	}

	FOpenGLRenderQuery_RHI* Query = ResourceCast(QueryRHI);
	return Query->GetResult(bWait, OutResult);
}

void FOpenGLEventQuery::IssueEvent()
{
	VERIFY_GL_SCOPE();
	if(Sync)
	{
		FOpenGL::DeleteSync(Sync);
		Sync = UGLsync();
	}
	Sync = FOpenGL::FenceSync(GL_SYNC_GPU_COMMANDS_COMPLETE, 0);
	FOpenGL::Flush();

	checkSlow(FOpenGL::IsSync(Sync));
}

void FOpenGLEventQuery::WaitForCompletion()
{
	VERIFY_GL_SCOPE();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOpenGLEventQuery_WaitForCompletion);

	checkSlow(FOpenGL::IsSync(Sync));

	// Wait up to 1/2 second for sync execution
	FOpenGL::EFenceResult Status = FOpenGL::ClientWaitSync( Sync, 0, 500*1000*1000);

	switch (Status)
	{
	case FOpenGL::FR_AlreadySignaled:
	case FOpenGL::FR_ConditionSatisfied:
		break;

	case FOpenGL::FR_TimeoutExpired:
		UE_LOGF(LogRHI, Log, "Timed out while waiting for GPU to catch up. (500 ms)");
		break;

	case FOpenGL::FR_WaitFailed:
		UE_LOGF(LogRHI, Log, "Wait on GPU failed in driver");
		break;

	default:
	    UE_LOGF(LogRHI, Log, "Unknown error while waiting on GPU");
	    check(0);
		break;
	}	
}

FOpenGLEventQuery::FOpenGLEventQuery()
{
	VERIFY_GL_SCOPE();

	// Initialize the query by issuing an initial event.
	IssueEvent();

	check(FOpenGL::IsSync(Sync));
}

FOpenGLEventQuery::~FOpenGLEventQuery()
{
	VERIFY_GL_SCOPE();
	FOpenGL::DeleteSync(Sync);
}

TQueue<FOpenGLGPUFence::FGLSync, EQueueMode::SingleThreaded> FOpenGLGPUFence::ActiveSyncs;

FOpenGLGPUFence::FOpenGLGPUFence(FName InName)
	: FRHIGPUFence(InName)
{
}

void FOpenGLGPUFence::Clear()
{
	Event = nullptr;
}

bool FOpenGLGPUFence::Poll() const
{
	return Event && Event->IsComplete();
}

void FOpenGLGPUFence::Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const
{
	if (Event && !Event->IsComplete())
	{
		//
		// The fence might get signalled by an earlier RHI command polling them, but we can't be sure that will happen.
		// The GPU might finish work after the RHI thread has gone idle, and then we'll never see the fence complete.
		//
		// Enqueue a command here that will block and wait for the fence if it still hasn't signalled by the time
		// the RHI thread is done with all prior commands.
		//
		RHICmdList.EnqueueLambda([Event = Event](FRHICommandListImmediate&)
		{
			if (!Event->IsComplete())
			{
				PollFencesUntil(Event);
			}
		});
		RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

		Event->Wait();
	}
}

void FOpenGLGPUFence::PollFencesUntil(FGraphEvent* Target)
{
	if (!PlatformOpenGLThreadHasRenderingContext() && !Target)
	{
		// Don't poll fences if this thread doesn't own the GL context.
		return;
	}

	VERIFY_GL_SCOPE();

	while (FGLSync* Sync = ActiveSyncs.Peek())
	{
		bool const bTarget = Sync->Event == Target;

		// Wait forever if this fence is the one we're looking for specifically, otherwise just poll.
		uint64 const Timeout = bTarget ? 0xffffffff'ffffffff : 0;

		switch (FOpenGL::ClientWaitSync(Sync->GLSync, 0, Timeout))
		{
		case FOpenGL::FR_AlreadySignaled:
		case FOpenGL::FR_ConditionSatisfied:
			break; // Fence completed

		case FOpenGL::FR_TimeoutExpired:
			return; // Fence is not done

		default:
			checkNoEntry();
			[[fallthrough]];
		case FOpenGL::FR_WaitFailed:
			// Some error state
			UE_LOGF(LogOpenGL, Fatal, "Waiting on FGLSync fence 0x%p failed.", Sync);
			return;
		}

		// The fence has completed. Signal the graph event and remove the node.
		Sync->Event->DispatchSubsequents();
		FOpenGL::DeleteSync(Sync->GLSync);

		ActiveSyncs.Pop();

		if (bTarget)
		{
			// We found the fence we wanted. Stop polling.
			return;
		}
	}

	checkf(!Target, TEXT("Attempt to poll for a specific fence, but it was not found in the queue."));
}

void FOpenGLGPUFence::WriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList)
{
	Event = FGraphEvent::CreateGraphEvent();

	RHICmdList.EnqueueLambda([Event = Event](FRHICommandListBase&) mutable
	{
		VERIFY_GL_SCOPE();

		UGLsync Fence = FOpenGL::FenceSync(GL_SYNC_GPU_COMMANDS_COMPLETE, 0);
		ActiveSyncs.Enqueue(FGLSync(MoveTemp(Event), Fence));
	});
}

void FOpenGLDynamicRHI::RHIWriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList, FRHIGPUFence* FenceRHI)
{
	ResourceCast(FenceRHI)->WriteGPUFence_TopOfPipe(RHICmdList);
}

void FOpenGLDynamicRHI::RHIWriteGPUFence(FRHIGPUFence* FenceRHI)
{
	checkNoEntry(); // Should never be called
}

FGPUFenceRHIRef FOpenGLDynamicRHI::RHICreateGPUFence(const FName& Name)
{
	return new FOpenGLGPUFence(Name);
}
