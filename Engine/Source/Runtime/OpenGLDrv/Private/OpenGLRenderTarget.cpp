// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLRenderTarget.cpp: OpenGL render target implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "RHICore.h"

// gDEBugger is currently very buggy. For example, it cannot show render buffers correctly and doesn't
// know what combined depth/stencil is. This define makes OpenGL render directly to textures and disables
// stencil. It results in broken post process effects, but allows to debug the rendering in gDEBugger.
//#define GDEBUGGER_MODE

#define ALL_SLICES uint32(0xffffffff)

// GL_MAX_DRAW_BUFFERS value
GLint GMaxOpenGLDrawBuffers = 0;

/**
* Key used to map a set of unique render/depth stencil target combinations to
* a framebuffer resource
*/
class FOpenGLFramebufferKey
{
	struct RenderTargetInfo
	{
		FOpenGLTexture* Texture;
		GLuint			Resource;
		uint32			MipmapLevel;
		uint32			ArrayIndex;
	};

public:

	FOpenGLFramebufferKey(
		uint32 InNumRenderTargets,
		FOpenGLTexture** InRenderTargets,
		const uint32* InRenderTargetArrayIndices,
		const uint32* InRenderTargetMipmapLevels,
		FOpenGLTexture* InDepthStencilTarget,
		FExclusiveDepthStencil DepthStencilAccess,
		int32 InNumRenderingSamples
		)
		:	DepthStencilTarget(InDepthStencilTarget)
		,	DepthStencilAccessIndex(DepthStencilAccess.GetIndex())
		,	NumRenderingSamples(InNumRenderingSamples)
	{
		uint32 RenderTargetIndex;
		for( RenderTargetIndex = 0; RenderTargetIndex < InNumRenderTargets; ++RenderTargetIndex )
		{
			FMemory::Memzero(RenderTargets[RenderTargetIndex]); // since memcmp is used, we need to zero memory
			RenderTargets[RenderTargetIndex].Texture = InRenderTargets ? InRenderTargets[RenderTargetIndex] : nullptr;
			RenderTargets[RenderTargetIndex].Resource = (InRenderTargets && InRenderTargets[RenderTargetIndex]) ? InRenderTargets[RenderTargetIndex]->GetResource() : 0;
			RenderTargets[RenderTargetIndex].MipmapLevel = InRenderTargetMipmapLevels[RenderTargetIndex];
			RenderTargets[RenderTargetIndex].ArrayIndex = (InRenderTargetArrayIndices == NULL || InRenderTargetArrayIndices[RenderTargetIndex] == -1) ? ALL_SLICES : InRenderTargetArrayIndices[RenderTargetIndex];
		}
		for( ; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex )
		{
			FMemory::Memzero(RenderTargets[RenderTargetIndex]); // since memcmp is used, we need to zero memory
			RenderTargets[RenderTargetIndex].ArrayIndex = ALL_SLICES;
		}
	}

	/**
	* Equality is based on render and depth stencil targets 
	* @param Other - instance to compare against
	* @return true if equal
	*/
	friend bool operator ==(const FOpenGLFramebufferKey& A,const FOpenGLFramebufferKey& B)
	{
		return	
			!FMemory::Memcmp(A.RenderTargets, B.RenderTargets, sizeof(A.RenderTargets) ) && 
			A.DepthStencilTarget == B.DepthStencilTarget &&
			A.DepthStencilAccessIndex == B.DepthStencilAccessIndex &&
			A.NumRenderingSamples == B.NumRenderingSamples;
	}

	/**
	* Get the hash for this type. 
	* @param Key - struct to hash
	* @return uint32 hash based on type
	*/
	friend uint32 GetTypeHash(const FOpenGLFramebufferKey &Key)
	{
		return FCrc::MemCrc_DEPRECATED(Key.RenderTargets, sizeof(Key.RenderTargets)) ^ GetTypeHash(Key.DepthStencilTarget) ^ GetTypeHash(Key.DepthStencilAccessIndex) ^ GetTypeHash(Key.NumRenderingSamples);
	}

	const FOpenGLTexture* GetRenderTarget( int32 Index ) const { return RenderTargets[Index].Texture; }
	const FOpenGLTexture* GetDepthStencilTarget( void ) const { return DepthStencilTarget; }
	int32 GetNumRenderingSamples( void ) const { return NumRenderingSamples; }

private:

	RenderTargetInfo RenderTargets[MaxSimultaneousRenderTargets];
	FOpenGLTexture* DepthStencilTarget;
	uint32 DepthStencilAccessIndex; // we don't want to merge render passes that have different access to a depth buffer
	int32 NumRenderingSamples; // MSAA on tile
};

static void ConditionallyAllocateRenderbufferStorage(FOpenGLTexture& RenderTarget)
{
	if (RenderTarget.bMultisampleRenderbuffer && 
		RenderTarget.GetAllocatedStorageForMip(0,0) == false)
	{
		check(RenderTarget.IsMultisampled());
		check(RenderTarget.Target == GL_RENDERBUFFER);

		GLuint TextureID = RenderTarget.GetRawResourceName();
		const FRHITextureDesc& Desc = RenderTarget.GetDesc();
		const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[Desc.Format];
		const bool bSRGB = EnumHasAnyFlags(Desc.Flags, TexCreate_SRGB);
		
		glBindRenderbuffer(GL_RENDERBUFFER, TextureID);
		FOpenGL::RenderbufferStorageMultisample(GL_RENDERBUFFER, Desc.NumSamples, GLFormat.InternalFormat[bSRGB], Desc.Extent.X, Desc.Extent.Y);
		glBindRenderbuffer(GL_RENDERBUFFER, 0);

		RenderTarget.SetAllocatedStorage(true);
	}
}

typedef TMap<FOpenGLFramebufferKey,GLuint> FOpenGLFramebufferCache;

/** Lazily initialized framebuffer cache singleton. */
static FOpenGLFramebufferCache& GetOpenGLFramebufferCache()
{
	static FOpenGLFramebufferCache OpenGLFramebufferCache;
	return OpenGLFramebufferCache;
}

GLuint FOpenGLDynamicRHI::GetOpenGLFramebuffer(uint32 NumSimultaneousRenderTargets, FOpenGLTexture** RenderTargets, const uint32* ArrayIndices, const uint32* MipmapLevels, FOpenGLTexture* DepthStencilTarget)
{
	const int32 NumRenderingSamples = 1;
	return GetOpenGLFramebuffer(NumSimultaneousRenderTargets, RenderTargets, ArrayIndices, MipmapLevels, DepthStencilTarget, FExclusiveDepthStencil(), NumRenderingSamples);
}

GLuint FOpenGLDynamicRHI::GetOpenGLFramebuffer(uint32 NumSimultaneousRenderTargets, FOpenGLTexture** RenderTargets, const uint32* ArrayIndices, const uint32* MipmapLevels, FOpenGLTexture* DepthStencilTarget, FExclusiveDepthStencil DepthStencilAccess, int32 NumRenderingSamples)
{
	VERIFY_GL_SCOPE();

	check( NumSimultaneousRenderTargets <= MaxSimultaneousRenderTargets );

	uint32 FramebufferRet = GetOpenGLFramebufferCache().FindRef(FOpenGLFramebufferKey(NumSimultaneousRenderTargets, RenderTargets, ArrayIndices, MipmapLevels, DepthStencilTarget, DepthStencilAccess, NumRenderingSamples));
	if( FramebufferRet > 0 )
	{
		// Found and is valid. We never store zero as a result, increasing all results by 1 to avoid range overlap.
		return FramebufferRet-1;
	}

	const bool bRenderTargetsDefined = (RenderTargets != nullptr) && RenderTargets[0];

	// Check for rendering to screen back buffer.
	if (NumSimultaneousRenderTargets > 0 && bRenderTargetsDefined && RenderTargets[0]->GetResource() == GL_NONE)
	{
		// Use the default framebuffer (screen back/depth buffer)
		return GL_NONE;
	}

	// Not found. Preparing new one.
	GLuint Framebuffer;
	glGenFramebuffers(1, &Framebuffer);
	VERIFY_GL(glGenFramebuffer)
	glBindFramebuffer(GL_FRAMEBUFFER, Framebuffer);
	VERIFY_GL(glBindFramebuffer)

	static const auto CVarMobileMultiView = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.MobileMultiView"));

	// Allocate mobile multi-view frame buffer if enabled and supported.
	// Multi-view doesn't support read buffers, explicitly disable and only bind GL_DRAW_FRAMEBUFFER
	// TODO: We can't reliably use packed depth stencil?
	const bool bValidMultiViewDepthTarget = !DepthStencilTarget || DepthStencilTarget->Target == GL_TEXTURE_2D_ARRAY;
	const bool bUsingArrayTextures = (bRenderTargetsDefined) ? (RenderTargets[0]->Target == GL_TEXTURE_2D_ARRAY && bValidMultiViewDepthTarget) : false;
	const bool bMultiViewCVar = CVarMobileMultiView && CVarMobileMultiView->GetValueOnAnyThread() != 0;

	if (bUsingArrayTextures && FOpenGL::SupportsMobileMultiView() && bMultiViewCVar)
	{
		FOpenGLTexture* const RenderTarget = RenderTargets[0];
		glBindFramebuffer(GL_FRAMEBUFFER, 0);
		glBindFramebuffer(GL_DRAW_FRAMEBUFFER, Framebuffer);

		if (NumRenderingSamples > 1)
		{
			FOpenGL::FramebufferTextureMultisampleMultiviewOVR(GL_DRAW_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, RenderTarget->GetResource(), 0, NumRenderingSamples, 0, 2);
			VERIFY_GL(glFramebufferTextureMultisampleMultiviewOVR);

			if (DepthStencilTarget)
			{
				FOpenGL::FramebufferTextureMultisampleMultiviewOVR(GL_DRAW_FRAMEBUFFER, GL_DEPTH_ATTACHMENT, DepthStencilTarget->GetResource(), 0, NumRenderingSamples, 0, 2);
				VERIFY_GL(glFramebufferTextureMultisampleMultiviewOVR);
			}
		}
		else
		{
			FOpenGL::FramebufferTextureMultiviewOVR(GL_DRAW_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, RenderTarget->GetResource(), 0, 0, 2);
			VERIFY_GL(glFramebufferTextureMultiviewOVR);

			if (DepthStencilTarget)
			{
				FOpenGL::FramebufferTextureMultiviewOVR(GL_DRAW_FRAMEBUFFER, GL_DEPTH_ATTACHMENT, DepthStencilTarget->GetResource(), 0, 0, 2);
				VERIFY_GL(glFramebufferTextureMultiviewOVR);
			}
		}

		FOpenGL::CheckFrameBuffer();

		FOpenGL::ReadBuffer(GL_NONE);
		FOpenGL::DrawBuffer(GL_COLOR_ATTACHMENT0);

		GetOpenGLFramebufferCache().Add(FOpenGLFramebufferKey(NumSimultaneousRenderTargets, RenderTargets, ArrayIndices, MipmapLevels, DepthStencilTarget, DepthStencilAccess, NumRenderingSamples), Framebuffer + 1);
		
		return Framebuffer;
	}
	
	int32 FirstNonzeroRenderTarget = -1;
	for (int32 RenderTargetIndex = NumSimultaneousRenderTargets - 1; RenderTargetIndex >= 0 && bRenderTargetsDefined; --RenderTargetIndex)
	{
		FOpenGLTexture* RenderTarget = RenderTargets[RenderTargetIndex];
		if (!RenderTarget)
		{
			continue;
		}

		if (ArrayIndices == NULL || ArrayIndices[RenderTargetIndex] == -1)
		{
			// If no index was specified, bind the entire object, rather than a slice
			switch (RenderTarget->Target)
			{
			case GL_RENDERBUFFER:
			{
				// lazily allocate render buffer storage in case it's multisampled
				ConditionallyAllocateRenderbufferStorage(*RenderTarget);
				glFramebufferRenderbuffer(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, GL_RENDERBUFFER, RenderTarget->GetResource());
				break;
			}
			case GL_TEXTURE_2D:
			case GL_TEXTURE_EXTERNAL_OES:
			case GL_TEXTURE_2D_MULTISAMPLE:
			{
				if (NumRenderingSamples > 1)
				{
					// GL_EXT_multisampled_render_to_texture
					FOpenGL::FramebufferTexture2DMultisample(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->Target, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex], NumRenderingSamples);
				}
				else
				{
					FOpenGL::FramebufferTexture2D(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->Target, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex]);
				}
				break;
			}
			case GL_TEXTURE_3D:
			case GL_TEXTURE_2D_ARRAY:
			case GL_TEXTURE_CUBE_MAP:
			case GL_TEXTURE_CUBE_MAP_ARRAY:
				FOpenGL::FramebufferTexture(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex]);
				break;
			default:
				FOpenGL::FramebufferRenderbuffer(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, GL_RENDERBUFFER, RenderTarget->GetResource());
				break;
			}
		}
		else
		{
			// Bind just one slice of the object
			switch( RenderTarget->Target )
			{
			case GL_RENDERBUFFER:
			{
				check(ArrayIndices[RenderTargetIndex] == 0);
				// lazily allocate render buffer storage in case it's multisampled
				ConditionallyAllocateRenderbufferStorage(*RenderTarget);
				glFramebufferRenderbuffer(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, GL_RENDERBUFFER, RenderTarget->GetResource());
				break;
			}
			case GL_TEXTURE_2D:
			case GL_TEXTURE_EXTERNAL_OES:
			case GL_TEXTURE_2D_MULTISAMPLE:
			{
				check(ArrayIndices[RenderTargetIndex] == 0);
				if (NumRenderingSamples > 1)
				{
					// GL_EXT_multisampled_render_to_texture
					FOpenGL::FramebufferTexture2DMultisample(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->Target, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex], NumRenderingSamples);
				}
				else
				{
					FOpenGL::FramebufferTexture2D(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->Target, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex]);
				}
				break;
			}
			case GL_TEXTURE_3D:
				FOpenGL::FramebufferTexture3D(GL_FRAMEBUFFER,GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->Target, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex], ArrayIndices[RenderTargetIndex]);
				break;
			case GL_TEXTURE_CUBE_MAP:
				check( ArrayIndices[RenderTargetIndex] < 6);
				FOpenGL::FramebufferTexture2D(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, GL_TEXTURE_CUBE_MAP_POSITIVE_X + ArrayIndices[RenderTargetIndex], RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex]);
				break;
			case GL_TEXTURE_2D_ARRAY:
			case GL_TEXTURE_CUBE_MAP_ARRAY:
				FOpenGL::FramebufferTextureLayer(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, RenderTarget->GetResource(), MipmapLevels[RenderTargetIndex], ArrayIndices[RenderTargetIndex]);
				break;
			default:
				check( ArrayIndices[RenderTargetIndex] == 0);
				FOpenGL::FramebufferRenderbuffer(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0 + RenderTargetIndex, GL_RENDERBUFFER, RenderTarget->GetResource());
				break;
			}
		}
		FirstNonzeroRenderTarget = RenderTargetIndex;
	}

	if (DepthStencilTarget)
	{
		switch (DepthStencilTarget->Target)
		{
		case GL_TEXTURE_2D:
		case GL_TEXTURE_EXTERNAL_OES:
		case GL_TEXTURE_2D_MULTISAMPLE:
		{
			FOpenGL::FramebufferTexture2D(GL_FRAMEBUFFER, DepthStencilTarget->Attachment, DepthStencilTarget->Target, DepthStencilTarget->GetResource(), 0);
			break;
		}
		case GL_RENDERBUFFER:
		{
			// lazily allocate render buffer storage in case it's multisampled
			ConditionallyAllocateRenderbufferStorage(*DepthStencilTarget);
			glFramebufferRenderbuffer(GL_FRAMEBUFFER, GL_DEPTH_ATTACHMENT, GL_RENDERBUFFER, DepthStencilTarget->GetResource());
			glFramebufferRenderbuffer(GL_FRAMEBUFFER, GL_STENCIL_ATTACHMENT, GL_RENDERBUFFER, DepthStencilTarget->GetResource());
			VERIFY_GL(glFramebufferRenderbuffer);
			break;
		}
		case GL_TEXTURE_3D:
		case GL_TEXTURE_2D_ARRAY:
		case GL_TEXTURE_CUBE_MAP:
		case GL_TEXTURE_CUBE_MAP_ARRAY:
			FOpenGL::FramebufferTexture(GL_FRAMEBUFFER, DepthStencilTarget->Attachment, DepthStencilTarget->GetResource(), 0);
			break;
		default:
			FOpenGL::FramebufferRenderbuffer(GL_FRAMEBUFFER, DepthStencilTarget->Attachment, GL_RENDERBUFFER, DepthStencilTarget->GetResource());
			break;
		}
	}

	if (FirstNonzeroRenderTarget != -1)
	{
		FOpenGL::ReadBuffer(GL_COLOR_ATTACHMENT0 + FirstNonzeroRenderTarget);
		FOpenGL::DrawBuffer(GL_COLOR_ATTACHMENT0 + FirstNonzeroRenderTarget);
	}
	else
	{
		FOpenGL::ReadBuffer(GL_NONE);
		FOpenGL::DrawBuffer(GL_NONE);
	}

	//  End frame can bind NULL / NULL 
	//  An FBO with no attachments is framebuffer incomplete (INCOMPLETE_MISSING_ATTACHMENT)
	//  In this case just delete the FBO and map in the default
	//  In GL 4.x, NULL/NULL is valid and can be done =by specifying a default width/height
	if ( FirstNonzeroRenderTarget == -1 && !DepthStencilTarget )
	{
		glDeleteFramebuffers( 1, &Framebuffer);
		Framebuffer = 0;
		glBindFramebuffer(GL_FRAMEBUFFER, Framebuffer);
	}
	
	FOpenGL::CheckFrameBuffer();

	GetOpenGLFramebufferCache().Add(FOpenGLFramebufferKey(NumSimultaneousRenderTargets, RenderTargets, ArrayIndices, MipmapLevels, DepthStencilTarget, DepthStencilAccess, NumRenderingSamples), Framebuffer+1);

	return Framebuffer;
}

void ReleaseOpenGLFramebuffers(FRHITexture* TextureRHI)
{
	VERIFY_GL_SCOPE();

	const FOpenGLTexture* Texture = FOpenGLDynamicRHI::ResourceCast(TextureRHI);

	if (Texture)
	{
		for (FOpenGLFramebufferCache::TIterator It(GetOpenGLFramebufferCache()); It; ++It)
		{
			bool bPurgeFramebuffer = false;
			FOpenGLFramebufferKey Key = It.Key();

			const FOpenGLTexture* DepthStencilTarget = Key.GetDepthStencilTarget();
			if( DepthStencilTarget && DepthStencilTarget->Target == Texture->Target && DepthStencilTarget->GetRawResourceName() == Texture->GetRawResourceName() )
			{
				bPurgeFramebuffer = true;
			}
			else
			{
				for( uint32 RenderTargetIndex = 0; RenderTargetIndex < MaxSimultaneousRenderTargets; ++RenderTargetIndex )
				{
					const FOpenGLTexture* RenderTarget = Key.GetRenderTarget(RenderTargetIndex);
					if( RenderTarget && RenderTarget->Target == Texture->Target && RenderTarget->GetRawResourceName() == Texture->GetRawResourceName() )
					{
						bPurgeFramebuffer = true;
						break;
					}
				}
			}

			if( bPurgeFramebuffer )
			{
				GLuint FramebufferToDelete = It.Value()-1;
				check(FramebufferToDelete > 0);

				FOpenGLDynamicRHI::Get().PurgeFramebufferFromCaches( FramebufferToDelete );
				glDeleteFramebuffers( 1, &FramebufferToDelete );

				It.RemoveCurrent();
			}
		}
	}
}

void FOpenGLDynamicRHI::PurgeFramebufferFromCaches( GLuint Framebuffer )
{
	VERIFY_GL_SCOPE();

	if (Framebuffer == PendingState.Framebuffer)
	{
		PendingState.Framebuffer = 0;
		FMemory::Memset(PendingState.RenderTargets, 0, sizeof(PendingState.RenderTargets));
		FMemory::Memset(PendingState.RenderTargetMipmapLevels, 0, sizeof(PendingState.RenderTargetMipmapLevels));
		FMemory::Memset(PendingState.RenderTargetArrayIndex, 0, sizeof(PendingState.RenderTargetArrayIndex));
		PendingState.DepthStencil = 0;
		PendingState.bFramebufferSetupInvalid = true;
	}

	if (Framebuffer == ContextState.Framebuffer)
	{
		ContextState.Framebuffer = -1;
	}
}

void FOpenGLDynamicRHI::ReadSurfaceDataRaw(FRHITexture* TextureRHI, FIntRect Rect, TArray<uint8>& OutData, FReadSurfaceDataFlags InFlags)
{
	VERIFY_GL_SCOPE();

	FOpenGLTexture* Texture = ResourceCast(TextureRHI);
	if( !Texture)
	{
		return;	// just like in D3D11
	}

	GLuint FramebufferToDelete = 0;
	GLuint RenderbufferToDelete = 0;
	const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[TextureRHI->GetFormat()];

	bool bFloatFormat = false;
	bool bUnsupportedFormat = false;
	bool bDepthFormat = false;
	bool bDepthStencilFormat = false;

	switch( TextureRHI->GetFormat() )
	{
	case PF_DepthStencil:
		bDepthStencilFormat = true;
		// pass-through
	case PF_ShadowDepth:
	case PF_D24:
		bDepthFormat = true;
		break;

	case PF_A32B32G32R32F:
	case PF_FloatRGBA:
	case PF_FloatRGB:
	case PF_R32_FLOAT:
	case PF_G16R16F:
	case PF_G16R16F_FILTER:
	case PF_G32R32F:
	case PF_R16F:
	case PF_R16F_FILTER:
	case PF_FloatR11G11B10:
		bFloatFormat = true;
		break;

	case PF_DXT1:
	case PF_DXT3:
	case PF_DXT5:
	case PF_UYVY:
	case PF_BC5:
	case PF_PVRTC2:
	case PF_PVRTC4:
	case PF_ATC_RGB:
	case PF_ATC_RGBA_E:
	case PF_ATC_RGBA_I:
		bUnsupportedFormat = true;
		break;

	default:	// the rest is assumed to be integer formats with one or more of ARG and B components in OpenGL
		break;
	}

	if( bUnsupportedFormat )
	{
#if UE_BUILD_DEBUG
		check(0);
#endif
		return;
	}

	check( !bDepthFormat || FOpenGL::SupportsDepthStencilReadSurface() );
	check( !bFloatFormat || FOpenGL::SupportsFloatReadSurface() );
	const GLenum Attachment = bDepthFormat ? (bDepthStencilFormat ? GL_DEPTH_STENCIL_ATTACHMENT : GL_DEPTH_ATTACHMENT) : GL_COLOR_ATTACHMENT0;
	const bool bIsColorBuffer = (Texture->Attachment == GL_COLOR_ATTACHMENT0) || (Texture->Attachment == 0);

	const uint32 MipmapLevel = InFlags.GetMip();
	GLuint SourceFramebuffer = bIsColorBuffer ? GetOpenGLFramebuffer(1, &Texture, NULL, &MipmapLevel, NULL) : GetOpenGLFramebuffer(0, NULL, NULL, NULL, Texture);
	if (TextureRHI->IsMultisampled())
	{
		// OpenGL doesn't allow to read pixels from multisample framebuffers, we need a single sample copy
		glGenFramebuffers(1, &FramebufferToDelete);
		glBindFramebuffer(GL_FRAMEBUFFER, FramebufferToDelete);

		GLuint Renderbuffer = 0;
		glGenRenderbuffers(1, &RenderbufferToDelete);
		glBindRenderbuffer(GL_RENDERBUFFER, RenderbufferToDelete);
		glRenderbufferStorage(GL_RENDERBUFFER, GLFormat.InternalFormat[false], Texture->GetSizeX(), Texture->GetSizeY());
		glBindRenderbuffer(GL_RENDERBUFFER, 0);

		glFramebufferRenderbuffer(GL_FRAMEBUFFER, Attachment, GL_RENDERBUFFER, RenderbufferToDelete);
		FOpenGL::CheckFrameBuffer();
		glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);
		FOpenGL::BlitFramebuffer(
			0, 0, Texture->GetSizeX(), Texture->GetSizeY(),
			0, 0, Texture->GetSizeX(), Texture->GetSizeY(),
			(bDepthFormat ? (bDepthStencilFormat ? (GL_DEPTH_BUFFER_BIT | GL_STENCIL_BUFFER_BIT) : GL_DEPTH_BUFFER_BIT) : GL_COLOR_BUFFER_BIT),
			GL_NEAREST
			);

		SourceFramebuffer = FramebufferToDelete;
	}

	uint32 SizeX = Rect.Width();
	uint32 SizeY = Rect.Height();

	OutData.Empty( SizeX * SizeY * sizeof(FColor) );
	uint8* TargetBuffer = (uint8*)&OutData[OutData.AddUninitialized(SizeX * SizeY * sizeof(FColor))];

	glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);
	FOpenGL::ReadBuffer( (!bDepthFormat && !bDepthStencilFormat && !SourceFramebuffer) ? GL_BACK : Attachment);
	glPixelStorei(GL_PACK_ALIGNMENT, 1);

	if( bDepthFormat )
	{
		// Get the depth as luminosity, with non-transparent alpha.
		// If depth values are between 0 and 1, keep them, otherwise rescale them linearly so they fit within 0-1 range

		int32 DepthValueCount = SizeX * SizeY;
		int32 FloatDepthDataSize = sizeof(float) * DepthValueCount;
		float* FloatDepthData = (float*)FMemory::Malloc( FloatDepthDataSize );
		glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_DEPTH_COMPONENT, GL_FLOAT, FloatDepthData );

		// Determine minimal and maximal float value present in received data
		float MinValue = FLT_MAX;
		float MaxValue = FLT_MIN;
		float* DataPtr = FloatDepthData;
		for( int32 DepthValueIndex = 0; DepthValueIndex < DepthValueCount; ++DepthValueIndex, ++DataPtr )
		{
			if( *DataPtr < MinValue )
			{
				MinValue = *DataPtr;
			}

			if( *DataPtr > MaxValue )
			{
				MaxValue = *DataPtr;
			}
		}

		// If necessary, rescale the data.
		if( ( MinValue < 0.f ) || ( MaxValue > 1.f ) )
		{
			DataPtr = FloatDepthData;
			float RescaleFactor = MaxValue - MinValue;
			for( int32 DepthValueIndex = 0; DepthValueIndex < DepthValueCount; ++DepthValueIndex, ++DataPtr )
			{
				*DataPtr = ( *DataPtr - MinValue ) / RescaleFactor;
			}
		}

		// Convert the data into rgba8 buffer
		DataPtr = FloatDepthData;
		uint8* TargetPtr = TargetBuffer;
		for( int32 DepthValueIndex = 0; DepthValueIndex < DepthValueCount; ++DepthValueIndex )
		{
			uint8 Value = FColor::QuantizeUNormFloatTo8( *DataPtr++ );
			*TargetPtr++ = Value;
			*TargetPtr++ = Value;
			*TargetPtr++ = Value;
			*TargetPtr++ = 255;
		}

		FMemory::Free( FloatDepthData );
	}
	else if( bFloatFormat )
	{
		bool bLinearToGamma = InFlags.GetLinearToGamma();

		// Determine minimal and maximal float value present in received data. Treat alpha separately.

		int32 PixelComponentCount = 4 * SizeX * SizeY;
		int32 FloatBGRADataSize = sizeof(float) * PixelComponentCount;
		float* FloatBGRAData = (float*)FMemory::Malloc( FloatBGRADataSize );
		if ( FOpenGL::SupportsBGRA8888() )
		{
			glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_BGRA, GL_FLOAT, FloatBGRAData );
			GLenum Error = glGetError();
			if (Error != GL_NO_ERROR)
			{
				glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_RGBA, GL_FLOAT, FloatBGRAData );
				Error = glGetError();
				if (Error == GL_NO_ERROR)
				{
					float* FloatDataPtr = FloatBGRAData;
					float* FloatDataPtrEnd = FloatBGRAData + PixelComponentCount;
					while (FloatDataPtr != FloatDataPtrEnd)
					{
						float Temp = FloatDataPtr[0];
						FloatDataPtr[0] = FloatDataPtr[2];
						FloatDataPtr[2] = Temp;
						FloatDataPtr += 4;
					}
				}
			}
		}
		else 
		{
			glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_RGBA, GL_FLOAT, FloatBGRAData );
		}
		// Determine minimal and maximal float values present in received data. Treat each component separately.
		float MinValue[4] = { 0.0f, 0.0f, 0.0f, 0.0f };
		float MaxValue[4] = { 1.0f, 1.0f, 1.0f, 1.0f };
		float* DataPtr = FloatBGRAData;
		for( int32 PixelComponentIndex = 0; PixelComponentIndex < PixelComponentCount; ++PixelComponentIndex, ++DataPtr )
		{
			int32 ComponentIndex = PixelComponentIndex % 4;
			MinValue[ComponentIndex] = FMath::Min<float>(*DataPtr,MinValue[ComponentIndex]);
			MaxValue[ComponentIndex] = FMath::Max<float>(*DataPtr,MaxValue[ComponentIndex]);
		}

		// Convert the data into BGRA8 buffer
		DataPtr = FloatBGRAData;
		uint8* TargetPtr = TargetBuffer;
		float RescaleFactor[4] = { MaxValue[0] - MinValue[0], MaxValue[1] - MinValue[1], MaxValue[2] - MinValue[2], MaxValue[3] - MinValue[3] };
		for( int32 PixelIndex = 0; PixelIndex < PixelComponentCount / 4; ++PixelIndex )
		{
			float R = (DataPtr[2] - MinValue[2]) / RescaleFactor[2]; 
			float G = (DataPtr[1] - MinValue[1]) / RescaleFactor[1];
			float B = (DataPtr[0] - MinValue[0]) / RescaleFactor[0]; 
			float A = (DataPtr[3] - MinValue[3]) / RescaleFactor[3]; 
			
			if ( !FOpenGL::SupportsBGRA8888() )
			{
				Swap<float>( R,B );
			}		   
			FColor NormalizedColor = FLinearColor( R,G,B,A ).ToFColor(bLinearToGamma);
			FMemory::Memcpy(TargetPtr,&NormalizedColor,sizeof(FColor));
			DataPtr += 4;
			TargetPtr += 4;
		}

		FMemory::Free( FloatBGRAData );
	}
#if PLATFORM_ANDROID
	else
	{
		// Flip texture data only for render targets, textures loaded from disk have Attachment set to 0 and don't need flipping.
		const bool bFlipTextureData = Texture->Attachment != 0;
		GLubyte* RGBAData = TargetBuffer;
		if (bFlipTextureData)
		{
			// OpenGL ES is limited in what it can do with ReadPixels
			const int32 PixelComponentCount = 4 * SizeX * SizeY;
			const int32 RGBADataSize = sizeof(GLubyte) * PixelComponentCount;
			RGBAData = (GLubyte*)FMemory::Malloc(RGBADataSize);
		}

		glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_RGBA, GL_UNSIGNED_BYTE, RGBAData);

		if (bFlipTextureData)
		{
			//OpenGL ES reads the pixels "upside down" from what we're expecting (flipped vertically), so we need to transfer the data from the bottom up.
			uint8* TargetPtr = TargetBuffer;
			int32 Stride = SizeX * 4;
			int32 FlipHeight = SizeY;
			GLubyte* LinePtr = RGBAData + (SizeY - 1) * Stride;

			while (FlipHeight--)
			{
				GLubyte* DataPtr = LinePtr;
				int32 Pixels = SizeX;
				while (Pixels--)
				{
					TargetPtr[0] = DataPtr[2];
					TargetPtr[1] = DataPtr[1];
					TargetPtr[2] = DataPtr[0];
					TargetPtr[3] = DataPtr[3];
					DataPtr += 4;
					TargetPtr += 4;
				}
				LinePtr -= Stride;
			}

			FMemory::Free(RGBAData);
		}
	}
#else
	else
	{
		// It's a simple int format. OpenGL converts them internally to what we want.
		glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_BGRA, UGL_ABGR8, TargetBuffer );
	}
#endif

	glPixelStorei(GL_PACK_ALIGNMENT, 4);

	if( FramebufferToDelete )
	{
		glBindFramebuffer(GL_FRAMEBUFFER, 0);
		glDeleteFramebuffers( 1, &FramebufferToDelete );
	}

	if( RenderbufferToDelete )
	{
		glDeleteRenderbuffers( 1, &RenderbufferToDelete );
	}

	ContextState.Framebuffer = (GLuint)-1;
}

void FOpenGLDynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI,FIntRect Rect,TArray<FColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	const uint32 Size = Rect.Width() * Rect.Height();
	OutData.SetNumUninitialized(Size);

	if (!ensure(TextureRHI))
	{
		FMemory::Memzero(OutData.GetData(), Size * sizeof(FColor));
		return;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	
	RHICmdList.EnqueueLambda([this, TextureRHI, Rect, InFlags, &OutData](FRHICommandListImmediate&)
	{
		const uint32 Size = Rect.Width() * Rect.Height();
		TArray<uint8> Temp;

		ReadSurfaceDataRaw(TextureRHI, Rect, Temp, InFlags);
		FMemory::Memcpy(OutData.GetData(), Temp.GetData(), Size * sizeof(FColor));
	});

	RHITHREAD_GLTRACE_BLOCKING;
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}

void FOpenGLDynamicRHI::RHIReadSurfaceData(FRHITexture* TextureRHI, FIntRect Rect, TArray<FLinearColor>& OutData, FReadSurfaceDataFlags InFlags)
{
	// Verify requirements, but don't crash
	// Ignore texture format here, GL will convert it for us in glReadPixels
	if (!ensure(FOpenGL::SupportsFloatReadSurface()) || !ensure(TextureRHI))
	{
		return;
	}
	
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	
	RHICmdList.EnqueueLambda([this, TextureRHI, Rect, InFlags, &OutData](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();

		FOpenGLTexture* Texture = ResourceCast(TextureRHI);
		if (!ensure(Texture))
		{
			return;
		}

		// Get framebuffer for texture
		const uint32 MipmapLevel = InFlags.GetMip();
		GLuint SourceFramebuffer = GetOpenGLFramebuffer(1, &Texture, NULL, &MipmapLevel, NULL);

		uint32 SizeX = Rect.Width();
		uint32 SizeY = Rect.Height();

		// Initialize output
		OutData.SetNumUninitialized(SizeX * SizeY);

		// Bind the framebuffer
		// @TODO: Do we need to worry about multisampling?
		glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);
		FOpenGL::ReadBuffer(SourceFramebuffer == 0 ? GL_BACK : GL_COLOR_ATTACHMENT0);

		// Read the float data from the buffer directly into the output data
		// @TODO: Do we need to support BGRA?
		glPixelStorei(GL_PACK_ALIGNMENT, 1);
		glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_RGBA, GL_FLOAT, OutData.GetData());
		glPixelStorei(GL_PACK_ALIGNMENT, 4);

		ContextState.Framebuffer = (GLuint)-1;
	});
	RHITHREAD_GLTRACE_BLOCKING;
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}

void FOpenGLDynamicRHI::RHIMapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* TextureRHI, uint32 GPUIndex, FRHIGPUFence* Fence, void*& OutData, int32& OutWidth, int32& OutHeight)
{
	// Fence is not important, GL driver handles synchonization
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

	RHICmdList.EnqueueLambda([TextureRHI, &OutData, &OutWidth, &OutHeight](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();

		FOpenGLTexture* Texture = ResourceCast(TextureRHI->GetTexture2D());
		check(Texture);
		check(EnumHasAnyFlags(Texture->GetDesc().Flags, TexCreate_CPUReadback));

		OutWidth = Texture->GetSizeX();
		OutHeight = Texture->GetSizeY();

		uint32 Stride = 0;
		OutData = Texture->Lock( 0, 0, RLM_ReadOnly, Stride );
	});
	RHITHREAD_GLTRACE_BLOCKING;
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}

void FOpenGLDynamicRHI::RHIUnmapStagingSurface_RenderThread(class FRHICommandListImmediate& RHICmdList, FRHITexture* TextureRHI, uint32 GPUIndex)
{
	RHICmdList.EnqueueLambda([TextureRHI = TextureRHI](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();
		FOpenGLTexture* Texture = ResourceCast(TextureRHI->GetTexture2D());
		check(Texture);
		Texture->Unlock( 0, 0 );
	});
}

void FOpenGLDynamicRHI::RHIMapStagingSurface(FRHITexture* TextureRHI, FRHIGPUFence* FenceRHI, void*& OutData, int32& OutWidth, int32& OutHeight, uint32 GPUIndex)
{
	// Everything is handled in RHIMapStagingSurface_RenderThread. This function should not be called directly.
	checkNoEntry(); 
}

void FOpenGLDynamicRHI::RHIUnmapStagingSurface(FRHITexture* TextureRHI, uint32 GPUIndex)
{
	// Everything is handled in RHIUnmapStagingSurface_RenderThread. This function should not be called directly.
	checkNoEntry(); 
}

void FOpenGLDynamicRHI::RHIReadSurfaceFloatData(FRHITexture* TextureRHI,FIntRect Rect,TArray<FFloat16Color>& OutData,ECubeFace CubeFace,int32 ArrayIndex,int32 MipIndex)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	RHICmdList.EnqueueLambda([this, TextureRHI, Rect, CubeFace, ArrayIndex, MipIndex, &OutData](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();	

		//reading from arrays only supported on SM5 and up.
		check(FOpenGL::SupportsFloatReadSurface() && (ArrayIndex == 0 || GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5));	
		FOpenGLTexture* Texture = ResourceCast(TextureRHI);
		check(TextureRHI->GetFormat() == PF_FloatRGBA);

		const uint32 MipmapLevel = MipIndex;

		// Temp FBO is introduced to prevent a ballooning of FBO objects, which can have a detrimental
		// impact on object management performance in the driver, only for CubeMapArray presently
		// as it is the target that really drives  FBO permutations
		const bool bTempFBO = Texture->Target == GL_TEXTURE_CUBE_MAP_ARRAY;
		uint32 Index = uint32(CubeFace) + ( (Texture->Target == GL_TEXTURE_CUBE_MAP_ARRAY) ? 6 : 1) * ArrayIndex;

		GLuint SourceFramebuffer = 0;

		if (bTempFBO)
		{
			glGenFramebuffers( 1, &SourceFramebuffer);

			glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);

			FOpenGL::FramebufferTextureLayer(UGL_READ_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, Texture->GetResource(), MipmapLevel, Index);
		}
		else
		{
			SourceFramebuffer = GetOpenGLFramebuffer(1, &Texture, &Index, &MipmapLevel, NULL);
		}

		uint32 SizeX = Rect.Width();
		uint32 SizeY = Rect.Height();

		OutData.SetNumUninitialized(SizeX * SizeY);

		glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);
		FOpenGL::ReadBuffer(SourceFramebuffer == 0 ? GL_BACK : GL_COLOR_ATTACHMENT0);
		glPixelStorei(GL_PACK_ALIGNMENT, 1);	

		if (FOpenGL::GetReadHalfFloatPixelsEnum() == GL_FLOAT)
		{
			// Slow path: Some Adreno devices won't work with HALF_FLOAT ReadPixels
			TArray<FLinearColor> FloatData;
			// 4 float components per texel (RGBA)
			FloatData.AddUninitialized(SizeX * SizeY);
			FMemory::Memzero(FloatData.GetData(),SizeX * SizeY*sizeof(FLinearColor));
			glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_RGBA, GL_FLOAT, FloatData.GetData());
			FLinearColor* FloatDataPtr = FloatData.GetData();
			for (uint32 DataIndex = 0; DataIndex < SizeX * SizeY; ++DataIndex, ++FloatDataPtr)
			{
				OutData[DataIndex] = FFloat16Color(*FloatDataPtr);
			}
		}
		else
		{
			glReadPixels(Rect.Min.X, Rect.Min.Y, SizeX, SizeY, GL_RGBA, FOpenGL::GetReadHalfFloatPixelsEnum(), OutData.GetData());
		}

		glPixelStorei(GL_PACK_ALIGNMENT, 4);

		if (bTempFBO)
		{
			glDeleteFramebuffers( 1, &SourceFramebuffer);
		}

		ContextState.Framebuffer = (GLuint)-1;
	});
	RHITHREAD_GLTRACE_BLOCKING;
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}

void FOpenGLDynamicRHI::RHIRead3DSurfaceFloatData(FRHITexture* TextureRHI,FIntRect Rect,FIntPoint ZMinMax,TArray<FFloat16Color>& OutData)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	RHICmdList.EnqueueLambda([this, TextureRHI, Rect, ZMinMax, &OutData](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();

		check( FOpenGL::SupportsFloatReadSurface() );
		check( FOpenGL::SupportsTexture3D() );
		check( TextureRHI->GetFormat() == PF_FloatRGBA );

		FOpenGLTexture* Texture = ResourceCast(TextureRHI);

		uint32 SizeX = Rect.Width();
		uint32 SizeY = Rect.Height();
		uint32 SizeZ = ZMinMax.Y - ZMinMax.X;

		// Allocate the output buffer.
		OutData.SetNumUninitialized(SizeX * SizeY * SizeZ);

		// Set up the source as a temporary FBO
		uint32 MipmapLevel = 0;
		uint32 Index = 0;
		GLuint SourceFramebuffer = 0;
		glGenFramebuffers( 1, &SourceFramebuffer);
		glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);

		// Set up the destination as a temporary texture
		GLuint TempTexture = 0;
		FOpenGL::GenTextures(1, &TempTexture);
		glActiveTexture( GL_TEXTURE0 );
		glBindTexture( GL_TEXTURE_3D, TempTexture );
		FOpenGL::TexImage3D( GL_TEXTURE_3D, 0, GL_RGBA16F, SizeX, SizeY, SizeZ, 0, GL_RGBA, GL_HALF_FLOAT, NULL );

		// Copy the pixels within the specified region, minimizing the amount of data that needs to be transferred from GPU to CPU memory
		for ( uint32 Z=0; Z < SizeZ; ++Z )
		{
			FOpenGL::FramebufferTextureLayer(UGL_READ_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, Texture->GetResource(), MipmapLevel, ZMinMax.X + Z);
			FOpenGL::ReadBuffer(SourceFramebuffer == 0 ? GL_BACK : GL_COLOR_ATTACHMENT0);
			FOpenGL::CopyTexSubImage3D( GL_TEXTURE_3D, 0, 0, 0, Z, Rect.Min.X, Rect.Min.Y, SizeX, SizeY );
		}

		// Grab the raw data from the temp texture.
		glPixelStorei( GL_PACK_ALIGNMENT, 1 );
		FOpenGL::GetTexImage( GL_TEXTURE_3D, 0, GL_RGBA, GL_HALF_FLOAT, OutData.GetData() );
		glPixelStorei( GL_PACK_ALIGNMENT, 4 );

		// Clean up
		auto& TextureState = ContextState.Textures[0];
		glBindTexture(GL_TEXTURE_3D, (TextureState.Target == GL_TEXTURE_3D) ? TextureState.Resource : 0);
		glActiveTexture( GL_TEXTURE0 + ContextState.ActiveTexture );
		glDeleteFramebuffers( 1, &SourceFramebuffer);
		FOpenGL::DeleteTextures( 1, &TempTexture );
		ContextState.Framebuffer = (GLuint)-1;
	});
	RHITHREAD_GLTRACE_BLOCKING;
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
}



void FOpenGLDynamicRHI::BindPendingFramebuffer()
{
	VERIFY_GL_SCOPE();

	check((GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5) || !PendingState.bFramebufferSetupInvalid);

	if (ContextState.Framebuffer != PendingState.Framebuffer)
	{
		if (PendingState.Framebuffer)
		{
			glBindFramebuffer(GL_FRAMEBUFFER, PendingState.Framebuffer);

			FOpenGL::ReadBuffer( PendingState.FirstNonzeroRenderTarget >= 0 ? GL_COLOR_ATTACHMENT0 + PendingState.FirstNonzeroRenderTarget : GL_NONE);
			GLenum DrawFramebuffers[MaxSimultaneousRenderTargets];
			const GLint MaxDrawBuffers = GMaxOpenGLDrawBuffers;

			for (int32 RenderTargetIndex = 0; RenderTargetIndex < MaxDrawBuffers; ++RenderTargetIndex)
			{
				DrawFramebuffers[RenderTargetIndex] = PendingState.RenderTargets[RenderTargetIndex] ? GL_COLOR_ATTACHMENT0 + RenderTargetIndex : GL_NONE;
			}
			FOpenGL::DrawBuffers(MaxDrawBuffers, DrawFramebuffers);
		}
		else
		{
			glBindFramebuffer(GL_FRAMEBUFFER, 0);
			FOpenGL::ReadBuffer(GL_BACK);
			FOpenGL::DrawBuffer(GL_BACK);
		}

		ContextState.Framebuffer = PendingState.Framebuffer;
	}
}

// Replaces RenderTargets with ResoveTargets to utilize GL_EXT_multisampled_render_to_texture
static int32 SetupMultisampleRenderingInfo(FRHISetRenderTargetsInfo& RTInfo)
{
	if (FOpenGL::GetMaxMSAASamplesTileMem() > 1 && RTInfo.NumColorRenderTargets > 0)
	{
		int32 NumRenderingSamples = RTInfo.ColorRenderTarget[0].Texture->GetDesc().NumSamples;
		if (NumRenderingSamples > 1)
		{
			for (int32 i = 0; i < RTInfo.NumColorRenderTargets; ++i)
			{
				if (RTInfo.ColorResolveRenderTarget[i].Texture)
				{
					RTInfo.ColorRenderTarget[i].Texture = RTInfo.ColorResolveRenderTarget[i].Texture;
				}
			}
			
			return NumRenderingSamples;
		}
	}
		
	return 1;
}

void FOpenGLDynamicRHI::RHIBeginRenderPass(const FRHIRenderPassInfo& InInfo, const TCHAR* InName)
{
	FRHISetRenderTargetsInfo RTInfo;
	InInfo.ConvertToRenderTargetsInfo(RTInfo);
	// Begin GL_EXT_multisampled_render_to_texture if any
	PendingState.NumRenderingSamples = SetupMultisampleRenderingInfo(RTInfo);
	SetRenderTargetsAndClear(RTInfo);

	RenderPassInfo = InInfo;

#if PLATFORM_ANDROID
	if (RenderPassInfo.SubpassHint == ESubpassHint::DeferredShadingSubpass &&
		 FOpenGL::SupportsPixelLocalStorage() && FOpenGL::SupportsShaderDepthStencilFetch())
	{
		glEnable(GL_SHADER_PIXEL_LOCAL_STORAGE_EXT);
	}

	if (FAndroidOpenGL::RequiresAdrenoTilingModeHint())
	{
		FAndroidOpenGL::EnableAdrenoTilingModeHint(FCString::Strcmp(InName, TEXT("SceneColorRendering")) == 0);
	}

	// Reenable non-coherent framebuffer fetch if needed
	if (!ContextState.bNonCoherentFramebufferFetchEnabled)
	{
		ContextState.bNonCoherentFramebufferFetchEnabled = FAndroidOpenGL::ResetNonCoherentFramebufferFetch();
	}
#endif
}

void FOpenGLDynamicRHI::RHIEndRenderPass()
{
	// End GL_EXT_multisampled_render_to_texture
	PendingState.NumRenderingSamples = 1;

	// Discard transient color targets
	uint32 ColorMask = 0u;
	for (int32 ColorIndex = 0; ColorIndex < MaxSimultaneousRenderTargets; ++ColorIndex)
	{
		const FRHIRenderPassInfo::FColorEntry& Entry = RenderPassInfo.ColorRenderTargets[ColorIndex];
		if (!Entry.RenderTarget)
		{
			break;
		}

		if (GetStoreAction(Entry.Action) == ERenderTargetStoreAction::ENoAction)
		{
			ColorMask |= (1u << ColorIndex);
		}
	}

	// Discard transient DepthStencil
	bool bDiscardDepthStencil = false;
	if (RenderPassInfo.DepthStencilRenderTarget.DepthStencilTarget)
	{
		ERenderTargetActions DepthActions = GetDepthActions(RenderPassInfo.DepthStencilRenderTarget.Action);
		bDiscardDepthStencil = GetStoreAction(DepthActions) == ERenderTargetStoreAction::ENoAction;
	}

	if (bDiscardDepthStencil || ColorMask != 0)
	{
		DiscardRenderTargets(bDiscardDepthStencil, bDiscardDepthStencil, ColorMask);
	}

	FRHIRenderTargetView RTV(nullptr, ERenderTargetLoadAction::ENoAction);
	FRHIDepthRenderTargetView DepthRTV(nullptr, ERenderTargetLoadAction::ENoAction, ERenderTargetStoreAction::ENoAction);
	SetRenderTargets(1, &RTV, &DepthRTV);

#if PLATFORM_ANDROID
	if (RenderPassInfo.SubpassHint == ESubpassHint::DeferredShadingSubpass &&
		FOpenGL::SupportsPixelLocalStorage() && FOpenGL::SupportsShaderDepthStencilFetch())
	{
		glDisable(GL_SHADER_PIXEL_LOCAL_STORAGE_EXT);
	}
#endif
}

void FOpenGLDynamicRHI::RHINextSubpass()
{
	IRHICommandContext::RHINextSubpass();
	
	if (RenderPassInfo.SubpassHint == ESubpassHint::DepthReadSubpass ||
		RenderPassInfo.SubpassHint == ESubpassHint::DeferredShadingSubpass)
	{
		if (ContextState.bNonCoherentFramebufferFetchEnabled)
		{
			FOpenGL::FrameBufferFetchBarrier();
		}
	}
}

void FOpenGLDynamicRHI::RHIBeginTransitions(TArrayView<const FRHITransition*> Transitions)
{
}

void FOpenGLDynamicRHI::RHIEndTransitions(TArrayView<const FRHITransition*> Transitions)
{
}
