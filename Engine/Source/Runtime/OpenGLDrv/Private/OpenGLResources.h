// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLResources.h: OpenGL resource RHI definitions.
=============================================================================*/

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "OpenGLUtil.h"
#include "OpenGLPlatform.h"
#include "OpenGL.h"

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "HAL/LowLevelMemTracker.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"
#include "Logging/LogMacros.h"
#include "Containers/BitArray.h"
#include "Math/IntPoint.h"
#include "Misc/CommandLine.h"
#include "Templates/RefCounting.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "BoundShaderStateCache.h"
#include "RenderResource.h"
#include "OpenGLShaderResources.h"
#include "PsoLruCache.h"

#define UE_API OPENGLDRV_API

class FOpenGLDynamicRHI;
class FOpenGLLinkedProgram;
class FOpenGLTexture;
typedef TArray<ANSICHAR> FAnsiCharArray;

namespace OpenGLConsoleVariables
{
	extern int32 bUseMapBuffer;
	extern int32 MaxSubDataSize;
	extern int32 bUseStagingBuffer;
	extern int32 bUseBufferDiscard;
};

#if PLATFORM_WINDOWS
#define RESTRICT_SUBDATA_SIZE 1
#else
#define RESTRICT_SUBDATA_SIZE 0
#endif

namespace OpenGLBufferStats
{
	void UpdateUniformBufferStats(int64 BufferSize, bool bAllocating);
	void UpdateBufferStats(const FRHIBufferDesc& BufferDesc, bool bAllocating);
}

// Extra stats for finer-grained timing
// They shouldn't always be on, as they may impact overall performance
#define OPENGLRHI_DETAILED_STATS 0


#if OPENGLRHI_DETAILED_STATS
	DECLARE_CYCLE_STAT_EXTERN(TEXT("MapBuffer time"),STAT_OpenGLMapBufferTime,STATGROUP_OpenGLRHI, );
	DECLARE_CYCLE_STAT_EXTERN(TEXT("UnmapBuffer time"),STAT_OpenGLUnmapBufferTime,STATGROUP_OpenGLRHI, );
	#define SCOPE_CYCLE_COUNTER_DETAILED(Stat)	SCOPE_CYCLE_COUNTER(Stat)
	#define DETAILED_QUICK_SCOPE_CYCLE_COUNTER(x) QUICK_SCOPE_CYCLE_COUNTER(x)
#else
	#define SCOPE_CYCLE_COUNTER_DETAILED(Stat)
	#define DETAILED_QUICK_SCOPE_CYCLE_COUNTER(x)
#endif

#define GLDEBUG_LABELS_ENABLED (!UE_BUILD_SHIPPING)

class FOpenGLViewableResource
{
public:
	~FOpenGLViewableResource()
	{
		checkf(!HasLinkedViews(), TEXT("All linked views must have been removed before the underlying resource can be deleted."));
	}

	bool HasLinkedViews() const
	{
		return LinkedViews != nullptr;
	}

	void UpdateLinkedViews();

private:
	friend class FOpenGLShaderResourceView;
	friend class FOpenGLUnorderedAccessView;
	class FOpenGLView* LinkedViews = nullptr;
};

class FOpenGLBufferBase
{
protected:
	FOpenGLBufferBase(GLenum Type)
		: Type(Type)
	{}

public:
	GLenum Type;
	GLuint Resource = 0;

	void Bind();
	void OnBufferDeletion();
};

template <typename BaseType>
class TOpenGLBuffer : public BaseType, public FOpenGLBufferBase
{
	void LoadData(uint32 InOffset, uint32 InSize, const void* InData)
	{
		VERIFY_GL_SCOPE();
		const uint8* Data = (const uint8*)InData;
		const uint32 BlockSize = OpenGLConsoleVariables::MaxSubDataSize;

		if (BlockSize > 0)
		{
			while ( InSize > 0)
			{
				const uint32 BufferSize = FMath::Min<uint32>( BlockSize, InSize);

				FOpenGL::BufferSubData( Type, InOffset, BufferSize, Data);

				InOffset += BufferSize;
				InSize -= BufferSize;
				Data += BufferSize;
			}
		}
		else
		{
			FOpenGL::BufferSubData( Type, InOffset, InSize, InData);
		}
	}

	GLenum GetAccess()
	{
		// Previously there was special-case logic to always use GL_STATIC_DRAW for vertex buffers allocated from staging buffer.
		// However it seems to be incorrect as NVidia drivers complain (via debug output callback) about VIDEO->HOST copying for buffers with such hints
		return IsDynamic() ? GL_DYNAMIC_DRAW : GL_STATIC_DRAW;
	}

public:
	TOpenGLBuffer(FRHICommandListBase* RHICmdList, GLenum InType, const FRHIBufferCreateDesc& CreateDesc, const void *InData)
		: BaseType(CreateDesc)
		, FOpenGLBufferBase(InType)
	{
		if (RHICmdList && RHICmdList->IsTopOfPipe() && InData)
		{
			InData = RHICmdList->AllocCopy(InData, CreateDesc.Size, 16);
		}

		auto InitLambda = [this, InData]()
		{
			VERIFY_GL_SCOPE();

			FOpenGL::GenBuffers(1, &Resource);

			Bind();

#if !RESTRICT_SUBDATA_SIZE
			glBufferData(Type, BaseType::GetSize(), InData, GetAccess());
#else
			glBufferData(Type, BaseType::GetSize(), nullptr, GetAccess());
			if (InData != nullptr)
			{
				LoadData(0, BaseType::GetSize(), InData);
			}
#endif
			BaseType::UpdateBufferStats(true);
		};

		if (!CreateDesc.IsNull())
		{
			if (RHICmdList)
			{
				RHICmdList->EnqueueLambda([Lambda = MoveTemp(InitLambda)](FRHICommandListBase&) { Lambda(); });
			}
			else
			{
				InitLambda();
			}
		}
	}

	virtual ~TOpenGLBuffer()
	{
		ReleaseOwnership();
	}

	uint8* Lock(uint32 InOffset, uint32 InSize, bool bReadOnly, bool bDiscard)
	{
		SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLMapBufferTime);
		check(InOffset + InSize <= BaseType::GetSize());
		//check( LockBuffer == NULL );	// Only one outstanding lock is allowed at a time!
		check( !bIsLocked );	// Only one outstanding lock is allowed at a time!
		VERIFY_GL_SCOPE();

		Bind();

		bIsLocked = true;
		bIsLockReadOnly = bReadOnly;
		uint8 *Data = NULL;

		// Discard if the input size is the same as the backing store size, regardless of the input argument, as orphaning the backing store will typically be faster.
		bDiscard = (bDiscard || (!bReadOnly && InSize == BaseType::GetSize())) && FOpenGL::DiscardFrameBufferToResize();

		// Map buffer is faster in some circumstances and slower in others, decide when to use it carefully.
		bool const bUseMapBuffer = (bReadOnly || OpenGLConsoleVariables::bUseMapBuffer);

		// If we're able to discard the current data, do so right away
		// If we can then we should orphan the buffer name & reallocate the backing store only once as calls to glBufferData may do so even when the size is the same.
		uint32 DiscardSize = (bDiscard && !bUseMapBuffer && InSize == BaseType::GetSize() && !RESTRICT_SUBDATA_SIZE) ? 0 : BaseType::GetSize();

		if (bDiscard && OpenGLConsoleVariables::bUseBufferDiscard)
		{
			glBufferData(Type, DiscardSize, NULL, GetAccess());
		}

		if ( bUseMapBuffer)
		{
			FOpenGL::EResourceLockMode LockMode = bReadOnly ? FOpenGL::EResourceLockMode::RLM_ReadOnly : FOpenGL::EResourceLockMode::RLM_WriteOnly;
			Data = static_cast<uint8*>( FOpenGL::MapBufferRange( Type, InOffset, InSize, LockMode ) );
//			checkf(Data != NULL, TEXT("FOpenGL::MapBufferRange Failed, glError %d (0x%x)"), glGetError(), glGetError());

			LockOffset = InOffset;
			LockSize = InSize;
			LockBuffer = Data;
			bLockBufferWasAllocated = false;
		}
		else
		{
			// Allocate a temp buffer to write into
			LockOffset = InOffset;
			LockSize = InSize;
			if (CachedBuffer && InSize <= CachedBufferSize)
			{
				LockBuffer = CachedBuffer;
				CachedBuffer = nullptr;
				// Keep CachedBufferSize to keep the actual size allocated.
			}
			else
			{
				ReleaseCachedBuffer();
				LockBuffer = FMemory::Malloc( InSize );
				CachedBufferSize = InSize; // Safegard
			}
			Data = static_cast<uint8*>( LockBuffer );
			bLockBufferWasAllocated = true;
		}

		if (Data == nullptr)
		{
			UE_LOGF(LogRHI, Fatal, "Failed to lock buffer: Resource %u, Size %u, Offset %u, bReadOnly %d, bUseMapBuffer %d, glError (0x%x)", 
				(uint32)Resource, 
				InSize, 
				InOffset, 
				bReadOnly ? 1:0, 
				bUseMapBuffer ? 1:0, 
				glGetError());
		}
		
		return Data;
	}

	uint8* LockWriteOnlyUnsynchronized(uint32 InOffset, uint32 InSize, bool bDiscard)
	{
		//SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLMapBufferTime);
		check(InOffset + InSize <= BaseType::GetSize());
		//check( LockBuffer == NULL );	// Only one outstanding lock is allowed at a time!
		check( !bIsLocked );	// Only one outstanding lock is allowed at a time!
		VERIFY_GL_SCOPE();

		Bind();

		bIsLocked = true;
		bIsLockReadOnly = false;
		uint8 *Data = NULL;

		// Discard if the input size is the same as the backing store size, regardless of the input argument, as orphaning the backing store will typically be faster.
		bDiscard = (bDiscard || InSize == BaseType::GetSize()) && FOpenGL::DiscardFrameBufferToResize();

		// Map buffer is faster in some circumstances and slower in others, decide when to use it carefully.
		bool const bUseMapBuffer = OpenGLConsoleVariables::bUseMapBuffer;

		// If we're able to discard the current data, do so right away
		// If we can then we should orphan the buffer name & reallocate the backing store only once as calls to glBufferData may do so even when the size is the same.
		uint32 DiscardSize = (bDiscard && !bUseMapBuffer && InSize == BaseType::GetSize() && !RESTRICT_SUBDATA_SIZE) ? 0 : BaseType::GetSize();

		if (bDiscard && OpenGLConsoleVariables::bUseBufferDiscard)
		{
			glBufferData( Type, DiscardSize, NULL, GetAccess());
		}

		if ( bUseMapBuffer)
		{
			FOpenGL::EResourceLockMode LockMode = bDiscard ? FOpenGL::EResourceLockMode::RLM_WriteOnly : FOpenGL::EResourceLockMode::RLM_WriteOnlyUnsynchronized;
			Data = static_cast<uint8*>( FOpenGL::MapBufferRange( Type, InOffset, InSize, LockMode ) );
			LockOffset = InOffset;
			LockSize = InSize;
			LockBuffer = Data;
			bLockBufferWasAllocated = false;
		}
		else
		{
			// Allocate a temp buffer to write into
			LockOffset = InOffset;
			LockSize = InSize;
			if (CachedBuffer && InSize <= CachedBufferSize)
			{
				LockBuffer = CachedBuffer;
				CachedBuffer = nullptr;
				// Keep CachedBufferSize to keep the actual size allocated.
			}
			else
			{
				ReleaseCachedBuffer();
				LockBuffer = FMemory::Malloc( InSize );
				CachedBufferSize = InSize; // Safegard
			}
			Data = static_cast<uint8*>( LockBuffer );
			bLockBufferWasAllocated = true;
		}

		if (Data == nullptr)
		{
			UE_LOGF(LogRHI, Fatal, "Failed to lock buffer (write only): Resource %u, Size %u, Offset %u, bUseMapBuffer %d, glError (0x%x)", 
				(uint32)Resource, 
				InSize, 
				InOffset, 
				bUseMapBuffer ? 1:0, 
				glGetError());
		}

		return Data;
	}

	void Unlock()
	{
		//SCOPE_CYCLE_COUNTER_DETAILED(STAT_OpenGLUnmapBufferTime);
		VERIFY_GL_SCOPE();
		if (bIsLocked)
		{
			Bind();

			if (OpenGLConsoleVariables::bUseMapBuffer || bIsLockReadOnly)
			{
				check(!bLockBufferWasAllocated);
				FOpenGL::UnmapBufferRange(Type, LockOffset, LockSize);
				LockBuffer = NULL;
			}
			else
			{
#if !RESTRICT_SUBDATA_SIZE
				// Check for the typical, optimized case
				if(LockSize == BaseType::GetSize())
				{
					if (FOpenGL::DiscardFrameBufferToResize())
					{
						glBufferData(Type, BaseType::GetSize(), LockBuffer, GetAccess());
					}
					else
					{
						FOpenGL::BufferSubData(Type, 0, LockSize, LockBuffer);
					}
					check( LockBuffer != NULL );
				}
				else
				{
					// Only updating a subset of the data
					FOpenGL::BufferSubData(Type, LockOffset, LockSize, LockBuffer);
					check( LockBuffer != NULL );
				}
#else
				LoadData(LockOffset, LockSize, LockBuffer);
				check(LockBuffer != NULL);
#endif

				check(bLockBufferWasAllocated);

				if (EnumHasAnyFlags(this->GetUsage(), BUF_Volatile))
				{
					ReleaseCachedBuffer(); // Safegard

					CachedBuffer = LockBuffer;
					// Possibly > LockSize when reusing cached allocation.
					CachedBufferSize = FMath::Max<GLuint>(CachedBufferSize, LockSize);
				}
				else
				{
					FMemory::Free(LockBuffer);
				}
				LockBuffer = NULL;
				bLockBufferWasAllocated = false;
				LockSize = 0;
			}
			bIsLocked = false;
		}
	}

	void Update(void *InData, uint32 InOffset, uint32 InSize, bool bDiscard)
	{
		check(InOffset + InSize <= BaseType::GetSize());
		VERIFY_GL_SCOPE();
		Bind();
#if !RESTRICT_SUBDATA_SIZE
		FOpenGL::BufferSubData(Type, InOffset, InSize, InData);
#else
		LoadData(InOffset, InSize, InData);
#endif
	}

	bool IsDynamic() const { return EnumHasAnyFlags(this->GetUsage(), BUF_AnyDynamic); }
	bool IsLocked() const { return bIsLocked; }
	bool IsLockReadOnly() const { return bIsLockReadOnly; }
	void* GetLockedBuffer() const { return LockBuffer; }

	void ReleaseCachedBuffer()
	{
		if (CachedBuffer)
		{
			FMemory::Free(CachedBuffer);
			CachedBuffer = nullptr;
			CachedBufferSize = 0;
		}
		// Don't reset CachedBufferSize if !CachedBuffer since it could be the locked buffer allocation size.
	}

	void TakeOwnership(TOpenGLBuffer& Other)
	{
		VERIFY_GL_SCOPE();
		check(!bIsLocked && !Other.bIsLocked);

		ReleaseOwnership();

		BaseType::TakeOwnership(Other);

		Type             = Other.Type;
		Resource         = Other.Resource;
		CachedBuffer     = Other.CachedBuffer;
		CachedBufferSize = Other.CachedBufferSize;

		Other.Type             = 0;
		Other.Resource         = 0;
		Other.CachedBuffer     = nullptr;
		Other.CachedBufferSize = 0;
	}

	void ReleaseOwnership()
	{
		VERIFY_GL_SCOPE();

		if (Resource != 0)
		{
			if (LockBuffer != NULL)
			{
				if (bLockBufferWasAllocated)
				{
					FMemory::Free(LockBuffer);
				}
				else
				{
					UE_LOGF(LogRHI, Warning, "Destroying TOpenGLBuffer without returning memory to the driver; possibly called RHIMapStagingSurface() but didn't call RHIUnmapStagingSurface()? Resource %u", Resource);
				}
			}

			OnBufferDeletion();

			FOpenGL::DeleteBuffers(1, &Resource);
			Resource = 0;

			LockBuffer = nullptr;
			BaseType::UpdateBufferStats(false);

			ReleaseCachedBuffer();
		}

		BaseType::ReleaseOwnership();
	}

private:
	uint32 bIsLocked               : 1 = false;
	uint32 bIsLockReadOnly         : 1 = false;
	uint32 bLockBufferWasAllocated : 1 = false;

	GLuint LockSize = 0;
	GLuint LockOffset = 0;
	void* LockBuffer = nullptr;

	// A cached allocation that can be reused. The same allocation can never be in CachedBuffer and LockBuffer at the same time.
	void* CachedBuffer = nullptr;
	// The size of the cached buffer allocation. Can be non zero even though CachedBuffer is  null, to preserve the allocation size.
	GLuint CachedBufferSize = 0;
};

class FOpenGLBasePixelBuffer : public FRefCountedObject
{
public:
	FOpenGLBasePixelBuffer(const FRHIBufferCreateDesc& CreateDesc)
	: Desc(CreateDesc)
	{}

	const FRHIBufferDesc& GetDesc() const { return Desc; }
	uint32 GetSize() const { return Desc.Size; }
	EBufferUsageFlags GetUsage() const { return Desc.Usage; }

	void TakeOwnership(FOpenGLBasePixelBuffer& Other)
	{
		Desc = Other.Desc;
		Other.Desc = FRHIBufferDesc::Null();
	}

	void ReleaseOwnership()
	{
		Desc = FRHIBufferDesc::Null();
	}

private:
	FRHIBufferDesc Desc;
	
protected:
	void UpdateBufferStats(bool bAllocating)
	{
		OpenGLBufferStats::UpdateBufferStats(Desc, bAllocating);
	}
};

class FOpenGLBaseBuffer : public FRHIBuffer, public FOpenGLViewableResource
{
public:
	FOpenGLBaseBuffer(const FRHIBufferCreateDesc& InCreateDesc)
		: FRHIBuffer(InCreateDesc)
	{
	}


protected:
	void UpdateBufferStats(bool bAllocating)
	{
		int64 AllocSize = ( bAllocating ? 1 : -1 ) * static_cast<int64>( GetSize() );
		LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(ELLMTag::GraphicsPlatform, AllocSize, ELLMTracker::Platform, ELLMAllocType::None);
		LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(ELLMTag::Meshes, AllocSize, ELLMTracker::Default, ELLMAllocType::None);
		
		OpenGLBufferStats::UpdateBufferStats(GetDesc(), bAllocating);
	}
};

typedef TOpenGLBuffer<FOpenGLBasePixelBuffer> FOpenGLPixelBuffer;
typedef TOpenGLBuffer<FOpenGLBaseBuffer     > FOpenGLBuffer;

struct FOpenGLEUniformBufferData : public FRefCountedObject
{
	FOpenGLEUniformBufferData(uint32 SizeInBytes)
	{
		uint32 SizeInUint32s = (SizeInBytes + 3) / 4;
		Data.Empty(SizeInUint32s);
		Data.AddUninitialized(SizeInUint32s);
		OpenGLBufferStats::UpdateUniformBufferStats(Data.GetAllocatedSize(), true);
	}

	~FOpenGLEUniformBufferData()
	{
		OpenGLBufferStats::UpdateUniformBufferStats(Data.GetAllocatedSize(), false);
	}

	TArray<uint32> Data;
};
typedef TRefCountPtr<FOpenGLEUniformBufferData> FOpenGLEUniformBufferDataRef;

class FOpenGLUniformBuffer : public FRHIUniformBuffer
{
public:
	/** The GL resource for this uniform buffer. */
	GLuint Resource;

	/** The offset of the uniform buffer's contents in the resource. */
	uint32 Offset;

	/** The data range size of uniform buffer's contents in the resource. */
	uint32 RangeSize;

	/** When using a persistently mapped buffer this is a pointer to the CPU accessible data. */
	uint8* PersistentlyMappedBuffer;

	/** Unique ID for state shadowing purposes. */
	uint32 UniqueID;

	/** Emulated uniform data for ES2. */
	FOpenGLEUniformBufferDataRef EmulatedBufferData;

	/** The size of the buffer allocated to hold the uniform buffer contents. May be larger than necessary. */
	uint32 AllocatedSize;

	/** True if the uniform buffer is not used across frames. */
	bool bStreamDraw;

	/** True if the uniform buffer is emulated */
	bool bIsEmulatedUniformBuffer;

	/** True if Resource belongs to this UniformBuffer */
	bool bOwnsResource;

	/** Initialization constructor. */
	FOpenGLUniformBuffer(const FRHIUniformBufferLayout* InLayout);

	void SetGLUniformBufferParams(GLuint InResource, uint32 InOffset, uint8* InPersistentlyMappedBuffer, uint32 InAllocatedSize, FOpenGLEUniformBufferDataRef InEmulatedBuffer, bool bInStreamDraw);

	/** Destructor. */
	~FOpenGLUniformBuffer();

	// Provides public non-const access to ResourceTable.
	// @todo refactor uniform buffers to perform updates as a member function, so this isn't necessary.
	TArray<TRefCountPtr<FRHIResource>>& GetResourceTable() { return ResourceTable; }

	void SetLayoutTable(const void* Contents, EUniformBufferValidation Validation);
};

#define MAX_STREAMED_BUFFERS_IN_ARRAY 2	// must be > 1!
#define MIN_DRAWS_IN_SINGLE_BUFFER 16

template <typename BaseType, uint32 Stride>
class TOpenGLStreamedBufferArray
{
public:

	TOpenGLStreamedBufferArray( void ) {}
	virtual ~TOpenGLStreamedBufferArray( void ) = default;

	void Init( uint32 InitialBufferSize )
	{
		for( int32 BufferIndex = 0; BufferIndex < MAX_STREAMED_BUFFERS_IN_ARRAY; ++BufferIndex )
		{
			Buffer[BufferIndex] = new BaseType(Stride, InitialBufferSize, BUF_Volatile, NULL, true);
		}
		CurrentBufferIndex = 0;
		CurrentOffset = 0;
		LastOffset = 0;
		MinNeededBufferSize = InitialBufferSize / MIN_DRAWS_IN_SINGLE_BUFFER;
	}

	void Cleanup( void )
	{
		for( int32 BufferIndex = 0; BufferIndex < MAX_STREAMED_BUFFERS_IN_ARRAY; ++BufferIndex )
		{
			Buffer[BufferIndex].SafeRelease();
		}
	}

	uint8* Lock( uint32 DataSize )
	{
		check(!Buffer[CurrentBufferIndex]->IsLocked());
		DataSize = Align(DataSize, (1<<8));	// to keep the speed up, let's start data for each next draw at 256-byte aligned offset

		// Keep our dynamic buffers at least MIN_DRAWS_IN_SINGLE_BUFFER times bigger than max single request size
		uint32 NeededBufSize = Align( MIN_DRAWS_IN_SINGLE_BUFFER * DataSize, (1 << 20) );	// 1MB increments
		if (NeededBufSize > MinNeededBufferSize)
		{
			MinNeededBufferSize = NeededBufSize;
		}

		// Check if we need to switch buffer, as the current draw data won't fit in current one
		bool bDiscard = false;
		if (Buffer[CurrentBufferIndex]->GetSize() < CurrentOffset + DataSize)
		{
			// We do.
			++CurrentBufferIndex;
			if (CurrentBufferIndex == MAX_STREAMED_BUFFERS_IN_ARRAY)
			{
				CurrentBufferIndex = 0;
			}
			CurrentOffset = 0;

			// Check if we should extend the next buffer, as max request size has changed
			if (MinNeededBufferSize > Buffer[CurrentBufferIndex]->GetSize())
			{
				Buffer[CurrentBufferIndex].SafeRelease();
				Buffer[CurrentBufferIndex] = new BaseType(Stride, MinNeededBufferSize, BUF_Volatile);
			}

			bDiscard = true;
		}

		LastOffset = CurrentOffset;
		CurrentOffset += DataSize;

		return Buffer[CurrentBufferIndex]->LockWriteOnlyUnsynchronized(LastOffset, DataSize, bDiscard);
	}

	void Unlock( void )
	{
		check(Buffer[CurrentBufferIndex]->IsLocked());
		Buffer[CurrentBufferIndex]->Unlock();
	}

	BaseType* GetPendingBuffer( void ) { return Buffer[CurrentBufferIndex]; }
	uint32 GetPendingOffset( void ) { return LastOffset; }

private:
	TRefCountPtr<BaseType> Buffer[MAX_STREAMED_BUFFERS_IN_ARRAY];
	uint32 CurrentBufferIndex;
	uint32 CurrentOffset;
	uint32 LastOffset;
	uint32 MinNeededBufferSize;
};

struct FOpenGLVertexElement
{
	GLenum Type;
	GLuint StreamIndex;
	GLuint Offset;
	GLuint Size;
	GLuint Divisor;
	GLuint HashStride;
	uint8 bNormalized;
	uint8 AttributeIndex;
	uint8 bShouldConvertToFloat;
	uint8 Padding;

	FOpenGLVertexElement()
		: Padding(0)
	{
	}
};

/** Convenience typedef: preallocated array of OpenGL input element descriptions. */
typedef TArray<FOpenGLVertexElement,TFixedAllocator<MaxVertexElementCount> > FOpenGLVertexElements;

/** This represents a vertex declaration that hasn't been combined with a specific shader to create a bound shader. */
class FOpenGLVertexDeclaration : public FRHIVertexDeclaration
{
public:
	/** Elements of the vertex declaration. */
	FOpenGLVertexElements VertexElements;

	uint16 StreamStrides[MaxVertexElementCount];

	/** Initialization constructor. */
	FOpenGLVertexDeclaration(const FOpenGLVertexElements& InElements, const uint16* InStrides)
		: VertexElements(InElements)
	{
		FMemory::Memcpy(StreamStrides, InStrides, sizeof(StreamStrides));
	}
	
	virtual bool GetInitializer(FVertexDeclarationElementList& Init) override final;
};


/**
 * Combined shader state and vertex definition for rendering geometry.
 * Each unique instance consists of a vertex decl, vertex shader, and pixel shader.
 */
class FOpenGLBoundShaderState : public FRHIBoundShaderState
{
	static FOpenGLLinkedProgram* FindOrCreateLinkedProgram(FOpenGLVertexShader* VertexShader, FOpenGLPixelShader* PixelShader, FOpenGLGeometryShader* GeometryShader);

public:

	FCachedBoundShaderStateLink CacheLink;

	uint16 StreamStrides[MaxVertexElementCount];

	FOpenGLLinkedProgram* const LinkedProgram;
	TRefCountPtr<FOpenGLVertexDeclaration> const VertexDeclaration;
	TRefCountPtr<FOpenGLVertexShader     > const VertexShader;
	TRefCountPtr<FOpenGLPixelShader      > const PixelShader;
	TRefCountPtr<FOpenGLGeometryShader   > const GeometryShader;

	/** Initialization constructor. */
	FOpenGLBoundShaderState(
		FOpenGLVertexDeclaration* InVertexDeclarationRHI,
		FOpenGLVertexShader* InVertexShaderRHI,
		FOpenGLPixelShader* InPixelShaderRHI,
		FOpenGLGeometryShader* InGeometryShaderRHI
		);

	const TBitArray<>& GetTextureNeeds(int32& OutMaxTextureStageUsed);
	const TBitArray<>& GetUAVNeeds(int32& OutMaxUAVUnitUsed) const;
	void GetNumUniformBuffers(int32 NumVertexUniformBuffers[SF_NumGraphicsFrequencies]);

	bool NeedsTextureStage(int32 TextureStageIndex);
	int32 MaxTextureStageUsed();
	bool RequiresDriverInstantiation();

	FOpenGLVertexShader* GetVertexShader()
	{
		check(IsValidRef(VertexShader));
		return VertexShader;
	}

	FOpenGLPixelShader* GetPixelShader()
	{
		check(IsValidRef(PixelShader));
		return PixelShader;
	}

	FOpenGLGeometryShader* GetGeometryShader()
	{
		return GeometryShader;
	}

	virtual ~FOpenGLBoundShaderState();
};

class FTextureEvictionLRU
{
private:
	typedef TPsoLruCache<FOpenGLTexture*, FOpenGLTexture*> FOpenGLTextureLRUContainer;
	FCriticalSection TextureLRULock;

	static FORCEINLINE_DEBUGGABLE FOpenGLTextureLRUContainer& GetLRUContainer()
	{
		const int32 MaxNumLRUs = 10000;
		static FOpenGLTextureLRUContainer TextureLRU(MaxNumLRUs);
		return TextureLRU;
	}

public:

	static FORCEINLINE_DEBUGGABLE FTextureEvictionLRU& Get()
	{
		static FTextureEvictionLRU Lru;
		return Lru;
	}
	uint32 Num() const { return GetLRUContainer().Num(); }

	void Remove(FOpenGLTexture* TextureBase);
	bool Add(FOpenGLTexture* TextureBase);
	void Touch(FOpenGLTexture* TextureBase);
	void TickEviction();
	FOpenGLTexture* GetLeastRecent();
};
class FTextureEvictionParams
{
public:
	FTextureEvictionParams(uint32 NumMips);
	~FTextureEvictionParams();
	TArray<TArray<uint8>> MipImageData;

 	uint32 bHasRestored : 1;	
	FSetElementId LRUNode;
	uint32 FrameLastRendered;

#if GLDEBUG_LABELS_ENABLED
	FAnsiString TextureDebugName;
	void SetDebugLabelName(const FAnsiString& TextureDebugNameIn) { TextureDebugName = TextureDebugNameIn; }
	void SetDebugLabelName(const ANSICHAR * TextureDebugNameIn) { TextureDebugName.Append(TextureDebugNameIn, FCStringAnsi::Strlen(TextureDebugNameIn) + 1); }
	FAnsiString& GetDebugLabelName() { return TextureDebugName; }
#else
	void SetDebugLabelName(const FAnsiString& TextureDebugNameIn) { checkNoEntry(); }
	FAnsiCharArray& GetDebugLabelName() { checkNoEntry(); static FAnsiCharArray Dummy;  return Dummy; }
#endif

	void SetMipData(uint32 MipIndex, const void* Data, uint32 Bytes);
	void ReleaseMipData(uint32 RetainMips);

	void CloneMipData(const FTextureEvictionParams& Src, uint32 NumMips, int32 SrcOffset, int DstOffset);

	uint32 GetTotalAllocated() const
	{
		uint32 TotalAllocated = 0;
		for (const auto& MipData : MipImageData)
		{
			TotalAllocated += MipData.Num();
		}
		return TotalAllocated;
	}

	bool AreAllMipsPresent() const
	{
		bool bRet = MipImageData.Num() > 0;
		for (const auto& MipData : MipImageData)
		{
			bRet = bRet && MipData.Num() > 0;
		}
		return bRet;
	}
};

class FOpenGLTextureDesc
{
public:
	FOpenGLTextureDesc(FRHITextureDesc const& InDesc);

	GLenum Target = GL_NONE;
	GLenum Attachment = GL_NONE;

	uint32 MemorySize = 0;

	uint8 bCubemap            : 1;
	uint8 bArrayTexture       : 1;
	uint8 bStreamable         : 1;
	uint8 bDepthStencil       : 1;
	uint8 bCanCreateAsEvicted : 1;
	uint8 bIsPowerOfTwo       : 1;
	uint8 bMultisampleRenderbuffer : 1;

private:
	static bool CanDeferTextureCreation();
};

class FOpenGLTextureCreateDesc : public FRHITextureCreateDesc, public FOpenGLTextureDesc
{
public:
	FOpenGLTextureCreateDesc(FRHITextureCreateDesc const& CreateDesc)
		: FRHITextureCreateDesc(CreateDesc)
		, FOpenGLTextureDesc(CreateDesc)
	{
	}
};

class FOpenGLTexture : public FRHITexture, public FOpenGLViewableResource
{
	// Prevent copying
	FOpenGLTexture(FOpenGLTexture const&) = delete;
	FOpenGLTexture& operator = (FOpenGLTexture const&) = delete;

public:
	// Standard constructor.
	UE_API explicit FOpenGLTexture(FOpenGLTextureCreateDesc const& CreateDesc);
#if PLATFORM_ANDROID
	// Constructor for external hardware buffer.
	UE_API explicit FOpenGLTexture(FRHICommandListBase& RHICmdList, FOpenGLTextureCreateDesc const& CreateDesc, const AHardwareBuffer_Desc& HardwareBufferDesc, AHardwareBuffer* HardwareBuffer);
#endif

	// Constructor for external resources (RHICreateTexture2DFromResource etc).
	UE_API explicit FOpenGLTexture(FOpenGLTextureCreateDesc const& CreateDesc, GLuint Resource);

	// Constructor for RHICreateAliasedTexture
	enum EAliasConstructorParam { AliasResource };
	UE_API explicit FOpenGLTexture(FOpenGLTexture& Texture, const FString& Name, EAliasConstructorParam);

	UE_API virtual ~FOpenGLTexture();

	UE_API void Initialize(FRHICommandListBase& RHICmdList);
	UE_API void AliasResources(FOpenGLTexture& Texture);

	GLuint GetResource()
	{
		TryRestoreGLResource();
		return Resource;
	}

	GLuint& GetResourceRef() 
	{ 
		VERIFY_GL_SCOPE();
		TryRestoreGLResource();
		return Resource;
	}

	// GetRawResourceName - A const accessor to the resource name, this could potentially be an evicted resource.
	// It will not trigger the GL resource's creation.
	GLuint GetRawResourceName() const
	{
		return Resource;
	}

	// GetRawResourceNameRef - A const accessor to the resource name, this could potentially be an evicted resource.
	// It will not trigger the GL resource's creation.
	const GLuint& GetRawResourceNameRef() const
	{
		return Resource;
	}

	void SetResource(GLuint InResource)
	{
		VERIFY_GL_SCOPE();
		Resource = InResource;
	}

	bool IsEvicted() const { VERIFY_GL_SCOPE(); return EvictionParamsPtr.IsValid() && !EvictionParamsPtr->bHasRestored; }

	virtual void* GetTextureBaseRHI() override final
	{
		return this;
	}

	/**
	 * Locks one of the texture's mip-maps.
	 * @return A pointer to the specified texture data.
	 */
	UE_API void* Lock(uint32 MipIndex, uint32 ArrayIndex, EResourceLockMode LockMode, uint32& DestStride);

	/**
	* Returns the size of the memory block that is returned from Lock, threadsafe
	*/
	UE_API uint32 GetLockSize(uint32 MipIndex, EResourceLockMode LockMode, uint32& DestStride);

	/** Unlocks a previously locked mip-map. */
	UE_API void Unlock(uint32 MipIndex, uint32 ArrayIndex);

	/** FRHITexture override.  See FRHITexture::GetNativeResource() */
	virtual void* GetNativeResource() const override
	{
		// this must become a full GL resource here, calling the non-const GetResourceRef ensures this.
		return const_cast<void*>(reinterpret_cast<const void*>(&const_cast<FOpenGLTexture*>(this)->GetResourceRef()));
	}

	/**
	 * Accessors to mark whether or not we have allocated storage for each mip/face.
	 * For non-cubemaps FaceIndex should always be zero.
	 */
	bool GetAllocatedStorageForMip(uint32 MipIndex, uint32 FaceIndex) const
	{
		return bAllocatedStorage[MipIndex * (bCubemap ? 6 : 1) + FaceIndex];
	}
	void SetAllocatedStorageForMip(uint32 MipIndex, uint32 FaceIndex)
	{
		bAllocatedStorage[MipIndex * (bCubemap ? 6 : 1) + FaceIndex] = true;
	}

	// Set allocated storage state for all mip/faces
	void SetAllocatedStorage(bool bInAllocatedStorage)
	{
		bAllocatedStorage.Init(bInAllocatedStorage, this->GetNumMips() * (bCubemap ? 6 : 1));
	}

	/**
	 * Resolved the specified face for a read Lock, for non-renderable, CPU readable surfaces this eliminates the readback inside Lock itself.
	 */
	UE_API void Resolve(uint32 MipIndex, uint32 ArrayIndex);

	// Texture eviction
	UE_API void RestoreEvictedGLResource(bool bAttemptToRetainMips);
	UE_API bool CanBeEvicted();
	UE_API void TryEvictGLResource();
	UE_API void RemoveEvictionData();

private:
	static UE_API void UpdateTextureStats(FOpenGLTexture* Texture, bool bAllocating);

	void TryRestoreGLResource()
	{
		if (EvictionParamsPtr.IsValid())
		{
			VERIFY_GL_SCOPE();
			if (!EvictionParamsPtr->bHasRestored)
			{
				RestoreEvictedGLResource(true);
			}
			else
			{
				if(CanBeEvicted())
				{
					FTextureEvictionLRU::Get().Touch(this);
				}
			}
		}
	}

	UE_API void DeleteGLResource();

	UE_API void FillGLTextureImage(const FOpenGLTextureFormat& GLFormat, bool bSRGB, uint32 MipIndex, uint32 ArrayIndex, const FUintVector3& MipExtents, const void* BufferOrPBOOffset, size_t ImageSize);

public:
	uint32 GetEffectiveSizeZ() const
	{
		FRHITextureDesc const& Desc = GetDesc();

		return Desc.IsTexture3D()
			? Desc.Depth
			: Desc.ArraySize;
	}

private:
	/** The OpenGL texture resource. */
	GLuint Resource = GL_NONE;

public:
	/** The OpenGL texture target. */
	GLenum const Target = 0;

	/** The OpenGL attachment point. This should always be GL_COLOR_ATTACHMENT0 in case of color buffer, but the actual texture may be attached on other color attachments. */
	GLenum const Attachment = 0;

	TUniquePtr<FTextureEvictionParams> EvictionParamsPtr;

	// Pointer to current sampler state in this unit
	class FOpenGLSamplerState* SamplerState = nullptr;

	TArray<TRefCountPtr<FOpenGLPixelBuffer>> PixelBuffers;

	bool bInStencilTextureMode = false;

#if PLATFORM_ANDROID
	// The image created from an external hardware buffer.
	EGLImageKHR HardwareBufferImage;
#endif //PLATFORM_ANDROID

private:
	/** Bitfields marking whether we have allocated storage for each mip */
	TBitArray<TInlineAllocator<1> > bAllocatedStorage;

public:
	uint32 const MemorySize;

public:
	uint8 const bIsPowerOfTwo       : 1;
	uint8 const bCanCreateAsEvicted : 1;
	uint8 const bStreamable         : 1;
	uint8 const bCubemap            : 1;
	uint8 const bArrayTexture       : 1;
	uint8 const bDepthStencil       : 1;
	uint8 const bAlias              : 1;
	uint8 const bMultisampleRenderbuffer : 1;
};

struct FOpenGLProfiler
{
	struct FCalibration
	{
		// FPlatformTime::Cycles64()
		int64 CPUTimestamp = 0;
		int64 CPUFrequency = 0;

		// The GL timestamp as queried directly from the driver by the CPU
		int64 GLTimestamp = 0;

		// Frequency of the GLTimestamp clock
		static constexpr int64 GLFrequency = 1'000'000'000; // one tick per nanosecond
	};

	struct FFrame
	{
		UE::RHI::GPUProfiler::FEventStream EventStream;
		FCalibration Calibration;

		class FOpenGLRenderQuery* EndWorkQuery = nullptr;
		UE::RHI::GPUProfiler::FEvent::FFrameBoundary* FrameBoundaryEvent = nullptr;

		// True if this frame contained any timestamp queries which the driver flagged as disjoint. 
		// If this happens, we discard all the timestamp events from this frame, leaving only the 
		// frame boundary, since the timing will be unreliable.
		bool bDisjoint = false;

		FFrame(FOpenGLProfiler& Profiler);
	};

	bool bEnabled = false;

	TUniquePtr<FFrame> Current;
	TQueue<TUniquePtr<FFrame>> Pending;

	TArray<FOpenGLRenderQuery*> QueryPool;

	TOptional<uint32> ExternalGPUTime;

	FOpenGLRenderQuery* GetQuery();
	FOpenGLRenderQuery* InsertQuery(uint64* Target);

	uint64 ResolveQuery(uint64 Value, uint64* Target, bool bDisjoint);

	void Setup();
	void Shutdown();

	void EndFrame(FRHIEndFrameArgs const& Args);

	template <typename TEventType, typename... TArgs>
	TEventType& EmplaceEvent(TArgs&&... Args)
	{
		return Current->EventStream.Emplace<TEventType>(Forward<TArgs>(Args)...);
	}
};

class FOpenGLRenderQuery
{
public:
	enum class EType : uint8
	{
		Timestamp,
		Occlusion,
		Profiler,

		Num
	};

private:
	// Render queries that should be polled by the RHI thread.
	struct FActiveQueries
	{
		FOpenGLRenderQuery* First = nullptr;
		FOpenGLRenderQuery* Last = nullptr;
		int32 Count = 0;
	} static ActiveQueries;

	struct FQueryPool : public TStaticArray<TArray<GLuint>, (uint32)EType::Num>
	{
		TArray<GLuint>& operator [](EType InType) { return TStaticArray::operator[]((uint32)InType); }
	} static PooledQueries;

	// Linked list pointers. Used to build a list of "active" queries, i.e. queries that need data to be polled from the GPU.
	FOpenGLRenderQuery** Prev = nullptr;
	FOpenGLRenderQuery* Next = nullptr;

	uint64 Result = 0;

	/** The query resource. */
	GLuint Resource = 0;

	// Additional data used for profiler timestamps
	uint64* Target = nullptr;

protected:
	EType const Type;

	std::atomic<uint8> LastCachedBOPCounter = 0;
	uint8 BOPCounter = 0;
	uint8 TOPCounter = 0;

public:
	FOpenGLRenderQuery(EType Type)
		: Type(Type)
	{}

	~FOpenGLRenderQuery();

	void AcquireGlQuery();
	void ReleaseGlQuery();

	bool IsLinked() const { return Prev != nullptr; }

	void Begin();
	void End(uint64* InTarget = nullptr);

	uint64 GetResult() const
	{
		return Result;
	}

	bool CacheResult(bool bWait);

	static bool PollQueryResults(FOpenGLRenderQuery* TargetQuery = nullptr);
	static void Cleanup();

private:
	void Link();
	void Unlink();

	void SetResult(uint64 Value);

	bool IsCached();
};

class FOpenGLRenderQuery_RHI : public FRHIRenderQuery, public FOpenGLRenderQuery
{
public:
	FOpenGLRenderQuery_RHI(ERenderQueryType QueryType)
		: FOpenGLRenderQuery(QueryType == RQT_Occlusion ? EType::Occlusion : EType::Timestamp)
	{}

	void End_TopOfPipe()
	{
		TOPCounter++;
	}

	bool GetResult(bool bWait, uint64& OutResult);
};

class FOpenGLView : public TIntrusiveLinkedList<FOpenGLView>
{
public:
	virtual void UpdateView() = 0;
};

class FOpenGLUnorderedAccessView final : public FRHIUnorderedAccessView, public FOpenGLView
{
public:
	FOpenGLUnorderedAccessView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc);
	virtual ~FOpenGLUnorderedAccessView();

	FOpenGLViewableResource* GetBaseResource() const;

	void UpdateView() override;

	GLuint Resource = 0;
	GLuint BufferResource = 0;
	GLenum Format = 0;
	uint8  UnrealFormat = 0;
	uint8  Level = 0;

	uint32 GetBufferSize() const
	{
		return IsBuffer() ? GetBuffer()->GetSize() : 0;
	}

	bool IsLayered() const
	{
		return IsTexture() && (GetTexture()->GetDesc().Dimension == ETextureDimension::Texture3D || GetTexture()->GetDesc().Dimension == ETextureDimension::Texture2DArray);
	}

	GLint GetLayer() const
	{
		return 0;
	}

private:
	void Invalidate();
	bool OwnsResource = false;
};

class FOpenGLShaderResourceView final : public FRHIShaderResourceView, public FOpenGLView
{
public:
	FOpenGLShaderResourceView(FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc);
	virtual ~FOpenGLShaderResourceView();

	FOpenGLViewableResource* GetBaseResource() const;

	void UpdateView() override;

	/** OpenGL texture the buffer is bound with */
	GLuint Resource = GL_NONE;
	GLenum Target = GL_TEXTURE_BUFFER;

	int32 LimitMip = -1;

private:
	void Invalidate();
	bool OwnsResource = false;
};

void OPENGLDRV_API ReleaseOpenGLFramebuffers(FRHITexture* TextureRHI);

/** A OpenGL event query resource. */
class FOpenGLEventQuery
{
public:
	/** Initialization constructor. */
	FOpenGLEventQuery();
	~FOpenGLEventQuery();

	/** Issues an event for the query to poll. */
	void IssueEvent();

	/** Waits for the event query to finish. */
	void WaitForCompletion();

private:
	UGLsync Sync = {};
};

class FOpenGLViewport : public FRHIViewport
{
public:

	FOpenGLViewport(class FOpenGLDynamicRHI* InOpenGLRHI,void* InWindowHandle,uint32 InSizeX,uint32 InSizeY,bool bInIsFullscreen,EPixelFormat PreferredPixelFormat);
	~FOpenGLViewport();

	void Resize(uint32 InSizeX,uint32 InSizeY,bool bInIsFullscreen);

	// Accessors.
	FIntPoint GetSizeXY() const { return FIntPoint(SizeX, SizeY); }
	FOpenGLTexture* GetBackBuffer() const { return BackBuffer; }
	bool IsFullscreen( void ) const { return bIsFullscreen; }

	virtual void WaitForFrameEventCompletion() override;
	virtual void IssueFrameEvent() override;

	virtual void* GetNativeWindow(void** AddParam) const override;

	struct FPlatformOpenGLContext* GetGLContext() const { return OpenGLContext; }
	FOpenGLDynamicRHI* GetOpenGLRHI() const { return OpenGLRHI; }

	struct FPlatformOpenGLSurfaceContext* GetGLSurfaceContext() const { return OpenGLSurfaceContext; }

	virtual void SetCustomPresent(FRHICustomPresent* InCustomPresent) override
	{
		CustomPresent = InCustomPresent;
	}
	FRHICustomPresent* GetCustomPresent() const { return CustomPresent.GetReference(); }

	void* GetWindowHandle() { return WindowHandle; }

private:

	friend class FOpenGLDynamicRHI;

	FOpenGLDynamicRHI* OpenGLRHI;
	struct FPlatformOpenGLContext* OpenGLContext;
	FPlatformOpenGLSurfaceContext* OpenGLSurfaceContext;
	uint32 SizeX;
	uint32 SizeY;
	bool bIsFullscreen;
	EPixelFormat PixelFormat;
	bool bIsValid;
	TRefCountPtr<FOpenGLTexture> BackBuffer;
	TUniquePtr<FOpenGLEventQuery> FrameSyncEvent;
	FCustomPresentRHIRef CustomPresent;
	void* WindowHandle;
};

class FOpenGLGPUFence final : public FRHIGPUFence
{
private:
	struct FGLSync
	{
		// The graph event to trigger when the sync is resolved.
		FGraphEventRef Event;

		// The GL API sync object to poll / wait on.
		UGLsync GLSync;

		FGLSync() = default;
		FGLSync(FGraphEventRef Event, UGLsync GLSync)
			: Event(MoveTemp(Event))
			, GLSync(GLSync)
		{}
	};
	
	static TQueue<FGLSync, EQueueMode::SingleThreaded> ActiveSyncs;
	static void PollFencesUntil(FGraphEvent* Target);

	FGraphEventRef Event;

public:
	FOpenGLGPUFence(FName InName);

	void Clear() override;
	bool Poll() const override;
	void Wait(FRHICommandListImmediate& RHICmdList, FRHIGPUMask GPUMask) const override;

	void WriteGPUFence_TopOfPipe(FRHICommandListBase& RHICmdList);

	static void PollFences()
	{
		PollFencesUntil(nullptr);
	}
};

class FOpenGLStagingBuffer final : public FRHIStagingBuffer
{
	friend class FOpenGLDynamicRHI;
public:
	FOpenGLStagingBuffer() : FRHIStagingBuffer()
	{
		Initialize();
	}

	~FOpenGLStagingBuffer() override;

	// Locks the shadow of VertexBuffer for read. This will stall the RHI thread.
	void *Lock(uint32 Offset, uint32 NumBytes) override;

	// Unlocks the shadow. This is an error if it was not locked previously.
	void Unlock() override;

	uint64 GetGPUSizeBytes() const override { return ShadowSize; }
private:
	void Initialize();

	GLuint ShadowBuffer;
	uint32 ShadowSize;
	void* Mapping;
};

template<class T>
struct TOpenGLResourceTraits
{
};
template<>
struct TOpenGLResourceTraits<FRHIGPUFence>
{
	typedef FOpenGLGPUFence TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIStagingBuffer>
{
	typedef FOpenGLStagingBuffer TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIVertexDeclaration>
{
	typedef FOpenGLVertexDeclaration TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIVertexShader>
{
	typedef FOpenGLVertexShader TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIGeometryShader>
{
	typedef FOpenGLGeometryShader TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIPixelShader>
{
	typedef FOpenGLPixelShader TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIComputeShader>
{
	typedef FOpenGLComputeShader TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIBoundShaderState>
{
	typedef FOpenGLBoundShaderState TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIRenderQuery>
{
	typedef FOpenGLRenderQuery_RHI TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIUniformBuffer>
{
	typedef FOpenGLUniformBuffer TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIBuffer>
{
	typedef FOpenGLBuffer TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIShaderResourceView>
{
	typedef FOpenGLShaderResourceView TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIUnorderedAccessView>
{
	typedef FOpenGLUnorderedAccessView TConcreteType;
};

template<>
struct TOpenGLResourceTraits<FRHIViewport>
{
	typedef FOpenGLViewport TConcreteType;
};

#undef UE_API
