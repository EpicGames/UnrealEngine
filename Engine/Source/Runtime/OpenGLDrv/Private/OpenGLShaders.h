// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLShaders.h: OpenGL shader RHI declaration.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "OpenGLDrv.h"

enum EOpenGLShaderTargetPlatform
{
	OGLSTP_Unknown,
	OGLSTP_Desktop,
	OGLSTP_Android,
	OGLSTP_iOS,
};

/**
  * GL device capabilities for generating GLSL compilable on platform with described capabilities
  */
struct FOpenGLShaderDeviceCapabilities
{
	EOpenGLShaderTargetPlatform TargetPlatform;
	EShaderPlatform MaxRHIShaderPlatform;
	bool bRequiresARMShaderFramebufferFetchDepthStencilUndef;
	GLint MaxVaryingVectors;
	bool bRequiresDisabledEarlyFragmentTests;
	bool bRequiresReadOnlyBuffersWorkaround;
	bool bRequiresPreciseQualifierWorkaround;
	bool bRequiresControlFlowOptimizerWorkaround;
};

/**
  * Gets the GL device capabilities for the current device.
  *
  * @param Capabilities [out] The current platform's capabilities on device for shader compiling
  */
void OPENGLDRV_API GetCurrentOpenGLShaderDeviceCapabilities(FOpenGLShaderDeviceCapabilities& Capabilities);

/**
  * Processes the GLSL output of the shader cross compiler to get GLSL that can be compiled on a
  * platform with the specified capabilities. Works around inconsistencies between OpenGL
  * implementations, including lack of support for certain extensions and drivers with
  * non-conformant behavior.
  *
  * @param GlslCodeOriginal - [in,out] GLSL output from shader cross compiler to be modified.  Process is destructive; pass in a copy if still need original!
  * @param ShaderName - [in] Shader name
  * @param TypeEnum - [in] Type of shader (GL_[VERTEX, FRAGMENT, GEOMETRY, TESS_CONTROL, TESS_EVALUATION]_SHADER)
  * @param Capabilities - [in] GL Device capabilities
  * @param GlslCode - [out] Compilable GLSL
  */
void OPENGLDRV_API GLSLToDeviceCompatibleGLSL(FAnsiCharArray& GlslCodeOriginal, const FString& ShaderName, GLenum TypeEnum, const FOpenGLShaderDeviceCapabilities& Capabilities, FAnsiCharArray& GlslCode);


// make some anon ns functions available to platform extensions
extern "C" void PE_AppendCString(TArray<ANSICHAR> & Dest, const ANSICHAR * Source);
extern "C" void PE_ReplaceCString(TArray<ANSICHAR> & Dest, const ANSICHAR * Source, const ANSICHAR * Replacement);

