// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLState.cpp: OpenGL state implementation.
=============================================================================*/

#include "OpenGLState.h"
#include "Serialization/MemoryWriter.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "OpenGLDrv.h"

GLint GMaxOpenGLTextureFilterAnisotropic = 1;

// Hash of sampler states, used for caching sampler states and texture objects
static TMap<FSamplerStateInitializerRHI, FOpenGLSamplerState*> GSamplerStateCache;

namespace OpenGLConsoleVariables
{
	extern int32 GOpenGLForceBilinear;
};

void EmptyGLSamplerStateCache()
{
	for (auto Iter = GSamplerStateCache.CreateIterator(); Iter; ++Iter )
	{
		auto* State = Iter.Value();
		// Manually release
		State->Release();
	}

	GSamplerStateCache.Empty();
}

static GLenum TranslateAddressMode(ESamplerAddressMode AddressMode)
{
	switch(AddressMode)
	{
	case AM_Clamp: return GL_CLAMP_TO_EDGE;
	case AM_Mirror: return GL_MIRRORED_REPEAT;
	case AM_Border: return UGL_CLAMP_TO_BORDER;
	default: return GL_REPEAT;
	};
}

static GLenum TranslateCullMode(ERasterizerCullMode CullMode)
{
	switch(CullMode)
	{
	case CM_CW: return FOpenGL::SupportsClipControl() ? GL_BACK : GL_FRONT;
	case CM_CCW: return FOpenGL::SupportsClipControl() ? GL_FRONT : GL_BACK;
	default: return GL_NONE;
	};
}

static GLenum TranslateFillMode(ERasterizerFillMode FillMode)
{
	if ( FOpenGL::SupportsPolygonMode() )
	{
		switch(FillMode)
		{
			case FM_Point: return GL_POINT;
			case FM_Wireframe: return GL_LINE;
			default: break;
		};
	}
	return GL_FILL;
}

static ERasterizerCullMode TranslateCullMode(GLenum CullMode)
{
	if (FOpenGL::SupportsClipControl())
	{
		switch(CullMode)
		{
			case GL_BACK: return CM_CW;
			case GL_FRONT: return CM_CCW;
			default: return CM_None;	
		}
	}
	else
	{
		switch(CullMode)
		{
			case GL_FRONT: return CM_CW;
			case GL_BACK: return CM_CCW;
			default: return CM_None;	
		}
	}
}

static ERasterizerFillMode TranslateFillMode(GLenum FillMode)
{
	switch(FillMode)
	{
		case GL_POINT: return FM_Point;
		case GL_LINE: return FM_Wireframe;
		default: return FM_Solid;
	}
}

static GLenum TranslateCompareFunction(ECompareFunction CompareFunction)
{
	switch(CompareFunction)
	{
	case CF_Less: return GL_LESS;
	case CF_LessEqual: return GL_LEQUAL;
	case CF_Greater: return GL_GREATER;
	case CF_GreaterEqual: return GL_GEQUAL;
	case CF_Equal: return GL_EQUAL;
	case CF_NotEqual: return GL_NOTEQUAL;
	case CF_Never: return GL_NEVER;
	default: return GL_ALWAYS;
	};
}

static GLenum TranslateStencilOp(EStencilOp StencilOp)
{
	switch(StencilOp)
	{
	case SO_Zero: return GL_ZERO;
	case SO_Replace: return GL_REPLACE;
	case SO_SaturatedIncrement: return GL_INCR;
	case SO_SaturatedDecrement: return GL_DECR;
	case SO_Invert: return GL_INVERT;
	case SO_Increment: return GL_INCR_WRAP;
	case SO_Decrement: return GL_DECR_WRAP;
	default: return GL_KEEP;
	};
}

static ECompareFunction TranslateCompareFunction(GLenum CompareFunction)
{
	switch(CompareFunction)
	{
	case GL_LESS: return CF_Less;
	case GL_LEQUAL: return CF_LessEqual;
	case GL_GREATER: return CF_Greater;
	case GL_GEQUAL: return CF_GreaterEqual;
	case GL_EQUAL: return CF_Equal;
	case GL_NOTEQUAL: return CF_NotEqual;
	case GL_NEVER: return CF_Never;
	default: return CF_Always;
	};
}

static EStencilOp TranslateStencilOp(GLenum StencilOp)
{
	switch(StencilOp)
	{
	case GL_ZERO: return SO_Zero;
	case GL_REPLACE: return SO_Replace;
	case GL_INCR: return SO_SaturatedIncrement;
	case GL_DECR: return SO_SaturatedDecrement;
	case GL_INVERT: return SO_Invert;
	case GL_INCR_WRAP: return SO_Increment;
	case GL_DECR_WRAP: return SO_Decrement;
	default: return SO_Keep;
	};
}

static GLenum TranslateBlendOp(EBlendOperation BlendOp)
{
	switch(BlendOp)
	{
	case BO_Subtract: return GL_FUNC_SUBTRACT;
	case BO_Min: return GL_MIN;
	case BO_Max: return GL_MAX;
	case BO_ReverseSubtract: return GL_FUNC_REVERSE_SUBTRACT;
	default: return GL_FUNC_ADD;
	};
}

static GLenum TranslateBlendFactor(EBlendFactor BlendFactor)
{
	switch(BlendFactor)
	{
	case BF_One: return GL_ONE;
	case BF_SourceColor: return GL_SRC_COLOR;
	case BF_InverseSourceColor: return GL_ONE_MINUS_SRC_COLOR;
	case BF_SourceAlpha: return GL_SRC_ALPHA;
	case BF_InverseSourceAlpha: return GL_ONE_MINUS_SRC_ALPHA;
	case BF_DestAlpha: return GL_DST_ALPHA;
	case BF_InverseDestAlpha: return GL_ONE_MINUS_DST_ALPHA;
	case BF_DestColor: return GL_DST_COLOR;
	case BF_InverseDestColor: return GL_ONE_MINUS_DST_COLOR;
	case BF_ConstantBlendFactor: return GL_CONSTANT_COLOR;
	case BF_InverseConstantBlendFactor: return GL_ONE_MINUS_CONSTANT_COLOR;
	default: return GL_ZERO;
	};
}

static EBlendOperation TranslateBlendOp(GLenum BlendOp)
{
	switch(BlendOp)
	{
	case GL_FUNC_SUBTRACT: return BO_Subtract;
	case GL_MIN: return BO_Min;
	case GL_MAX: return BO_Max;
	case GL_FUNC_REVERSE_SUBTRACT: return BO_ReverseSubtract;
	default: return BO_Add;
	};
}

static EBlendFactor TranslateBlendFactor(GLenum BlendFactor)
{
	switch(BlendFactor)
	{
	case GL_ONE: return BF_One;
	case GL_SRC_COLOR: return BF_SourceColor;
	case GL_ONE_MINUS_SRC_COLOR: return BF_InverseSourceColor;
	case GL_SRC_ALPHA: return BF_SourceAlpha;
	case GL_ONE_MINUS_SRC_ALPHA: return BF_InverseSourceAlpha;
	case GL_DST_ALPHA: return BF_DestAlpha;
	case GL_ONE_MINUS_DST_ALPHA: return BF_InverseDestAlpha;
	case GL_DST_COLOR: return BF_DestColor;
	case GL_ONE_MINUS_DST_COLOR: return BF_InverseDestColor;
	case GL_CONSTANT_COLOR: return BF_ConstantBlendFactor;
	case GL_ONE_MINUS_CONSTANT_COLOR: return BF_InverseConstantBlendFactor;
	default: return BF_Zero;
	};
}

FOpenGLSamplerState::~FOpenGLSamplerState()
{
	VERIFY_GL_SCOPE();
	FOpenGL::DeleteSamplers(1,&Resource);
}

FSamplerStateRHIRef FOpenGLDynamicRHI::RHICreateSamplerState(const FSamplerStateInitializerRHI& Initializer)
{
	// Try to find an existing cached state
	FOpenGLSamplerState** Found = GSamplerStateCache.Find(Initializer);
	if (Found)
	{
		return *Found;
	}

	FOpenGLSamplerState* SamplerState = new FOpenGLSamplerState;

	SamplerState->Data.WrapS = TranslateAddressMode( Initializer.AddressU );
	SamplerState->Data.WrapT = TranslateAddressMode( Initializer.AddressV );
	SamplerState->Data.WrapR = TranslateAddressMode( Initializer.AddressW );
	SamplerState->Data.LODBias = Initializer.MipBias;

	SamplerState->Data.MaxAnisotropy = 1;
	const bool bComparisonEnabled = (Initializer.SamplerComparisonFunction != SCF_Never);

	switch(Initializer.Filter)
	{
	case SF_AnisotropicPoint:
		// This is set up like this in D3D11, so following suit.
		// Otherwise we're getting QA reports about weird artifacting, because QA scenes are set up in
		// D3D11 and AnisotropicPoint when Linear would be proper goes unnoticed there.

		// Once someone decides to fix things in D3D11, I assume they'll look here to fix things up too. The code below is waiting.

		// MagFilter	= GL_NEAREST;
		// MinFilter	= bComparisonEnabled ? GL_NEAREST : GL_NEAREST_MIPMAP_NEAREST;
		// break;

		// PASS-THROUGH to AnisotropicLinear!

	case SF_AnisotropicLinear:
		SamplerState->Data.MagFilter	= GL_LINEAR;
		SamplerState->Data.MinFilter	= bComparisonEnabled ? GL_LINEAR : GL_LINEAR_MIPMAP_LINEAR;
		SamplerState->Data.MaxAnisotropy = FMath::Min<uint32>(ComputeAnisotropyRT(Initializer.MaxAnisotropy), GMaxOpenGLTextureFilterAnisotropic);
		break;
	case SF_Trilinear:
		SamplerState->Data.MagFilter	= GL_LINEAR;
		SamplerState->Data.MinFilter	= bComparisonEnabled ? GL_LINEAR : GL_LINEAR_MIPMAP_LINEAR;
		break;
	case SF_Bilinear:
		SamplerState->Data.MagFilter	= GL_LINEAR;
		SamplerState->Data.MinFilter	= GL_LINEAR_MIPMAP_NEAREST;
		break;
	default:
	case SF_Point:
		SamplerState->Data.MagFilter	= GL_NEAREST;
		SamplerState->Data.MinFilter	= GL_NEAREST_MIPMAP_NEAREST;
		break;
	}

	if( bComparisonEnabled )
	{
		check(Initializer.SamplerComparisonFunction == SCF_Less);
		SamplerState->Data.CompareMode = GL_COMPARE_REF_TO_TEXTURE;
		SamplerState->Data.CompareFunc = GL_LESS;
	}
	else
	{
		SamplerState->Data.CompareMode = GL_NONE;
	}

	if (OpenGLConsoleVariables::GOpenGLForceBilinear && (SamplerState->Data.MinFilter == GL_LINEAR_MIPMAP_LINEAR))
	{
		SamplerState->Data.MinFilter = GL_LINEAR_MIPMAP_NEAREST;
	}

	SamplerState->Resource = 0;

	FRHICommandListImmediate::Get().EnqueueLambda([SamplerState](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();
		FOpenGL::GenSamplers( 1, &SamplerState->Resource);

		FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_WRAP_S, SamplerState->Data.WrapS);
		FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_WRAP_T, SamplerState->Data.WrapT);
		if (FOpenGL::SupportsTexture3D())
		{
			FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_WRAP_R, SamplerState->Data.WrapR);
		}
		if (FOpenGL::SupportsTextureLODBias())
		{
			FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_LOD_BIAS, SamplerState->Data.LODBias);
		}

		FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_MIN_FILTER, SamplerState->Data.MinFilter);
		FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_MAG_FILTER, SamplerState->Data.MagFilter);
		if (FOpenGL::SupportsTextureFilterAnisotropic())
		{
			FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_MAX_ANISOTROPY_EXT, SamplerState->Data.MaxAnisotropy);
		}

		if (FOpenGL::SupportsTextureCompare())
		{
			FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_COMPARE_MODE, SamplerState->Data.CompareMode);
			FOpenGL::SetSamplerParameter(SamplerState->Resource, GL_TEXTURE_COMPARE_FUNC, SamplerState->Data.CompareFunc);
		}
	});
	
	// Manually add reference as we control the creation/destructions
	SamplerState->AddRef();
	GSamplerStateCache.Add(Initializer, SamplerState);

	return SamplerState;
}

FRasterizerStateRHIRef FOpenGLDynamicRHI::RHICreateRasterizerState(const FRasterizerStateInitializerRHI& Initializer)
{
	FOpenGLRasterizerState* RasterizerState = new FOpenGLRasterizerState;
	RasterizerState->Data.CullMode = TranslateCullMode(Initializer.CullMode);
	RasterizerState->Data.FillMode = TranslateFillMode(Initializer.FillMode);
	RasterizerState->Data.DepthBias = Initializer.DepthBias;
	RasterizerState->Data.SlopeScaleDepthBias = Initializer.SlopeScaleDepthBias;
	RasterizerState->Data.DepthClipMode = Initializer.DepthClipMode;
	
	return RasterizerState;
}

bool FOpenGLRasterizerState::GetInitializer(FRasterizerStateInitializerRHI& Init)
{
	FMemory::Memzero(Init);
	Init.CullMode = TranslateCullMode(Data.CullMode);
	Init.FillMode = TranslateFillMode(Data.FillMode);
	Init.DepthBias = Data.DepthBias;
	Init.SlopeScaleDepthBias = Data.SlopeScaleDepthBias;
	Init.DepthClipMode = Data.DepthClipMode;
	return true;
}

FDepthStencilStateRHIRef FOpenGLDynamicRHI::RHICreateDepthStencilState(const FDepthStencilStateInitializerRHI& Initializer)
{
	FOpenGLDepthStencilState* DepthStencilState = new FOpenGLDepthStencilState;
	DepthStencilState->Data.bZEnable = Initializer.DepthTest != CF_Always || Initializer.bEnableDepthWrite;
	DepthStencilState->Data.bZWriteEnable = Initializer.bEnableDepthWrite;
	DepthStencilState->Data.ZFunc = TranslateCompareFunction(Initializer.DepthTest);
	DepthStencilState->Data.bStencilEnable = Initializer.bEnableFrontFaceStencil || Initializer.bEnableBackFaceStencil;
	DepthStencilState->Data.bTwoSidedStencilMode = Initializer.bEnableBackFaceStencil;
	DepthStencilState->Data.StencilFunc = TranslateCompareFunction(Initializer.FrontFaceStencilTest);
	DepthStencilState->Data.StencilFail = TranslateStencilOp(Initializer.FrontFaceStencilFailStencilOp);
	DepthStencilState->Data.StencilZFail = TranslateStencilOp(Initializer.FrontFaceDepthFailStencilOp);
	DepthStencilState->Data.StencilPass = TranslateStencilOp(Initializer.FrontFacePassStencilOp);
	DepthStencilState->Data.CCWStencilFunc = TranslateCompareFunction(Initializer.BackFaceStencilTest);
	DepthStencilState->Data.CCWStencilFail = TranslateStencilOp(Initializer.BackFaceStencilFailStencilOp);
	DepthStencilState->Data.CCWStencilZFail = TranslateStencilOp(Initializer.BackFaceDepthFailStencilOp);
	DepthStencilState->Data.CCWStencilPass = TranslateStencilOp(Initializer.BackFacePassStencilOp);
	DepthStencilState->Data.StencilReadMask = Initializer.StencilReadMask;
	DepthStencilState->Data.StencilWriteMask = Initializer.StencilWriteMask;

	return DepthStencilState;
}

bool FOpenGLDepthStencilState::GetInitializer(FDepthStencilStateInitializerRHI& Init)
{
	Init.bEnableBackFaceStencil = Data.bTwoSidedStencilMode;
	Init.bEnableFrontFaceStencil = Data.bStencilEnable;
	Init.bEnableDepthWrite = Data.bZWriteEnable;
	Init.StencilReadMask = Data.StencilReadMask;
	Init.StencilWriteMask = Data.StencilWriteMask;
	Init.DepthTest = TranslateCompareFunction(Data.ZFunc);
	Init.FrontFaceStencilTest = TranslateCompareFunction(Data.StencilFunc);
	Init.FrontFaceStencilFailStencilOp = TranslateStencilOp(Data.StencilFail);
	Init.FrontFaceDepthFailStencilOp = TranslateStencilOp(Data.StencilZFail);
	Init.FrontFacePassStencilOp = TranslateStencilOp(Data.StencilPass);
	Init.BackFaceStencilTest = TranslateCompareFunction(Data.CCWStencilFunc);
	Init.BackFaceStencilFailStencilOp = TranslateStencilOp(Data.CCWStencilFail);
	Init.BackFaceDepthFailStencilOp = TranslateStencilOp(Data.CCWStencilZFail);
	Init.BackFacePassStencilOp = TranslateStencilOp(Data.CCWStencilPass);
	return true;
}

FBlendStateRHIRef FOpenGLDynamicRHI::RHICreateBlendState(const FBlendStateInitializerRHI& Initializer)
{
	FOpenGLBlendState* BlendState = new FOpenGLBlendState(Initializer);

	BlendState->Data.bUseAlphaToCoverage = Initializer.bUseAlphaToCoverage;
	for(uint32 RenderTargetIndex = 0;RenderTargetIndex < MaxSimultaneousRenderTargets;++RenderTargetIndex)
	{
		const FBlendStateInitializerRHI::FRenderTarget& RenderTargetInitializer =
			Initializer.bUseIndependentRenderTargetBlendStates
			? Initializer.RenderTargets[RenderTargetIndex]
			: Initializer.RenderTargets[0];
		FOpenGLBlendStateData::FRenderTarget& RenderTarget = BlendState->Data.RenderTargets[RenderTargetIndex];
		RenderTarget.bAlphaBlendEnable = 
			RenderTargetInitializer.ColorBlendOp != BO_Add || RenderTargetInitializer.ColorDestBlend != BF_Zero || RenderTargetInitializer.ColorSrcBlend != BF_One ||
			RenderTargetInitializer.AlphaBlendOp != BO_Add || RenderTargetInitializer.AlphaDestBlend != BF_Zero || RenderTargetInitializer.AlphaSrcBlend != BF_One;
		RenderTarget.ColorBlendOperation = TranslateBlendOp(RenderTargetInitializer.ColorBlendOp);
		RenderTarget.ColorSourceBlendFactor = TranslateBlendFactor(RenderTargetInitializer.ColorSrcBlend);
		RenderTarget.ColorDestBlendFactor = TranslateBlendFactor(RenderTargetInitializer.ColorDestBlend);
		RenderTarget.bSeparateAlphaBlendEnable =
			RenderTargetInitializer.AlphaDestBlend != RenderTargetInitializer.ColorDestBlend ||
			RenderTargetInitializer.AlphaSrcBlend != RenderTargetInitializer.ColorSrcBlend;
		RenderTarget.AlphaBlendOperation = TranslateBlendOp(RenderTargetInitializer.AlphaBlendOp);
		RenderTarget.AlphaSourceBlendFactor = TranslateBlendFactor(RenderTargetInitializer.AlphaSrcBlend);
		RenderTarget.AlphaDestBlendFactor = TranslateBlendFactor(RenderTargetInitializer.AlphaDestBlend);
		RenderTarget.ColorWriteMaskR = (RenderTargetInitializer.ColorWriteMask & CW_RED) != 0;
		RenderTarget.ColorWriteMaskG = (RenderTargetInitializer.ColorWriteMask & CW_GREEN) != 0;
		RenderTarget.ColorWriteMaskB = (RenderTargetInitializer.ColorWriteMask & CW_BLUE) != 0;
		RenderTarget.ColorWriteMaskA = (RenderTargetInitializer.ColorWriteMask & CW_ALPHA) != 0;
	}
	
	return BlendState;
}

//!AB: moved from the header, since it was causing linker error when the header is included externally
void FOpenGLRHIState::InitializeResources(int32 NumCombinedTextures, int32 NumComputeUAVUnits)
{
	check(!ShaderParameters);
	FOpenGLCommonState::InitializeResources(NumCombinedTextures, NumComputeUAVUnits);
	ShaderParameters = new FOpenGLShaderParameterCache[CrossCompiler::NUM_SHADER_STAGES];
	ShaderParameters[CrossCompiler::SHADER_STAGE_VERTEX].InitializeResources(FOpenGL::GetMaxVertexUniformComponents() * 4 * sizeof(float));
	ShaderParameters[CrossCompiler::SHADER_STAGE_PIXEL].InitializeResources(FOpenGL::GetMaxPixelUniformComponents() * 4 * sizeof(float));
	ShaderParameters[CrossCompiler::SHADER_STAGE_GEOMETRY].InitializeResources(FOpenGL::GetMaxGeometryUniformComponents() * 4 * sizeof(float));

	LinkedProgramAndDirtyFlag = nullptr;
	ShaderParameters[CrossCompiler::SHADER_STAGE_COMPUTE].InitializeResources(FOpenGL::GetMaxComputeUniformComponents() * 4 * sizeof(float));

	for (int32 Frequency = 0; Frequency < SF_NumStandardFrequencies; ++Frequency)
	{
		DirtyUniformBuffers[Frequency] = MAX_uint16;
		bAnyDirtyRealUniformBuffers[Frequency] = true;
	}
	bAnyDirtyGraphicsUniformBuffers = true;
}
