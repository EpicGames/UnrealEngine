// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLState.h: OpenGL state definitions.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Math/Color.h"
#include "Math/IntRect.h"

#include "RHIDefinitions.h"
#include "Containers/StaticArray.h"
#include "RHI.h"
#include "OpenGLResources.h"

#define ZERO_FILLED_DUMMY_UNIFORM_BUFFER_SIZE 65536

class FRenderTarget;

struct FOpenGLSamplerStateData
{
	// These enum is just used to count the number of members in this struct
	enum EGLSamplerData
	{
		EGLSamplerData_WrapS,
		EGLSamplerData_WrapT,
		EGLSamplerData_WrapR,
		EGLSamplerData_LODBias,
		EGLSamplerData_MagFilter,
		EGLSamplerData_MinFilter,
		EGLSamplerData_MaxAniso,
		EGLSamplerData_CompareMode,
		EGLSamplerData_CompareFunc,
		EGLSamplerData_Num,
	};

	GLint WrapS;
	GLint WrapT;
	GLint WrapR;
	GLint LODBias;
	GLint MagFilter;
	GLint MinFilter;
	GLint MaxAnisotropy;
	GLint CompareMode;
	GLint CompareFunc;

	FOpenGLSamplerStateData()
		: WrapS(GL_REPEAT)
		, WrapT(GL_REPEAT)
		, WrapR(GL_REPEAT)
		, LODBias(0)
		, MagFilter(GL_NEAREST)
		, MinFilter(GL_NEAREST)
		, MaxAnisotropy(1)
		, CompareMode(GL_NONE)
		, CompareFunc(GL_ALWAYS)
	{
	}
};

class FOpenGLSamplerState : public FRHISamplerState
{
public:
	GLuint Resource;
	FOpenGLSamplerStateData Data;

	~FOpenGLSamplerState();
};

struct FOpenGLRasterizerStateData
{
	GLenum FillMode = GL_FILL;
	GLenum CullMode = GL_NONE;
	float DepthBias = 0.0f;
	float SlopeScaleDepthBias = 0.0f;
	ERasterizerDepthClipMode DepthClipMode = ERasterizerDepthClipMode::DepthClip;
};

class FOpenGLRasterizerState : public FRHIRasterizerState
{
public:
	virtual bool GetInitializer(FRasterizerStateInitializerRHI& Init) override final;
	
	FOpenGLRasterizerStateData Data;
};

struct FOpenGLDepthStencilStateData
{
	bool bZEnable;
	bool bZWriteEnable;
	GLenum ZFunc;
	

	bool bStencilEnable;
	bool bTwoSidedStencilMode;
	GLenum StencilFunc;
	GLenum StencilFail;
	GLenum StencilZFail;
	GLenum StencilPass;
	GLenum CCWStencilFunc;
	GLenum CCWStencilFail;
	GLenum CCWStencilZFail;
	GLenum CCWStencilPass;
	uint32 StencilReadMask;
	uint32 StencilWriteMask;

	FOpenGLDepthStencilStateData()
		: bZEnable(false)
		, bZWriteEnable(true)
		, ZFunc(GL_LESS)
		, bStencilEnable(false)
		, bTwoSidedStencilMode(false)
		, StencilFunc(GL_ALWAYS)
		, StencilFail(GL_KEEP)
		, StencilZFail(GL_KEEP)
		, StencilPass(GL_KEEP)
		, CCWStencilFunc(GL_ALWAYS)
		, CCWStencilFail(GL_KEEP)
		, CCWStencilZFail(GL_KEEP)
		, CCWStencilPass(GL_KEEP)
		, StencilReadMask(0xFFFFFFFF)
		, StencilWriteMask(0xFFFFFFFF)
	{
	}
};

class FOpenGLDepthStencilState : public FRHIDepthStencilState
{
public:
	virtual bool GetInitializer(FDepthStencilStateInitializerRHI& Init) override final;
	
	FOpenGLDepthStencilStateData Data;
};

struct FOpenGLBlendStateData
{
	struct FRenderTarget
	{
		bool bAlphaBlendEnable;
		GLenum ColorBlendOperation;
		GLenum ColorSourceBlendFactor;
		GLenum ColorDestBlendFactor;
		bool bSeparateAlphaBlendEnable;
		GLenum AlphaBlendOperation;
		GLenum AlphaSourceBlendFactor;
		GLenum AlphaDestBlendFactor;
		uint32 ColorWriteMaskR : 1;
		uint32 ColorWriteMaskG : 1;
		uint32 ColorWriteMaskB : 1;
		uint32 ColorWriteMaskA : 1;
	};
	
	TStaticArray<FRenderTarget,MaxSimultaneousRenderTargets> RenderTargets;

	bool bUseAlphaToCoverage;

	FOpenGLBlendStateData()
	{
		bUseAlphaToCoverage = false;
		for (int32 i = 0; i < MaxSimultaneousRenderTargets; ++i)
		{
			FRenderTarget& Target = RenderTargets[i];
			Target.bAlphaBlendEnable = false;
			Target.ColorBlendOperation = GL_NONE;
			Target.ColorSourceBlendFactor = GL_NONE;
			Target.ColorDestBlendFactor = GL_NONE;
			Target.bSeparateAlphaBlendEnable = false;
			Target.AlphaBlendOperation = GL_NONE;
			Target.AlphaSourceBlendFactor = GL_NONE;
			Target.AlphaDestBlendFactor = GL_NONE;
			Target.ColorWriteMaskR = false;
			Target.ColorWriteMaskG = false;
			Target.ColorWriteMaskB = false;
			Target.ColorWriteMaskA = false;
		}
	}
};

class FOpenGLBlendState : public FRHIBlendState
{
	FBlendStateInitializerRHI RHIInitializer;
public:
	FOpenGLBlendState(const FBlendStateInitializerRHI& Initializer) : RHIInitializer(Initializer) {}
	virtual bool GetInitializer(FBlendStateInitializerRHI& Init) override final
	{ 
		Init = RHIInitializer; 
		return true;
	}

	FOpenGLBlendStateData Data;
};

struct FTextureStage
{
	class FOpenGLTexture* Texture;
	class FOpenGLShaderResourceView* SRV;
	GLenum Target;
	GLuint Resource;
	int32 LimitMip;
	bool bHasMips;
	int32 NumMips;

	FTextureStage()
	:	Texture(NULL)
	,	SRV(NULL)
	,	Target(GL_NONE)
	,	Resource(0)
	,	LimitMip(-1)
	,	bHasMips(false)
	,	NumMips(0)
	{
	}
};

struct FUAVStage
{
	GLenum Format;
	GLuint Resource;
	GLenum Access;
	GLint Layer;
	bool bLayered;
	GLint Level;
	
	FUAVStage()
	:	Format(GL_NONE)
	,	Resource(0)
	,	Access(GL_READ_WRITE)
	,	Layer(0)
	,	bLayered(false)
	,	Level(0)
	{
	}
};
#define FOpenGLCachedAttr_Invalid (void*)(UPTRINT)0xFFFFFFFF
#define FOpenGLCachedAttr_SingleVertex (void*)(UPTRINT)0xFFFFFFFE

struct FOpenGLCachedAttr
{
	GLuint Size;
	GLenum Type;
	GLuint StreamOffset;
	GLuint StreamIndex;
	GLboolean bNormalized;
	GLboolean bShouldConvertToFloat;

	FOpenGLCachedAttr() : 
		Size(),
		Type(0), 
		StreamOffset(),
		StreamIndex(0xFFFFFFFF),
		bNormalized(),
		bShouldConvertToFloat()
	{
	}
};

struct FOpenGLStream
{
	GLuint VertexBufferResource;
	uint32 Stride;
	uint32 Offset;
	uint32 Divisor;
	
	FOpenGLStream()
		: VertexBufferResource(0)
		, Stride(0)
		, Offset(0)
		, Divisor(0)
	{}
};

#define NUM_OPENGL_VERTEX_STREAMS 16

struct FOpenGLCommonState
{
	TArray<FTextureStage>	Textures;
	TArray<FOpenGLSamplerState*>	SamplerStates;
	TArray<FUAVStage>		UAVs;

	FOpenGLCommonState()
	{}

	virtual ~FOpenGLCommonState()
	{
		FOpenGLCommonState::CleanupResources();
	}

	// NumCombinedTextures must be greater than or equal to FOpenGL::GetMaxCombinedTextureImageUnits()
	// NumCombinedUAVUnits must be greater than or equal to FOpenGL::GetMaxCombinedUAVUnits()
	virtual void InitializeResources(int32 NumCombinedTextures, int32 NumCombinedUAVUnits)
	{
		check(NumCombinedTextures >= FOpenGL::GetMaxCombinedTextureImageUnits());
		check(NumCombinedUAVUnits >= FOpenGL::GetMaxCombinedUAVUnits());
		check(Textures.IsEmpty() && SamplerStates.IsEmpty() && UAVs.Num() == 0);
		Textures.SetNum(NumCombinedTextures);
		SamplerStates.SetNumZeroed(NumCombinedTextures);
		
		UAVs.Reserve(NumCombinedUAVUnits);
		UAVs.AddDefaulted(NumCombinedUAVUnits);
	}

	virtual void CleanupResources()
	{
		SamplerStates.Empty();
		Textures.Empty();
		UAVs.Empty();
	}
};

struct FOpenGLContextState final : public FOpenGLCommonState
{
	FOpenGLRasterizerStateData		RasterizerState;
	FOpenGLDepthStencilStateData	DepthStencilState;
	uint32							StencilRef;
	FOpenGLBlendStateData			BlendState;
	GLuint							Framebuffer;
	uint32							RenderTargetWidth;
	uint32							RenderTargetHeight;
	GLuint							OcclusionQuery;
	GLuint							Program;
	GLuint 							UniformBuffers[CrossCompiler::NUM_SHADER_STAGES*OGL_MAX_UNIFORM_BUFFER_BINDINGS];
	GLuint 							UniformBufferOffsets[CrossCompiler::NUM_SHADER_STAGES*OGL_MAX_UNIFORM_BUFFER_BINDINGS];
	TArray<FOpenGLSamplerState*>	CachedSamplerStates;
	GLenum							ActiveTexture;
	bool							bScissorEnabled;
	FIntRect						Scissor;
	FIntRect						Viewport;
	float							DepthMinZ;
	float							DepthMaxZ;
	GLuint							ArrayBufferBound;
	GLuint							ElementArrayBufferBound;
	GLuint							StorageBufferBound;
	GLuint							PixelUnpackBufferBound;
	GLuint							UniformBufferBound;
	FLinearColor					ClearColor;
	uint16							ClearStencil;
	float							ClearDepth;
	int32							FirstNonzeroRenderTarget;
	bool							bAlphaToCoverageEnabled;
	bool							bNonCoherentFramebufferFetchEnabled;
	
	FOpenGLVertexDeclaration*		VertexDecl;
	FOpenGLCachedAttr				VertexAttrs[NUM_OPENGL_VERTEX_STREAMS];
	FOpenGLStream					VertexStreams[NUM_OPENGL_VERTEX_STREAMS];
		
	uint32							ActiveStreamMask;
	uint32							VertexAttrs_EnabledBits;
	FORCEINLINE bool GetVertexAttrEnabled(int32 Index) const
	{
		static_assert(NUM_OPENGL_VERTEX_STREAMS <= sizeof(VertexAttrs_EnabledBits) * 8, "Not enough bits in VertexAttrs_EnabledBits to store NUM_OPENGL_VERTEX_STREAMS");
		return !!(VertexAttrs_EnabledBits & (1 << Index));
	}
	FORCEINLINE void SetVertexAttrEnabled(int32 Index, bool bEnable)
	{
		if (bEnable)
		{
			VertexAttrs_EnabledBits |= (1 << Index);
		}
		else
		{
			VertexAttrs_EnabledBits &= ~(1 << Index);
		}
	}

	uint32 ActiveUAVMask;

	FOpenGLContextState()
	:	StencilRef(0)
	,	Framebuffer(0)
	,	Program(0)
	,	ActiveTexture(GL_TEXTURE0)
	,	bScissorEnabled(false)
	,	DepthMinZ(0.0f)
	,	DepthMaxZ(1.0f)
	,	ArrayBufferBound(0)
	,	ElementArrayBufferBound(0)
	,	StorageBufferBound(0)
	,	PixelUnpackBufferBound(0)
	,	UniformBufferBound(0)
	,	ClearColor(-1, -1, -1, -1)
	,	ClearStencil(0xFFFF)
	,	ClearDepth(-1.0f)
	,	FirstNonzeroRenderTarget(0)
	,	bAlphaToCoverageEnabled(false)
	,	bNonCoherentFramebufferFetchEnabled(false)
	,	VertexDecl(0)
	,   VertexAttrs()
	,	VertexStreams()
	,	ActiveStreamMask(0)
	,	VertexAttrs_EnabledBits(0)
	,	ActiveUAVMask(0)
	{
		Scissor.Min.X = Scissor.Min.Y = Scissor.Max.X = Scissor.Max.Y = 0;
		Viewport.Min.X = Viewport.Min.Y = Viewport.Max.X = Viewport.Max.Y = 0;
		FMemory::Memzero(UniformBuffers, sizeof(UniformBuffers));
		FMemory::Memzero(UniformBufferOffsets, sizeof(UniformBufferOffsets));
	}

	virtual void InitializeResources(int32 NumCombinedTextures, int32 NumCombinedUAVUnits) override
	{
		FOpenGLCommonState::InitializeResources(NumCombinedTextures, NumCombinedUAVUnits);
		CachedSamplerStates.Empty(NumCombinedTextures);
		CachedSamplerStates.AddZeroed(NumCombinedTextures);

		checkf(NumCombinedUAVUnits <= sizeof(ActiveUAVMask) * 8, TEXT("Not enough bits in ActiveUAVMask to store %d UAV units"), NumCombinedUAVUnits);
	}

	virtual void CleanupResources() override
	{
		CachedSamplerStates.Empty();
		FOpenGLCommonState::CleanupResources();
	}
};

struct FOpenGLRHIState final : public FOpenGLCommonState
{
	FOpenGLRasterizerStateData		RasterizerState;
	FOpenGLDepthStencilStateData	DepthStencilState;
	uint32							StencilRef;
	FOpenGLBlendStateData			BlendState;
	GLuint							Framebuffer;
	bool							bScissorEnabled;
	FIntRect						Scissor;
	FIntRect						Viewport;
	float							DepthMinZ;
	float							DepthMaxZ;
	GLuint							ZeroFilledDummyUniformBuffer;
	uint32							RenderTargetWidth;
	uint32							RenderTargetHeight;
	bool							bAlphaToCoverageEnabled;

	// Pending framebuffer setup
	int32							NumRenderingSamples;// Only used with GL_EXT_multisampled_render_to_texture
	int32							FirstNonzeroRenderTarget;
	FOpenGLTexture*					RenderTargets[MaxSimultaneousRenderTargets];
	uint32							RenderTargetMipmapLevels[MaxSimultaneousRenderTargets];
	uint32							RenderTargetArrayIndex[MaxSimultaneousRenderTargets];
	FOpenGLTexture*					DepthStencil;
	ERenderTargetStoreAction		StencilStoreAction;
	uint32							DepthTargetWidth;
	uint32							DepthTargetHeight;
	bool							bFramebufferSetupInvalid;

	// Information about pending BeginDraw[Indexed]PrimitiveUP calls.
	FOpenGLStream					DynamicVertexStream;
	uint32							NumVertices;
	uint32							PrimitiveType;
	uint32							NumPrimitives;
	uint32							MinVertexIndex;
	uint32							IndexDataStride;

	FOpenGLStream					Streams[NUM_OPENGL_VERTEX_STREAMS];

	// we null this when the we dirty PackedGlobalUniformDirty. Thus we can skip all of CommitNonComputeShaderConstants if it matches the current program
	FOpenGLLinkedProgram* LinkedProgramAndDirtyFlag;
	FOpenGLShaderParameterCache*	ShaderParameters;

	TRefCountPtr<FOpenGLBoundShaderState>	BoundShaderState;
	TRefCountPtr<FOpenGLComputeShader>		CurrentComputeShader;

	/** The RHI does not allow more than 14 constant buffers per shader stage due to D3D11 limits. */
	enum { MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE = 14 };

	/** Track the currently bound uniform buffers. */
	FRHIUniformBuffer* BoundUniformBuffers[SF_NumStandardFrequencies][MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE];
	uint32 BoundUniformBuffersDynamicOffset[SF_NumStandardFrequencies][MAX_UNIFORM_BUFFERS_PER_SHADER_STAGE];

	/** Array to track if any real (not emulated) uniform buffers have been bound since the last draw call */
	bool bAnyDirtyRealUniformBuffers[SF_NumStandardFrequencies];
	/** Bit array to track which uniform buffers have changed since the last draw call. */
	bool bAnyDirtyGraphicsUniformBuffers;
	uint16 DirtyUniformBuffers[SF_NumStandardFrequencies];

	// Used for if(!FOpenGL::SupportsFastBufferData())
	uint32 UpVertexBufferBytes;
	uint32 UpIndexBufferBytes;
	uint32 UpStride;
	void* UpVertexBuffer;
	void* UpIndexBuffer;

	FOpenGLRHIState()
	:	StencilRef(0)
	,	Framebuffer(0)
	,	bScissorEnabled(false)
	,	DepthMinZ(0.0f)
	,	DepthMaxZ(1.0f)
	,	ZeroFilledDummyUniformBuffer(0)
	,	RenderTargetWidth(0)
	,	RenderTargetHeight(0)
	,	bAlphaToCoverageEnabled(false)
	,	NumRenderingSamples(1)
	,	FirstNonzeroRenderTarget(-1)
	,	DepthStencil(0)
	,	StencilStoreAction(ERenderTargetStoreAction::ENoAction)
	,	DepthTargetWidth(0)
	,	DepthTargetHeight(0)
	,	bFramebufferSetupInvalid(true)
	,	NumVertices(0)
	,	PrimitiveType(0)
	,	NumPrimitives(0)
	,	MinVertexIndex(0)
	,	IndexDataStride(0)
	,	LinkedProgramAndDirtyFlag(nullptr)
	,	ShaderParameters(NULL)
	,	BoundShaderState(NULL)
	,	CurrentComputeShader(NULL)
	,	UpVertexBufferBytes(0)
	,   UpIndexBufferBytes(0)
	,	UpVertexBuffer(0)
	,	UpIndexBuffer(0)
	{
		Scissor.Min.X = Scissor.Min.Y = Scissor.Max.X = Scissor.Max.Y = 0;
		Viewport.Min.X = Viewport.Min.Y = Viewport.Max.X = Viewport.Max.Y = 0;
		FMemory::Memset( RenderTargets, 0, sizeof(RenderTargets) );	// setting all to 0 at start
		FMemory::Memset( RenderTargetMipmapLevels, 0, sizeof(RenderTargetMipmapLevels) );	// setting all to 0 at start
		FMemory::Memset( RenderTargetArrayIndex, 0, sizeof(RenderTargetArrayIndex) );	// setting all to 0 at start
		FMemory::Memset(BoundUniformBuffers, 0, sizeof(BoundUniformBuffers));
		FMemory::Memset(BoundUniformBuffersDynamicOffset, 0u, sizeof(BoundUniformBuffersDynamicOffset));
	}

	~FOpenGLRHIState()
	{
		CleanupResources();
	}

	virtual void InitializeResources(int32 NumCombinedTextures, int32 NumComputeUAVUnits) override;

	virtual void CleanupResources() override
	{
		delete [] ShaderParameters;
		ShaderParameters = NULL;
		FMemory::Memset(BoundUniformBuffers, 0, sizeof(BoundUniformBuffers));
		FMemory::Memset(BoundUniformBuffersDynamicOffset, 0u, sizeof(BoundUniformBuffersDynamicOffset));
		FOpenGLCommonState::CleanupResources();
	}
};

template<>
struct TOpenGLResourceTraits<FRHISamplerState>
{
	typedef FOpenGLSamplerState TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIRasterizerState>
{
	typedef FOpenGLRasterizerState TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIDepthStencilState>
{
	typedef FOpenGLDepthStencilState TConcreteType;
};
template<>
struct TOpenGLResourceTraits<FRHIBlendState>
{
	typedef FOpenGLBlendState TConcreteType;
};
