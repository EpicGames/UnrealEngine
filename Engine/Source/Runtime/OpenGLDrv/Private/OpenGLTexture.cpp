// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLTexture.cpp: OpenGL texture RHI implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Containers/ResourceArray.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "RenderUtils.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Engine/Texture.h"
#include "RHICoreStats.h"
#include "RHITextureUtils.h"
#include "RHICoreTexture.h"
#include "RHICoreTextureInitializer.h"

#if PLATFORM_ANDROID
#include "ThirdParty/Android/detex/AndroidETC.h"
#endif //PLATFORM_ANDROID

static TAutoConsoleVariable<int32> CVarDeferTextureCreation(
	TEXT("r.OpenGL.DeferTextureCreation"),
	0,
	TEXT("0: OpenGL textures are sent to the driver to be created immediately. (default)\n")
	TEXT("1: Where possible OpenGL textures are stored in system memory and created only when required for rendering.\n")
	TEXT("   This can avoid memory overhead seen in some GL drivers."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDeferTextureCreationExcludeMask(
	TEXT("r.OpenGL.DeferTextureCreationExcludeFlags"),
	static_cast<int32>(~(TexCreate_ShaderResource | TexCreate_SRGB | TexCreate_Streamable | TexCreate_OfflineProcessed)),
	TEXT("Deferred texture creation exclusion mask, any texture requested with flags in this mask will be excluded from deferred creation."),
	ECVF_RenderThreadSafe);

static int32 GOGLDeferTextureCreationKeepLowerMipCount = -1;
static FAutoConsoleVariableRef CVarDeferTextureCreationKeepLowerMipCount(
	TEXT("r.OpenGL.DeferTextureCreationKeepLowerMipCount"),
	GOGLDeferTextureCreationKeepLowerMipCount,
	TEXT("Maximum number of texture mips to retain in CPU memory after a deferred texture has been sent to the driver for GPU memory creation.\n")
	TEXT("-1: to match the number of mips kept resident by the texture streamer (default).\n")
	TEXT(" 0: to disable texture eviction and discard CPU mips after sending them to the driver.\n")
	TEXT(" 16: keep all mips around.\n"),
	ECVF_RenderThreadSafe);

static int32 GOGLTextureEvictFramesToLive = 500;
static FAutoConsoleVariableRef CVarTextureEvictionFrameCount(
	TEXT("r.OpenGL.TextureEvictionFrameCount"),
	GOGLTextureEvictFramesToLive,
	TEXT("The number of frames since a texture was last referenced before it will considered for eviction.\n")
	TEXT("Textures can only be evicted after creation if all their mips are resident, ie its mip count <= r.OpenGL.DeferTextureCreationKeepLowerMipCount."),
	ECVF_RenderThreadSafe
);

int32 GOGLTexturesToEvictPerFrame = 10;
static FAutoConsoleVariableRef CVarTexturesToEvictPerFrame(
	TEXT("r.OpenGL.TextureEvictsPerFrame"),
	GOGLTexturesToEvictPerFrame,
	TEXT("The maximum number of evictable textures to evict per frame, limited to avoid potential driver CPU spikes.\n")
	TEXT("Textures can only be evicted after creation if all their mips are resident, ie its mip count <= r.OpenGL.DeferTextureCreationKeepLowerMipCount."),
	ECVF_RenderThreadSafe
);

static int32 GOGLTextureEvictLogging = 0;
static FAutoConsoleVariableRef CVarTextureEvictionLogging(
	TEXT("r.OpenGL.TextureEvictionLogging"),
	GOGLTextureEvictLogging,
	TEXT("Enables debug logging for texture eviction."),
	ECVF_RenderThreadSafe
);

static int32 GOGLTextureMinLRUCapacity = 0;
static FAutoConsoleVariableRef CVarTextureEvictionMinLRUCapacity(
	TEXT("r.OpenGL.TextureEvictionMinLRUCapacity"),
	GOGLTextureMinLRUCapacity,
	TEXT("Keep a minimum number of textures resident in GL When using the texture LRU\n")
	TEXT("This can reduce LRU restore times when resuming from static scenes.\n")
	TEXT("0: (default)")
	,
	ECVF_RenderThreadSafe
);

struct FScopedPackAlignment
{
	FScopedPackAlignment()
	{
		glPixelStorei(GL_UNPACK_ALIGNMENT, 1);
	}
	~FScopedPackAlignment()
	{
		glPixelStorei(GL_UNPACK_ALIGNMENT, 4);
	}
};

/*-----------------------------------------------------------------------------
	Texture allocator support.
-----------------------------------------------------------------------------*/

/** Caching it here, to avoid getting it every time we create a texture. 0 is no multisampling. */
GLint GMaxOpenGLColorSamples = 0;
GLint GMaxOpenGLDepthSamples = 0;
GLint GMaxOpenGLIntegerSamples = 0;

// in bytes, never change after RHI, needed to scale game features
int64 GOpenGLDedicatedVideoMemory = 0;
// In bytes. Never changed after RHI init. Our estimate of the amount of memory that we can use for graphics resources in total.
int64 GOpenGLTotalGraphicsMemory = 0;

void FOpenGLTexture::UpdateTextureStats(FOpenGLTexture* Texture, bool bAllocating)
{
	const FRHITextureDesc& Desc = Texture->GetDesc();

	const uint64 TextureSize = Texture->MemorySize;

	const bool bOnlyStreamableTexturesInTexturePool = false;
	UE::RHICore::UpdateGlobalTextureStats(Desc, TextureSize, bOnlyStreamableTexturesInTexturePool, bAllocating);

	const int64 TextureSizeDelta = bAllocating ? int64(TextureSize) : -int64(TextureSize);

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	const ELLMTag TextureTag = EnumHasAnyFlags(Desc.Flags, ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ResolveTargetable | ETextureCreateFlags::DepthStencilTargetable)
		? ELLMTag::RenderTargets
		: ELLMTag::Textures;

	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(ELLMTag::GraphicsPlatform, TextureSizeDelta, ELLMTracker::Platform, ELLMAllocType::None);
	LLM_SCOPED_PAUSE_TRACKING_WITH_ENUM_AND_AMOUNT(TextureTag               , TextureSizeDelta, ELLMTracker::Default , ELLMAllocType::None);
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER
}

FDynamicRHI::FRHICalcTextureSizeResult FOpenGLDynamicRHI::RHICalcTexturePlatformSize(FRHITextureDesc const& Desc, uint32 FirstMipIndex)
{
	FDynamicRHI::FRHICalcTextureSizeResult Result;
	Result.Size = Desc.CalcMemorySizeEstimate(FirstMipIndex);
	Result.Align = 1;
	return Result;
}

/**
 * Retrieves texture memory stats. Unsupported with this allocator.
 *
 * @return false, indicating that out variables were left unchanged.
 */
void FOpenGLDynamicRHI::RHIGetTextureMemoryStats(FTextureMemoryStats& OutStats)
{
	UE::RHICore::FillBaselineTextureMemoryStats(OutStats);

	OutStats.DedicatedVideoMemory = GOpenGLDedicatedVideoMemory;
	OutStats.TotalGraphicsMemory = GOpenGLTotalGraphicsMemory ? GOpenGLTotalGraphicsMemory : -1;

	OutStats.LargestContiguousAllocation = OutStats.StreamingMemorySize;
}


/**
 * Fills a texture with to visualize the texture pool memory.
 *
 * @param	TextureData		Start address
 * @param	SizeX			Number of pixels along X
 * @param	SizeY			Number of pixels along Y
 * @param	Pitch			Number of bytes between each row
 * @param	PixelSize		Number of bytes each pixel represents
 *
 * @return true if successful, false otherwise
 */
bool FOpenGLDynamicRHI::RHIGetTextureMemoryVisualizeData( FColor* /*TextureData*/, int32 /*SizeX*/, int32 /*SizeY*/, int32 /*Pitch*/, int32 /*PixelSize*/ )
{
	return false;
}

FOpenGLTextureDesc::FOpenGLTextureDesc(FRHITextureDesc const& InDesc)
	: bCubemap           (InDesc.IsTextureCube())
	, bArrayTexture      (InDesc.IsTextureArray())
	, bStreamable        (EnumHasAnyFlags(InDesc.Flags, TexCreate_Streamable))
	, bDepthStencil      (EnumHasAnyFlags(InDesc.Flags, TexCreate_DepthStencilTargetable))
	, bCanCreateAsEvicted(false)
	, bIsPowerOfTwo      (false)
	, bMultisampleRenderbuffer(EnumHasAnyFlags(InDesc.Flags, TexCreate_Memoryless) && InDesc.NumSamples > 1)
{
	checkf(!bCubemap || InDesc.NumSamples == 1, TEXT("Texture cubes cannot be multisampled."));
	checkf(FOpenGL::SupportsTexture3D() || (!InDesc.IsTexture3D() && !InDesc.IsTextureArray()), TEXT("Texture3D / Texture2DArray support requires FOpenGL::SupportsTexture3D()."));
	checkf(!bMultisampleRenderbuffer || EnumHasAnyFlags(InDesc.Flags, TexCreate_RenderTargetable | TexCreate_DepthStencilTargetable), TEXT("Only render targets can be memoryless"));

	// Special case for multiview MSAA depth target. It has to be a non-MSAA texture with multisample rendering
	const bool bMultiviewMSAADepthTarget = (bDepthStencil && InDesc.NumSamples > 1 && InDesc.Dimension == ETextureDimension::Texture2DArray);
	if (bMultiviewMSAADepthTarget || FOpenGL::GetMaxMSAASamplesTileMem() == 1)
	{
		bMultisampleRenderbuffer = false;
	}
			
	// Select an appropriate texture target
	if (bMultisampleRenderbuffer)
	{
		// Special case for multisample memoryless render targets
		Target = GL_RENDERBUFFER;
	}
	else 
	{
		if (bMultiviewMSAADepthTarget)
		{
			Target = GL_TEXTURE_2D_ARRAY;
		}
		else if (EnumHasAnyFlags(InDesc.Flags, TexCreate_External))
		{
			check(InDesc.IsTexture2D());
			check(!InDesc.IsTextureArray());

			Target = FOpenGL::SupportsImageExternal()
				? GL_TEXTURE_EXTERNAL_OES
				// Fall back to a regular 2d texture if we don't have support.
				// Texture samplers in the shader will also fall back to a regular sampler2D.
				: GL_TEXTURE_2D;
		}
		else if (EnumHasAnyFlags(InDesc.Flags, TexCreate_Presentable))
		{
			check(InDesc.Dimension == ETextureDimension::Texture2D);
			Target = GL_RENDERBUFFER;
		}
		else
		{
			switch (InDesc.Dimension)
			{
			default: checkNoEntry();
			case ETextureDimension::Texture2D:		  Target = (InDesc.NumSamples > 1) ? GL_TEXTURE_2D_MULTISAMPLE       : GL_TEXTURE_2D;       break;
			case ETextureDimension::Texture2DArray:	  Target = (InDesc.NumSamples > 1) ? GL_TEXTURE_2D_MULTISAMPLE_ARRAY : GL_TEXTURE_2D_ARRAY; break;
			case ETextureDimension::TextureCubeArray: Target = GL_TEXTURE_CUBE_MAP_ARRAY; break;
			case ETextureDimension::TextureCube:	  Target = GL_TEXTURE_CUBE_MAP;       break;
			case ETextureDimension::Texture3D:		  Target = GL_TEXTURE_3D;             break;
			}
		}
	}
	check(Target != GL_NONE);

	// can run on RT.
	bCanCreateAsEvicted =
		CanDeferTextureCreation()
		&& InDesc.Flags != TexCreate_None // ignore TexCreate_None
		&& !EnumHasAnyFlags((ETextureCreateFlags)CVarDeferTextureCreationExcludeMask.GetValueOnAnyThread(), InDesc.Flags)  // Anything outside of these flags cannot be evicted.
		&& Target == GL_TEXTURE_2D
		&& InDesc.IsTexture2D(); // 2d only.

	if (GOGLTextureEvictLogging)
	{
		UE_CLOGF(!bCanCreateAsEvicted, LogRHI, Warning, "CanDeferTextureCreation:%d, Flags:%x Mask:%x, Target:%x",
						bCanCreateAsEvicted, int(InDesc.Flags), CVarDeferTextureCreationExcludeMask.GetValueOnAnyThread(), Target);
	}

	bIsPowerOfTwo = 
		   FMath::IsPowerOfTwo(InDesc.Extent.X)
		&& FMath::IsPowerOfTwo(InDesc.Extent.Y)
		&& FMath::IsPowerOfTwo(InDesc.Depth);

	MemorySize = InDesc.CalcMemorySizeEstimate();

	// Determine the attachment point for the texture.	
	if (EnumHasAnyFlags(InDesc.Flags, TexCreate_RenderTargetable | TexCreate_CPUReadback))
	{
		Attachment = GL_COLOR_ATTACHMENT0;
	}
	else if (EnumHasAnyFlags(InDesc.Flags, TexCreate_DepthStencilTargetable))
	{
		Attachment = (InDesc.Format == PF_DepthStencil) ? GL_DEPTH_STENCIL_ATTACHMENT : GL_DEPTH_ATTACHMENT;
	}
	else if (EnumHasAnyFlags(InDesc.Flags, TexCreate_ResolveTargetable))
	{
		Attachment = (InDesc.Format == PF_DepthStencil)
			? GL_DEPTH_STENCIL_ATTACHMENT
			: ((InDesc.Format == PF_ShadowDepth || InDesc.Format == PF_D24)
				? GL_DEPTH_ATTACHMENT
				: GL_COLOR_ATTACHMENT0);
	}
	else
	{
		Attachment = GL_NONE;
	}

	switch (Attachment)
	{
	case GL_COLOR_ATTACHMENT0:
		check(GMaxOpenGLColorSamples >= (GLint)InDesc.NumSamples);
		break;
	case GL_DEPTH_ATTACHMENT:
	case GL_DEPTH_STENCIL_ATTACHMENT:
		check(GMaxOpenGLDepthSamples >= (GLint)InDesc.NumSamples);
		break;
	default:
		break;
	}
}

// Constructor for RHICreateAliasedTexture
FOpenGLTexture::FOpenGLTexture(FOpenGLTexture& Other, const FString& Name, EAliasConstructorParam)
	: FRHITexture(FRHITextureCreateDesc(Other.GetDesc(), ERHIAccess::SRVMask, *Name))
	, Target             (Other.Target)
	, Attachment         (Other.Attachment)
	, MemorySize         (0)
	, bIsPowerOfTwo      (Other.bIsPowerOfTwo)
	, bCanCreateAsEvicted(false)
	, bStreamable        (Other.bStreamable)
	, bCubemap           (Other.bCubemap)
	, bArrayTexture      (Other.bArrayTexture)
	, bDepthStencil      (Other.bDepthStencil)
	, bAlias             (true)
	, bMultisampleRenderbuffer(Other.bMultisampleRenderbuffer)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	RHICmdList.EnqueueLambda([this, &Other](FRHICommandListImmediate&)
	{
		AliasResources(Other);
	});
}

void FOpenGLTexture::AliasResources(FOpenGLTexture& Texture)
{
	VERIFY_GL_SCOPE();
	check(bAlias && !Texture.bAlias);

	// restore the source texture, do not allow the texture to become evicted, the aliasing texture cannot re-create the resource.
	if (Texture.IsEvicted())
	{
		Texture.RestoreEvictedGLResource(false);
	}

	Resource = Texture.Resource;
}

// Constructor for external resources (RHICreateTexture2DFromResource etc).
FOpenGLTexture::FOpenGLTexture(FOpenGLTextureCreateDesc const& CreateDesc, GLuint InResource)
	: FRHITexture        (CreateDesc)
	, Resource           (InResource)
	, Target             (CreateDesc.Target)
	, Attachment         (CreateDesc.Attachment)
	, MemorySize         (CreateDesc.MemorySize)
	, bIsPowerOfTwo      (CreateDesc.bIsPowerOfTwo)
	, bCanCreateAsEvicted(false)
	, bStreamable        (CreateDesc.bStreamable)
	, bCubemap           (CreateDesc.bCubemap)
	, bArrayTexture      (CreateDesc.bArrayTexture)
	, bDepthStencil      (CreateDesc.bDepthStencil)
	, bAlias             (true)
	, bMultisampleRenderbuffer(CreateDesc.bMultisampleRenderbuffer)
{
}

#if PLATFORM_ANDROID
// Constructor for external hardware buffer.
FOpenGLTexture::FOpenGLTexture(FRHICommandListBase& RHICmdList, FOpenGLTextureCreateDesc const& CreateDesc, const AHardwareBuffer_Desc& HardwareBufferDesc, AHardwareBuffer* HardwareBuffer)
	: FRHITexture(CreateDesc)
	, Target(CreateDesc.Target)
	, Attachment(CreateDesc.Attachment)
	, MemorySize(CreateDesc.MemorySize)
	, bIsPowerOfTwo(CreateDesc.bIsPowerOfTwo)
	, bCanCreateAsEvicted(CreateDesc.bCanCreateAsEvicted)
	, bStreamable(CreateDesc.bStreamable)
	, bCubemap(CreateDesc.bCubemap)
	, bArrayTexture(CreateDesc.bArrayTexture)
	, bDepthStencil(CreateDesc.bDepthStencil)
	, bAlias(false)
	, bMultisampleRenderbuffer(CreateDesc.bMultisampleRenderbuffer)
	, HardwareBufferImage(nullptr)
{
	SCOPE_CYCLE_COUNTER(STAT_OpenGLCreateTextureTime);

	RHICmdList.EnqueueLambda([this, HardwareBuffer](FRHICommandListBase& CmdList)
		{
			check(IsInRHIThread());

			VERIFY_GL_SCOPE();
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateDecoderExternaTexture_Execute);

			ensure(eglGetNativeClientBufferANDROID_p != nullptr);
			ensure(eglCreateImageKHR_p != nullptr);
			ensure(eglDestroyImageKHR_p != nullptr);
			ensure(glEGLImageTargetTexture2DOES_p != nullptr);

			// Ext. EGL_ANDROID_get_native_client_buffer
			EGLClientBuffer NativeClientBuffer = eglGetNativeClientBufferANDROID_p(HardwareBuffer);
			if (NativeClientBuffer == nullptr)
			{
				UE_LOGF(LogRHI, Warning, "eglGetNativeClientBufferANDROID - Could not get native client buffer!");
				return;
			}

			// Ext. EGL_ANDROID_image_native_buffer
			EGLImageKHR EglImage = eglCreateImageKHR_p(eglGetCurrentDisplay(), EGL_NO_CONTEXT, EGL_NATIVE_BUFFER_ANDROID, NativeClientBuffer, nullptr);
			if (EglImage == nullptr)
			{
				UE_LOGF(LogRHI, Warning, "eglCreateImageKHR - Could not create EGLimage from native client buffer! B=0x%llx E=0x%x", NativeClientBuffer, eglGetError());
				return;
			}

			this->HardwareBufferImage = EglImage;

			glEGLImageTargetTexture2DOES_p(GL_TEXTURE_EXTERNAL_OES, this->HardwareBufferImage);
			check(eglGetError() == EGL_SUCCESS);
			check(glGetError() == 0);

			// Bind the RHI texture's GL texture to the external image we got data about now...
			glBindTexture(GL_TEXTURE_EXTERNAL_OES, *reinterpret_cast<int32*>(this->GetNativeResource()));
			check(glGetError() == 0);
		});

	UpdateTextureStats(this, true);
}
#endif

// Standard constructor.
FOpenGLTexture::FOpenGLTexture(const FOpenGLTextureCreateDesc& CreateDesc)
	: FRHITexture        (CreateDesc)
	, Target             (CreateDesc.Target)
	, Attachment         (CreateDesc.Attachment)
	, MemorySize         (CreateDesc.MemorySize)
	, bIsPowerOfTwo      (CreateDesc.bIsPowerOfTwo)
	, bCanCreateAsEvicted(CreateDesc.bCanCreateAsEvicted)
	, bStreamable        (CreateDesc.bStreamable)
	, bCubemap           (CreateDesc.bCubemap)
	, bArrayTexture      (CreateDesc.bArrayTexture)
	, bDepthStencil      (CreateDesc.bDepthStencil)
	, bAlias             (false)
	, bMultisampleRenderbuffer(CreateDesc.bMultisampleRenderbuffer)
{
	SCOPE_CYCLE_COUNTER(STAT_OpenGLCreateTextureTime);

	if (bCanCreateAsEvicted)
	{
		EvictionParamsPtr = MakeUnique<FTextureEvictionParams>(CreateDesc.NumMips);
	}

	UpdateTextureStats(this, true);

	PixelBuffers.AddZeroed(CreateDesc.NumMips * (bCubemap ? 6 : 1) * GetEffectiveSizeZ());
}

FOpenGLTexture::~FOpenGLTexture()
{
	VERIFY_GL_SCOPE();

	FTextureEvictionLRU::Get().Remove(this);

	if (!bCanCreateAsEvicted)
	{
		ReleaseOpenGLFramebuffers(this);
	}

	DeleteGLResource();
	UpdateTextureStats(this, false);
}

void FOpenGLTexture::Initialize(FRHICommandListBase& RHICmdList)
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		FOpenGLDynamicRHI::Get().InitializeGLTexture(this, nullptr, 0);
	});
}

void FOpenGLTexture::DeleteGLResource()
{
	VERIFY_GL_SCOPE();
	QUICK_SCOPE_CYCLE_COUNTER(STAT_OpenGLDeleteGLTextureTime);

#if PLATFORM_ANDROID
	if (HardwareBufferImage != nullptr)
	{
		eglDestroyImageKHR_p(eglGetCurrentDisplay(), HardwareBufferImage);
	}
#endif //PLATFORM_ANDROID

	if (Resource != 0)
	{
		switch (Target)
		{
		case GL_TEXTURE_2D:
		case GL_TEXTURE_2D_MULTISAMPLE:
		case GL_TEXTURE_3D:
		case GL_TEXTURE_CUBE_MAP:
		case GL_TEXTURE_2D_ARRAY:
		case GL_TEXTURE_CUBE_MAP_ARRAY:
		case GL_TEXTURE_EXTERNAL_OES:
			FOpenGLDynamicRHI::Get().InvalidateTextureResourceInCache(Resource);
			if (!bAlias)
			{
				FOpenGL::DeleteTextures(1, &Resource);
			}
			break;

		case GL_RENDERBUFFER:
			if (!bAlias)
			{
				glDeleteRenderbuffers(1, &Resource);
			}
			break;

		default:
			checkNoEntry();
			break;
		}
	}

	Resource = GL_NONE;
}

static inline bool IsAstcLdrRGBAFormat(GLenum Format)
{
	return Format >= GL_COMPRESSED_RGBA_ASTC_4x4_KHR && Format <= GL_COMPRESSED_RGBA_ASTC_12x12_KHR;
}

uint32 GTotalTexStorageSkipped = 0;
uint32 GTotalCompressedTexStorageSkipped = 0;

void FOpenGLDynamicRHI::InitializeGLTexture(FOpenGLTexture* Texture, const void* BulkDataPtr, uint64 BulkDataSize)
{
	VERIFY_GL_SCOPE();

	if (EnumHasAnyFlags(Texture->GetDesc().Flags, TexCreate_Presentable))
		return;

	// Allocate the GL resource ID
	GLuint TextureID;
	if (Texture->bMultisampleRenderbuffer)
	{
		check(Texture->Target == GL_RENDERBUFFER);
		glGenRenderbuffers(1, &TextureID);
	}
	else
	{
		check(Texture->Target != GL_RENDERBUFFER);
		glGenTextures(1, &TextureID);
	}
	Texture->SetResource(TextureID);

	if (!Texture->IsEvicted())
	{
		InitializeGLTextureInternal(Texture, BulkDataPtr, BulkDataSize);
	}
	else
	{
		// creating this as 'evicted'.
		GTotalTexStorageSkipped++;

		EPixelFormat PixelFormat = Texture->GetFormat();
		const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[PixelFormat];
		bool bIsCompressed = GLFormat.bCompressed;
		GTotalCompressedTexStorageSkipped += bIsCompressed ? 1 : 0;

		if (BulkDataPtr)
		{
			check(!GLFormat.bCompressed);
			const uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
			const uint32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;

			uint8* Data = (uint8*)BulkDataPtr;
			uint32 MipOffset = 0;

			const FRHITextureDesc& Desc = Texture->GetDesc();

			// copy bulk data to evicted mip store:
			for (uint32 MipIndex = 0; MipIndex < Desc.NumMips; MipIndex++)
			{
				uint32 NumBlocksX = AlignArbitrary(FMath::Max<uint32>(1, (Desc.Extent.X >> MipIndex)), BlockSizeX) / BlockSizeX;
				uint32 NumBlocksY = AlignArbitrary(FMath::Max<uint32>(1, (Desc.Extent.Y >> MipIndex)), BlockSizeY) / BlockSizeY;
				uint32 NumLayers = FMath::Max<uint32>(1, Desc.ArraySize);
				uint32 MipDataSize = NumBlocksX * NumBlocksY * NumLayers * GPixelFormats[PixelFormat].BlockBytes;

				Texture->EvictionParamsPtr->SetMipData(MipIndex, &Data[MipOffset], MipDataSize);
				MipOffset += MipDataSize;
			}
		}
	}
}

void FOpenGLDynamicRHI::InitializeGLTextureInternal(FOpenGLTexture* Texture, void const* BulkDataPtr, uint64 BulkDataSize)
{
	VERIFY_GL_SCOPE();

	GLuint const TextureID = Texture->GetRawResourceName();

	const FRHITextureDesc& Desc = Texture->GetDesc();
	const GLenum Target = Texture->Target;

	const bool bSRGB = EnumHasAnyFlags(Desc.Flags, TexCreate_SRGB);
	const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[Desc.Format];
	if (GLFormat.InternalFormat[bSRGB] == GL_NONE)
	{
		UE_LOGF(LogRHI, Fatal,"Texture format '%ls' not supported (sRGB=%d).", GPixelFormats[Desc.Format].Name, bSRGB);
	}
	
	const bool bMultiviewMSAADepthTarget = (Texture->bDepthStencil && Desc.NumSamples > 1 && Desc.Dimension == ETextureDimension::Texture2DArray);

	// Make sure PBO is disabled
	CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);

	bool bAllocatedStorage = false;
	if (Texture->bMultisampleRenderbuffer)
	{
		check(Texture->IsMultisampled());
		check(Target == GL_RENDERBUFFER);
		// Multisample Renderbuffers will be allocated on first use. See ConditionallyAllocateRenderbufferStorage
	}
	else
	{
		// Use a texture stage that's not likely to be used for draws, to avoid waiting
		CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Target, TextureID, 0, Desc.NumMips);

		if((GLFormat.bBGRA && !EnumHasAnyFlags(Desc.Flags, TexCreate_RenderTargetable)) 
#if !PLATFORM_ANDROID
			|| (GLFormat.InternalFormat[0] == GL_RGB5_A1)
#endif
			)
		{
			glTexParameteri(Target, GL_TEXTURE_SWIZZLE_R, GL_BLUE);
			glTexParameteri(Target, GL_TEXTURE_SWIZZLE_B, GL_RED);
		}

		if (!Texture->IsMultisampled())
		{
			if (Target == GL_TEXTURE_EXTERNAL_OES || !FMath::IsPowerOfTwo(Desc.Extent.X) || !FMath::IsPowerOfTwo(Desc.Extent.Y))
			{
				glTexParameteri(Target, GL_TEXTURE_WRAP_S, GL_CLAMP_TO_EDGE);
				glTexParameteri(Target, GL_TEXTURE_WRAP_T, GL_CLAMP_TO_EDGE);
				if (FOpenGL::SupportsTexture3D())
				{
					glTexParameteri(Target, GL_TEXTURE_WRAP_R, GL_CLAMP_TO_EDGE);
				}
			}
			else
			{
				glTexParameteri(Target, GL_TEXTURE_WRAP_S, GL_REPEAT);
				glTexParameteri(Target, GL_TEXTURE_WRAP_T, GL_REPEAT);
				if (FOpenGL::SupportsTexture3D())
				{
					glTexParameteri(Target, GL_TEXTURE_WRAP_R, GL_REPEAT);
				}
			}

			glTexParameteri(Target, GL_TEXTURE_MAG_FILTER, GL_NEAREST);
			glTexParameteri(Target, GL_TEXTURE_MIN_FILTER, Desc.NumMips > 1 ? GL_NEAREST_MIPMAP_NEAREST : GL_NEAREST);

			if (FOpenGL::SupportsTextureFilterAnisotropic())
			{
				glTexParameteri(Target, GL_TEXTURE_MAX_ANISOTROPY_EXT, 1);
			}
		}

		glTexParameteri(Target, GL_TEXTURE_BASE_LEVEL, 0);

		TextureMipLimits.Add(TextureID, TPair<GLenum, GLenum>(0, Desc.NumMips - 1));

		if (FOpenGL::SupportsASTCDecodeMode())
		{
			if (IsAstcLdrRGBAFormat(GLFormat.InternalFormat[bSRGB]))
			{
				glTexParameteri(Target, TEXTURE_ASTC_DECODE_PRECISION_EXT, GL_RGBA8);
			}
		}

		if (Target != GL_TEXTURE_EXTERNAL_OES)
		{
			using TEnumerationCallback = TFunctionRef<bool(GLenum Target, FUintVector3 Extent, uint32 MipIndex, uint32 ArraySlice, const void* MipSliceData, uint64 MipSliceSize)>;

			auto EnumerateSubresources = [&](void const* Data, TEnumerationCallback&& Callback)
			{
				FScopedPackAlignment PackAlignment;

				const FPixelFormatInfo& PixelFormat = GPixelFormats[Desc.Format];

				for (uint32 ArraySlice = 0; ArraySlice < Desc.ArraySize; ++ArraySlice)
				{
					for (uint32 FaceIndex = 0; FaceIndex < (Desc.IsTextureCube() ? 6u : 1u); ++FaceIndex)
					{
						for (uint32 MipIndex = 0; MipIndex < Desc.NumMips; ++MipIndex)
						{
							uint64 SubresourceStride{};
							uint64 SubresourceSize{};
							uint64 ResourceOffset = UE::RHITextureUtils::CalculateSubresourceOffset(Desc, FaceIndex, ArraySlice, MipIndex, SubresourceStride, SubresourceSize);
							const uint64 MipSize = SubresourceSize;

							const FUintVector3 BlockCounts = UE::RHITextureUtils::CalculateMipBlockCounts(Desc, MipIndex);
							const FUintVector3 MipExtents = FUintVector3(BlockCounts.X * PixelFormat.BlockSizeX, BlockCounts.Y * PixelFormat.BlockSizeY, BlockCounts.Z * PixelFormat.BlockSizeZ);

							const GLenum CurrentTarget = Target == GL_TEXTURE_CUBE_MAP
								? GL_TEXTURE_CUBE_MAP_POSITIVE_X + FaceIndex
								: Target;

							if (!Callback(CurrentTarget, MipExtents, MipIndex, ArraySlice, reinterpret_cast<const uint8*>(Data) + ResourceOffset, MipSize))
							{
								return;
							}
						}
					}
				}
			};

			// Create the texture resource
			switch (Target)
			{
			default: checkNoEntry();
			case GL_RENDERBUFFER:
			case GL_TEXTURE_2D:
			case GL_TEXTURE_CUBE_MAP:
				{
					// Try to create the texture using immutable storage
					FOpenGL::TexStorage2D(Target, Desc.NumMips, GLFormat.InternalFormat[bSRGB], Desc.Extent.X, Desc.Extent.Y, GLFormat.Format, GLFormat.Type, Desc.Flags);

					// Texture created with immutable storage. Now fill in the bulk data.
					bAllocatedStorage = true;

					if (BulkDataPtr)
					{
						EnumerateSubresources(BulkDataPtr, [&](GLenum CurrentTarget, FUintVector3 MipExtent, uint32 MipIndex, uint32 ArraySlice, void const* MipSliceData, uint32 MipSliceSize)
						{
							if (GLFormat.bCompressed)
							{
								glCompressedTexSubImage2D(
									CurrentTarget,
									MipIndex,
									0, 0, // X/Y offset
									MipExtent.X, MipExtent.Y,
									GLFormat.Format,
									MipSliceSize,
									MipSliceData
								);
								GL_CHECK();
							}
							else
							{
								glTexSubImage2D(
									CurrentTarget,
									MipIndex,
									0, 0, // X/Y offset
									MipExtent.X, MipExtent.Y,
									GLFormat.Format,
									GLFormat.Type,
									MipSliceData
								);
								GL_CHECK();
							}

							return true;
						});
					}
				}
				break;

			case GL_TEXTURE_2D_ARRAY:
			case GL_TEXTURE_CUBE_MAP_ARRAY:
			case GL_TEXTURE_3D:
				{
					bAllocatedStorage = true; // Always supported if 3D textures are supported.

					const uint32 SizeZ =
						Target == GL_TEXTURE_3D ? Desc.Depth :
						Target == GL_TEXTURE_CUBE_MAP_ARRAY ? Desc.ArraySize * 6 :
						Desc.ArraySize;

					FOpenGL::TexStorage3D(Target, Desc.NumMips, GLFormat.InternalFormat[bSRGB], Desc.Extent.X, Desc.Extent.Y, SizeZ, GLFormat.Format, GLFormat.Type);

					// Texture created with immutable storage. Now fill in the bulk data.
					if (BulkDataPtr)
					{
						EnumerateSubresources(BulkDataPtr, [&](GLenum CurrentTarget, FUintVector3 MipExtent, uint32 MipIndex, uint32 ArraySlice, void const* MipSliceData, uint32 MipSliceSize)
						{
							if (GLFormat.bCompressed)
							{
								glCompressedTexSubImage3D(
									CurrentTarget,
									MipIndex,
									0, 0, ArraySlice, // X/Y/Z offset
									MipExtent.X, MipExtent.Y, MipExtent.Z,
									GLFormat.Format,
									MipSliceSize,
									MipSliceData
								);
								GL_CHECK();
							}
							else
							{
								glTexSubImage3D(
									CurrentTarget,
									MipIndex,
									0, 0, ArraySlice, // X/Y/Z offset
									MipExtent.X, MipExtent.Y, MipExtent.Z,
									GLFormat.Format,
									GLFormat.Type,
									MipSliceData
								);
								GL_CHECK();
							}

							return true;
						});
					}
				}
				break;

			case GL_TEXTURE_2D_MULTISAMPLE:
				{
					checkf(BulkDataPtr == nullptr, TEXT("Multisample textures cannot be created with initial bulk data."));

					// Try to create an immutable storage texture and fallback if it fails
					const int32 NumSamples = Texture->GetDesc().NumSamples;
					const bool FixedSampleLocations = true;
					FOpenGL::TexStorage2DMultisample(Target, NumSamples, GLFormat.InternalFormat[bSRGB], Desc.Extent.X, Desc.Extent.Y, FixedSampleLocations);
					bAllocatedStorage = true;
				}
				break;
			}
		}
	}

	// @todo: If integer pixel format
	//check(GMaxOpenGLIntegerSamples>=NumSamples);
	Texture->SetAllocatedStorage(bAllocatedStorage);

	// No need to restore texture stage; leave it like this,
	// and the next draw will take care of cleaning it up; or
	// next operation that needs the stage will switch something else in on it.
}


void FOpenGLDynamicRHI::ResolveTexture(FOpenGLTexture* Texture, uint32 MipIndex,uint32 ArrayIndex)
{
	VERIFY_GL_SCOPE();
	check(!Texture->GetTexture2D() || Texture->GetNumSamples() == 1);
	
	// Calculate the dimensions of the mip-map.
	EPixelFormat PixelFormat = Texture->GetFormat();
	const uint32 BlockSizeX = GPixelFormats[PixelFormat].BlockSizeX;
	const uint32 BlockSizeY = GPixelFormats[PixelFormat].BlockSizeY;
	const uint32 BlockBytes = GPixelFormats[PixelFormat].BlockBytes;
	const uint32 MipSizeX = FMath::Max(Texture->GetSizeX() >> MipIndex, BlockSizeX);
	const uint32 MipSizeY = FMath::Max(Texture->GetSizeY() >> MipIndex, BlockSizeY);
	uint32 NumBlocksX = (MipSizeX + BlockSizeX - 1) / BlockSizeX;
	uint32 NumBlocksY = (MipSizeY + BlockSizeY - 1) / BlockSizeY;
	const uint32 MipBytes = NumBlocksX * NumBlocksY * BlockBytes;
	
	const int32 BufferIndex = MipIndex * (Texture->bCubemap ? 6 : 1) * Texture->GetEffectiveSizeZ() + ArrayIndex;
	
	// Standard path with a PBO mirroring ever slice of a texture to allow multiple simulataneous maps
	if (!IsValidRef(Texture->PixelBuffers[BufferIndex]))
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("PixelBuffer"), MipBytes, 0, EBufferUsageFlags::Dynamic);

		Texture->PixelBuffers[BufferIndex] = new FOpenGLPixelBuffer(nullptr, GL_PIXEL_UNPACK_BUFFER, CreateDesc, nullptr);
	}
	
	TRefCountPtr<FOpenGLPixelBuffer> PixelBuffer = Texture->PixelBuffers[BufferIndex];
	check(PixelBuffer->GetSize() == MipBytes);
	check(!PixelBuffer->IsLocked());
	
	// Transfer data from texture to pixel buffer.
	// This may be further optimized by caching information if surface content was changed since last lock.
	
	const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[PixelFormat];
	const bool bSRGB = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_SRGB);
	
	// Use a texture stage that's not likely to be used for draws, to avoid waiting
	CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Texture->Target, Texture->GetResource(), -1, Texture->GetNumMips());
	
	glBindBuffer( GL_PIXEL_PACK_BUFFER, PixelBuffer->Resource );

	{
		if (Texture->GetDesc().IsTextureArray() || Texture->GetDesc().IsTexture3D())
		{
			// apparently it's not possible to retrieve compressed image from GL_TEXTURE_2D_ARRAY in OpenGL for compressed images
			// and for uncompressed ones it's not possible to specify the image index
			check(0);
		}
		else
		{
			if (GLFormat.bCompressed)
			{
				FOpenGL::GetCompressedTexImage(
					Texture->bCubemap
						? GL_TEXTURE_CUBE_MAP_POSITIVE_X + ArrayIndex
						: Texture->Target,
					MipIndex,
					0);	// offset into PBO
			}
			else
			{
				// Get framebuffer for texture
				GLuint SourceFramebuffer = GetOpenGLFramebuffer(1, &Texture, (Texture->bCubemap ? &ArrayIndex : nullptr), &MipIndex, nullptr);
				// Bind the framebuffer
				glBindFramebuffer(UGL_READ_FRAMEBUFFER, SourceFramebuffer);
				FOpenGL::ReadBuffer(GL_COLOR_ATTACHMENT0);

				glPixelStorei(GL_PACK_ALIGNMENT, 1);
				glReadPixels(0, 0, MipSizeX, MipSizeY, GLFormat.Format, GLFormat.Type, 0);
				glPixelStorei(GL_PACK_ALIGNMENT, 4);

				ContextState.Framebuffer = (GLuint)-1;
			}
		}
	}
	
	glBindBuffer(GL_PIXEL_PACK_BUFFER, 0);
	
	// No need to restore texture stage; leave it like this,
	// and the next draw will take care of cleaning it up; or
	// next operation that needs the stage will switch something else in on it.
}

uint32 FOpenGLTexture::GetLockSize(uint32 InMipIndex, EResourceLockMode LockMode, uint32& DestStride)
{
	check(LockMode != EResourceLockMode::RLM_WriteOnly_NoOverwrite);
	uint64 LargerStride{};
	const uint64 MipSize = UE::RHITextureUtils::CalculateTextureMipSize(GetDesc(), InMipIndex, LargerStride);

	DestStride = static_cast<uint32>(LargerStride);
	return MipSize;
}

void FOpenGLTexture::FillGLTextureImage(const FOpenGLTextureFormat& GLFormat, bool bSRGB, uint32 MipIndex, uint32 ArrayIndex, const FUintVector3& MipExtents, const void* BufferOrPBOOffset, size_t ImageSize)
{
	FScopedPackAlignment PackAlignment;

	if (GetDesc().IsTextureArray() || GetDesc().IsTexture3D())
	{
		const GLsizei Depth = 1;

		if (GLFormat.bCompressed)
		{
			FOpenGL::CompressedTexSubImage3D(
				Target,
				MipIndex,
				0, 0, ArrayIndex,
				MipExtents.X, MipExtents.Y, Depth,
				GLFormat.InternalFormat[bSRGB],
				ImageSize,
				BufferOrPBOOffset
			);
			GL_CHECK();
		}
		else
		{
			check(FOpenGL::SupportsTexture3D());
			FOpenGL::TexSubImage3D(
				Target,
				MipIndex,
				0, 0, ArrayIndex,
				MipExtents.X, MipExtents.Y, Depth,
				GLFormat.Format,
				GLFormat.Type,
				BufferOrPBOOffset
			);
			GL_CHECK();
		}
	}
	else
	{
		const GLenum ActualTarget = bCubemap ? GL_TEXTURE_CUBE_MAP_POSITIVE_X + ArrayIndex : Target;

		if (GLFormat.bCompressed)
		{
			if (GetAllocatedStorageForMip(MipIndex, ArrayIndex))
			{
				glCompressedTexSubImage2D(
					ActualTarget,
					MipIndex,
					0, 0,
					MipExtents.X, MipExtents.Y,
					GLFormat.InternalFormat[bSRGB],
					ImageSize,
					BufferOrPBOOffset
				);
				GL_CHECK();
			}
			else
			{
				glCompressedTexImage2D(
					ActualTarget,
					MipIndex,
					GLFormat.InternalFormat[bSRGB],
					MipExtents.X, MipExtents.Y,
					0,
					ImageSize,
					BufferOrPBOOffset
				);
				GL_CHECK();

				SetAllocatedStorageForMip(MipIndex, ArrayIndex);
			}
		}
		else
		{
			// All construction paths should have called TexStorage2D or TexImage2D. So we will always call TexSubImage2D.
			check(GetAllocatedStorageForMip(MipIndex, ArrayIndex) == true);

			glTexSubImage2D(
				ActualTarget,
				MipIndex,
				0, 0,
				MipExtents.X, MipExtents.Y,
				GLFormat.Format,
				GLFormat.Type,
				BufferOrPBOOffset
			);
			GL_CHECK();
		}
	}
}

void* FOpenGLTexture::Lock(uint32 InMipIndex,uint32 ArrayIndex,EResourceLockMode LockMode,uint32& DestStride)
{
	VERIFY_GL_SCOPE();
	check(!GetTexture2D() || GetNumSamples() == 1);

	SCOPE_CYCLE_COUNTER(STAT_OpenGLLockTextureTime);
	
	const uint32 MipBytes = GetLockSize(InMipIndex, LockMode, DestStride);

	check(!IsEvicted() || ArrayIndex==0);
	void* result = NULL;

	const int32 BufferIndex = InMipIndex * (bCubemap ? 6 : 1) * this->GetEffectiveSizeZ() + ArrayIndex;
	EPixelFormat PixelFormat = this->GetFormat();

	// Should we use client-storage to improve update time on platforms that require it
	const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[PixelFormat];
	if (IsEvicted())
	{
		check(ArrayIndex == 0);
		EvictionParamsPtr->SetMipData(InMipIndex, nullptr, MipBytes);
		return EvictionParamsPtr->MipImageData[InMipIndex].GetData();
	}
	else
	{
		if(LockMode != RLM_ReadOnly)
		{
			// if we modify an active but evictable texture then just make it non-evictable, this is to avoid keeping the cpu backup copies in sync.
			RemoveEvictionData();
		}

		// Standard path with a PBO mirroring ever slice of a texture to allow multiple simulataneous maps
		bool bBufferExists = true;
		if (!IsValidRef(PixelBuffers[BufferIndex]))
		{
			bBufferExists = false;
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::Create(TEXT("PixelBuffer"), MipBytes, 0, EBufferUsageFlags::Dynamic);
			PixelBuffers[BufferIndex] = new FOpenGLPixelBuffer(nullptr, GL_PIXEL_UNPACK_BUFFER, CreateDesc, nullptr);
		}

		TRefCountPtr<FOpenGLPixelBuffer> PixelBuffer = PixelBuffers[BufferIndex];
		check(PixelBuffer->GetSize() == MipBytes);
		check(!PixelBuffer->IsLocked());
		
		// If the buffer already exists & the flags are such that the texture cannot be rendered to & is CPU accessible then we can skip the internal resolve for read locks. This makes HZB occlusion faster.
		const bool bCPUTexResolved = bBufferExists && EnumHasAnyFlags(this->GetFlags(), TexCreate_CPUReadback) && !EnumHasAnyFlags(this->GetFlags(), TexCreate_RenderTargetable|TexCreate_DepthStencilTargetable);

		if (LockMode != RLM_WriteOnly && !bCPUTexResolved)
		{
			FOpenGLDynamicRHI::Get().ResolveTexture(this, InMipIndex, ArrayIndex);
		}

		result = PixelBuffer->Lock(0, PixelBuffer->GetSize(), LockMode == RLM_ReadOnly, LockMode != RLM_ReadOnly);
	}
	
	return result;
}

// Copied from OpenGLDebugFrameDump.
inline uint32 HalfFloatToFloatInteger(uint16 HalfFloat)
{
	uint32 Sign = (HalfFloat >> 15) & 0x00000001;
	uint32 Exponent = (HalfFloat >> 10) & 0x0000001f;
	uint32 Mantiss = HalfFloat & 0x000003ff;

	if (Exponent == 0)
	{
		if (Mantiss == 0) // Plus or minus zero
		{
			return Sign << 31;
		}
		else // Denormalized number -- renormalize it
		{
			while ((Mantiss & 0x00000400) == 0)
			{
				Mantiss <<= 1;
				Exponent -= 1;
			}

			Exponent += 1;
			Mantiss &= ~0x00000400;
		}
	}
	else if (Exponent == 31)
	{
		if (Mantiss == 0) // Inf
			return (Sign << 31) | 0x7f800000;
		else // NaN
			return (Sign << 31) | 0x7f800000 | (Mantiss << 13);
	}

	Exponent = Exponent + (127 - 15);
	Mantiss = Mantiss << 13;

	return (Sign << 31) | (Exponent << 23) | Mantiss;
}

inline float HalfFloatToFloat(uint16 HalfFloat)
{
	union
	{
		float F;
		uint32 I;
	} Convert;

	Convert.I = HalfFloatToFloatInteger(HalfFloat);
	return Convert.F;
}

void FOpenGLTexture::Unlock(uint32 MipIndex,uint32 ArrayIndex)
{
	VERIFY_GL_SCOPE();
	SCOPE_CYCLE_COUNTER(STAT_OpenGLUnlockTextureTime);

	if (IsEvicted())
	{
		// evicted textures didn't actually perform a lock, so we can bail out early
		check(ArrayIndex == 0);
		// check the space was allocated
		ensure(MipIndex < (uint32)EvictionParamsPtr->MipImageData.Num() && EvictionParamsPtr->MipImageData[MipIndex].Num());
		return;
	}

	const int32 BufferIndex = MipIndex * (bCubemap ? 6 : 1) * this->GetEffectiveSizeZ() + ArrayIndex;
	const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[this->GetFormat()];
	const bool bSRGB = EnumHasAnyFlags(this->GetFlags(), TexCreate_SRGB);

	check(BufferIndex < PixelBuffers.Num());
	TRefCountPtr<FOpenGLPixelBuffer> PixelBuffer = PixelBuffers[BufferIndex];

	check(IsValidRef(PixelBuffer));
	
#if PLATFORM_ANDROID
	// check for FloatRGBA to RGBA8 conversion needed
	if (this->GetFormat() == PF_FloatRGBA && GLFormat.Type == GL_UNSIGNED_BYTE)
	{
		UE_LOGF(LogRHI, Warning, "Converting texture from PF_FloatRGBA to RGBA8!  Only supported for limited cases of 0.0 to 1.0 values (clamped)");

		// Code path for non-PBO: and always uncompressed!
		// Volume/array textures are currently only supported if PixelBufferObjects are also supported.
		check(this->GetSizeZ() == 0);

		// Use a texture stage that's not likely to be used for draws, to avoid waiting
		FOpenGLDynamicRHI::Get().CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Target, GetResource(), -1, this->GetNumMips());

		FOpenGLDynamicRHI::Get().CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);

		// get the source data and size
		uint16* floatData = (uint16*)PixelBuffer->GetLockedBuffer();
		int32 texWidth = FMath::Max<uint32>(1, (this->GetSizeX() >> MipIndex));
		int32 texHeight = FMath::Max<uint32>(1, (this->GetSizeY() >> MipIndex));

		// always RGBA8 so 4 bytes / pixel
		int nValues = texWidth * texHeight * 4;
		uint8* rgbaData = (uint8*)FMemory::Malloc(nValues);

		// convert to GL_BYTE (saturate)
		uint8* outPtr = rgbaData;
		while (nValues--)
		{
			int32 pixelValue = (int32)(HalfFloatToFloat(*floatData++) * 255.0f);
			*outPtr++ = (uint8)(pixelValue < 0 ? 0 : (pixelValue < 256 ? pixelValue : 255));
		}

		// All construction paths should have called TexStorage2D or TexImage2D. So we will
		// always call TexSubImage2D.
		check(GetAllocatedStorageForMip(MipIndex, ArrayIndex) == true);
		glTexSubImage2D(
			bCubemap ? GL_TEXTURE_CUBE_MAP_POSITIVE_X + ArrayIndex : Target,
			MipIndex,
			0,
			0,
			texWidth,
			texHeight,
			GLFormat.Format,
			GLFormat.Type,
			rgbaData);

		// free temporary conversion buffer
		FMemory::Free(rgbaData);

		// Unlock "PixelBuffer" and free the temp memory after the texture upload.
		PixelBuffer->Unlock();

		// No need to restore texture stage; leave it like this,
		// and the next draw will take care of cleaning it up; or
		// next operation that needs the stage will switch something else in on it.

		FOpenGLDynamicRHI::Get().CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);

		return;
	}
#endif

	// Code path for PBO per slice
	check(IsValidRef(PixelBuffers[BufferIndex]));
			
	PixelBuffer->Unlock();

	// Modify permission?
	if (!PixelBuffer->IsLockReadOnly())
	{
		// Use a texture stage that's not likely to be used for draws, to avoid waiting
		FOpenGLDynamicRHI::Get().CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Target, GetResource(), -1, this->GetNumMips());

		const FUintVector3 MipExtents = UE::RHITextureUtils::CalculateMipExtents(GetDesc(), MipIndex);

		FillGLTextureImage(GLFormat, bSRGB, MipIndex, ArrayIndex, MipExtents, nullptr, PixelBuffer->GetSize());
	}

	//need to free PBO if we aren't keeping shadow copies
	PixelBuffers[BufferIndex] = NULL;

	// No need to restore texture stage; leave it like this,
	// and the next draw will take care of cleaning it up; or
	// next operation that needs the stage will switch something else in on it.

	FOpenGLDynamicRHI::Get().CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);
}


uint32 GTotalEvictedMipMemStored = 0;
uint32 GTotalEvictedMipMemDuplicated = 0;
uint32 GTotalMipStoredCount = 0;
uint32 GTotalMipRestores = 0;

extern uint32 GTotalEvictedMipMemStored;
extern uint32 GTotalTexStorageSkipped;

float GMaxRestoreTime = 0.0f;
float GAvgRestoreTime = 0.0f;
uint32 GAvgRestoreCount = 0;

void FOpenGLTexture::RestoreEvictedGLResource(bool bAttemptToRetainMips)
{
//	double StartTime = FPlatformTime::Seconds();

	QUICK_SCOPE_CYCLE_COUNTER(STAT_OpenGLRestoreEvictedTextureTime);

	check(!EvictionParamsPtr->bHasRestored);
	EvictionParamsPtr->bHasRestored = true;

	const FClearValueBinding ClearBinding = this->GetClearBinding();
	FOpenGLDynamicRHI::Get().InitializeGLTextureInternal(this, nullptr, 0);

	EPixelFormat PixelFormat = this->GetFormat();
	const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[PixelFormat];
	const bool bSRGB = EnumHasAnyFlags(this->GetFlags(), TexCreate_SRGB);
	checkf(EvictionParamsPtr->MipImageData.Num() == this->GetNumMips(), TEXT("EvictionParamsPtr->MipImageData.Num() =%d, this->GetNumMips() = %d"), EvictionParamsPtr->MipImageData.Num(), this->GetNumMips());

	FOpenGLDynamicRHI::Get().CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);

	for (int32 MipIndex = EvictionParamsPtr->MipImageData.Num() - 1; MipIndex >= 0; MipIndex--)
	{
		TConstArrayView<uint8> MipMem = EvictionParamsPtr->MipImageData[MipIndex];
		if (MipMem.Num())
		{
			const FUintVector3 MipExtents = UE::RHITextureUtils::CalculateMipExtents(GetDesc(), MipIndex);

			FillGLTextureImage(GLFormat, bSRGB, MipIndex, 0, MipExtents, MipMem.GetData(), MipMem.Num());
		}
	}

	// Use the resident streaming mips if our cvar is -1.
	uint32 DeferTextureCreationKeepLowerMipCount = (uint32)(GOGLDeferTextureCreationKeepLowerMipCount >= 0 ? GOGLDeferTextureCreationKeepLowerMipCount : UTexture::GetStaticMinTextureResidentMipCount());

	uint32 RetainMips = bAttemptToRetainMips && EnumHasAnyFlags(this->GetFlags(), TexCreate_Streamable) && this->GetNumMips() > 1 && !bAlias
		? DeferTextureCreationKeepLowerMipCount
		: 0;

	// keep the mips for streamable textures
	EvictionParamsPtr->ReleaseMipData(RetainMips);

	if (CanBeEvicted())
	{
		if (FTextureEvictionLRU::Get().Add(this) == false)
		{
			// could not store this in the LRU. Deleting all backup mips, as this texture will never be evicted.
			EvictionParamsPtr->ReleaseMipData(0);
		}
	}

#if GLDEBUG_LABELS_ENABLED
	if (FAnsiString& TextureDebugName = EvictionParamsPtr->GetDebugLabelName(); TextureDebugName.Len())
	{
		SetOpenGLResourceName(this, *TextureDebugName);
		if (RetainMips == 0)
		{
			TextureDebugName.Empty();
		}
	}
#endif

	GTotalEvictedMipMemDuplicated += EvictionParamsPtr->GetTotalAllocated();
// 	float ThisTime = (float)(FPlatformTime::Seconds() - StartTime);
// 	GAvgRestoreCount++;
// 	GMaxRestoreTime = FMath::Max(GMaxRestoreTime, ThisTime);
// 	GAvgRestoreTime += ThisTime;
}

void FOpenGLTexture::TryEvictGLResource()
{
	VERIFY_GL_SCOPE();
	if (bCanCreateAsEvicted && EvictionParamsPtr->bHasRestored)
	{
		if (CanBeEvicted())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_OpenGLTryEvictGLResource);
			DeleteGLResource();

			// create a new texture id.
			EvictionParamsPtr->bHasRestored = false;
			const FClearValueBinding ClearBinding = this->GetClearBinding();
			// recreate the GL tex resource name (but not allocate the memory)
			FOpenGLDynamicRHI::Get().InitializeGLTexture(this, nullptr, 0);
			GTotalEvictedMipMemDuplicated -= EvictionParamsPtr->GetTotalAllocated();
		}
	}
}

bool FOpenGLTextureDesc::CanDeferTextureCreation()
{
	bool bCanDeferTextureCreation = CVarDeferTextureCreation.GetValueOnAnyThread() != 0;
#if PLATFORM_ANDROID
	static bool bDeferTextureCreationConfigRulesChecked = false;
	static TOptional<bool> bConfigRulesCanDeferTextureCreation;
	if (!bDeferTextureCreationConfigRulesChecked)
	{
		const FString* ConfigRulesDeferOpenGLTextureCreationStr = FAndroidMisc::GetConfigRulesVariable(TEXT("DeferOpenGLTextureCreation"));
		if (ConfigRulesDeferOpenGLTextureCreationStr)
		{
			bConfigRulesCanDeferTextureCreation = ConfigRulesDeferOpenGLTextureCreationStr->Equals("true", ESearchCase::IgnoreCase);
			UE_LOGF(LogRHI, Log, "OpenGL deferred texture creation, set by config rules: %d", (int)bConfigRulesCanDeferTextureCreation.GetValue());
		}
		else
		{
			UE_LOGF(LogRHI, Log, "OpenGL deferred texture creation, no config rule set: %d", (int)bCanDeferTextureCreation);
		}
		bDeferTextureCreationConfigRulesChecked = true;
	}

	if (bConfigRulesCanDeferTextureCreation.IsSet())
	{
		bCanDeferTextureCreation = bConfigRulesCanDeferTextureCreation.GetValue();
	}
#endif
	return bCanDeferTextureCreation;
}

bool FOpenGLTexture::CanBeEvicted()
{
	VERIFY_GL_SCOPE();
	checkf(!bCanCreateAsEvicted || EvictionParamsPtr.IsValid(), TEXT("%p, bCanCreateAsEvicted %d, EvictionParamsPtr.IsValid() %d"), this, bCanCreateAsEvicted, EvictionParamsPtr.IsValid());

	// if we're aliased check that there's no eviction data.
	check(!bCanCreateAsEvicted || !bAlias || (EvictionParamsPtr->MipImageData.Num() == 0 && EvictionParamsPtr->MipImageData.Num() != this->GetNumMips()));

	// cant evict if we're aliased, or there are mips are not backed by stored data.
	bool bRet = bCanCreateAsEvicted && EvictionParamsPtr->MipImageData.Num() == this->GetNumMips() && EvictionParamsPtr->AreAllMipsPresent();

	return bRet;
}

void FOpenGLTexture::RemoveEvictionData()
{
	if (EvictionParamsPtr.IsValid() && CanBeEvicted())
	{
		// Make sure the GL resource is created before removing the MIP data.
		TryRestoreGLResource();

		EvictionParamsPtr->ReleaseMipData(0);
		FTextureEvictionLRU::Get().Remove(this);
	}
}

/*-----------------------------------------------------------------------------
	2D texture support.
-----------------------------------------------------------------------------*/

static UE::RHICore::FDefaultLayoutTextureInitializer CreateWritableTextureInitializer(FRHICommandListBase& RHICmdList, FOpenGLTexture* Texture, const FRHITextureCreateDesc& CreateDesc)
{
	const uint64 WritableMemorySize = UE::RHITextureUtils::CalculateTextureSize(CreateDesc);
	void* WritableMemory = FMemory::Malloc(WritableMemorySize, 16);

	// TODO: when not evicted, we can use PixelBuffer memory for the Initializer. This may get too complicated to justify.

	return UE::RHICore::FDefaultLayoutTextureInitializer(RHICmdList, Texture, WritableMemory, WritableMemorySize,
		[Texture = TRefCountPtr<FOpenGLTexture>(Texture), WritableMemory = UE::RHICore::FInitializerScopedMemory(WritableMemory), WritableMemorySize](FRHICommandListBase& RHICmdList) mutable
		{
			RHICmdList.EnqueueLambda(
				[Texture, WritableMemory = MoveTemp(WritableMemory), WritableMemorySize](FRHICommandListBase& RHICmdList)
				{
					FOpenGLDynamicRHI::Get().InitializeGLTexture(Texture, nullptr, 0);

					const FRHITextureDesc& TextureDesc = Texture->GetDesc();
					const uint32 FaceCount = TextureDesc.IsTextureCube() ? 6 : 1;

					for (uint32 FaceIndex = 0; FaceIndex < FaceCount; FaceIndex++)
					{
						for (uint32 ArrayIndex = 0; ArrayIndex < TextureDesc.ArraySize; ArrayIndex++)
						{
							for (uint32 MipIndex = 0; MipIndex < TextureDesc.NumMips; MipIndex++)
							{
								uint64 SubresourceStride = 0;
								uint64 SubresourceSize = 0;
								const uint64 Offset = UE::RHITextureUtils::CalculateSubresourceOffset(TextureDesc, FaceIndex, ArrayIndex, MipIndex, SubresourceStride, SubresourceSize);

								check((Offset + SubresourceSize) <= WritableMemorySize);

								const uint32 GLArrayIndex = FaceIndex + ArrayIndex * FaceCount;

								uint32 DestStride = 0;
								void* DestinationMemory = Texture->Lock(MipIndex, GLArrayIndex, RLM_WriteOnly, DestStride);

								FMemory::Memcpy(DestinationMemory, reinterpret_cast<const uint8*>(WritableMemory.Pointer) + Offset, SubresourceSize);
								Texture->Unlock(MipIndex, GLArrayIndex);
							}
						}
					}
				}
			);

			return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
		}
	);
}

FRHITextureInitializer FOpenGLDynamicRHI::RHICreateTextureInitializer(FRHICommandListBase& RHICmdList, const FRHITextureCreateDesc& CreateDesc)
{
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.OwnerName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(CreateDesc.GetTraceClassName(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(CreateDesc.DebugName, CreateDesc.GetTraceClassName(), CreateDesc.OwnerName);

	FOpenGLTexture* Texture = new FOpenGLTexture(CreateDesc);

	if (CreateDesc.InitAction == ERHITextureInitAction::Default)
	{
		return UE::RHICore::FBaseTextureInitializerImplementation(RHICmdList, Texture,
			[Texture = TRefCountPtr<FOpenGLTexture>(Texture)](FRHICommandListBase& RHICmdList) mutable
			{
				Texture->Initialize(RHICmdList);
				return TRefCountPtr<FRHITexture>(MoveTemp(Texture));
			}
		);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::BulkData)
	{
		check(CreateDesc.BulkData);

		UE::RHICore::FDefaultLayoutTextureInitializer Initializer = CreateWritableTextureInitializer(RHICmdList, Texture, CreateDesc);
		check(CreateDesc.BulkData->GetResourceBulkDataSize() <= Initializer.GetWritableSize());

		FMemory::Memcpy(Initializer.GetWritableData(), CreateDesc.BulkData->GetResourceBulkData(), CreateDesc.BulkData->GetResourceBulkDataSize());

		CreateDesc.BulkData->Discard();

		return MoveTemp(Initializer);
	}

	if (CreateDesc.InitAction == ERHITextureInitAction::Initializer)
	{
		return CreateWritableTextureInitializer(RHICmdList, Texture, CreateDesc);
	}

	return UE::RHICore::HandleUnknownTextureInitializerInitAction(RHICmdList, CreateDesc);
}


FTextureRHIRef FOpenGLDynamicRHI::RHIAsyncCreateTexture2D(uint32 SizeX, uint32 SizeY, uint8 Format, uint32 NumMips, ETextureCreateFlags Flags, ERHIAccess InResourceState, void** InitialMipData, uint32 NumInitialMips, const TCHAR* DebugName, FGraphEventRef& OutCompletionEvent)
{
	check(0);
	return FTextureRHIRef();
}

/** Generates mip maps for the surface. */
void FOpenGLDynamicRHI::RHIGenerateMips(FRHITexture* TextureRHI)
{
	VERIFY_GL_SCOPE();

	FOpenGLTexture* Texture = ResourceCast(TextureRHI);
	// Setup the texture on a disused unit
	// need to figure out how to setup mips properly in no views case
	CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Texture->Target, Texture->GetResource(), -1, Texture->GetNumMips());
	FOpenGL::GenerateMipmap(Texture->Target);
}



/**
 * Computes the size in memory required by a given texture.
 *
 * @param	TextureRHI		- Texture we want to know the size of
 * @return					- Size in Bytes
 */
uint32 FOpenGLDynamicRHI::RHIComputeMemorySize(FRHITexture* TextureRHI)
{
	if (!TextureRHI)
	{
		return 0;
	}

	FOpenGLTexture* Texture = ResourceCast(TextureRHI);
	return Texture->MemorySize;
}

void FOpenGLDynamicRHI::RHIUpdateTexture2D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion2D& UpdateRegion, uint32 SourcePitch, const uint8* SourceData)
{
	const FPixelFormatInfo& FormatInfo = GPixelFormats[TextureRHI->GetFormat()];

	check(UpdateRegion.Width  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.Height % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.DestX  % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.DestY  % FormatInfo.BlockSizeY == 0);
	check(UpdateRegion.SrcX   % FormatInfo.BlockSizeX == 0);
	check(UpdateRegion.SrcY   % FormatInfo.BlockSizeY == 0);

	const uint32 SrcXInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcX,   FormatInfo.BlockSizeX);
	const uint32 SrcYInBlocks   = FMath::DivideAndRoundUp<uint32>(UpdateRegion.SrcY,   FormatInfo.BlockSizeY);
	const uint32 WidthInBlocks  = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Width,  FormatInfo.BlockSizeX);
	const uint32 HeightInBlocks = FMath::DivideAndRoundUp<uint32>(UpdateRegion.Height, FormatInfo.BlockSizeY);

	const void* UpdateMemory = SourceData + FormatInfo.BlockBytes * SrcXInBlocks + SourcePitch * SrcYInBlocks;
	uint32 UpdatePitch = SourcePitch;

	const bool bNeedStagingMemory = RHICmdList.IsTopOfPipe();
	if (bNeedStagingMemory)
	{
		const size_t SourceDataSizeInBlocks = static_cast<size_t>(WidthInBlocks) * static_cast<size_t>(HeightInBlocks);
		const size_t SourceDataSize = SourceDataSizeInBlocks * FormatInfo.BlockBytes;

		uint8* const StagingMemory = (uint8*)FMemory::Malloc(SourceDataSize);
		const size_t StagingPitch = static_cast<size_t>(WidthInBlocks) * FormatInfo.BlockBytes;

		const uint8* CopySrc = (const uint8*)UpdateMemory;
		uint8* CopyDst = (uint8*)StagingMemory;
		for (uint32 BlockRow = 0; BlockRow < HeightInBlocks; BlockRow++)
		{
			FMemory::Memcpy(CopyDst, CopySrc, WidthInBlocks * FormatInfo.BlockBytes);
			CopySrc += SourcePitch;
			CopyDst += StagingPitch;
		}

		UpdateMemory = StagingMemory;
		UpdatePitch = StagingPitch;
	}

	RHICmdList.EnqueueLambda([this, TextureRHI, MipIndex, UpdateRegion, UpdatePitch, UpdateMemory, bNeedStagingMemory](FRHICommandListBase&)
	{
		VERIFY_GL_SCOPE();

		FOpenGLTexture* Texture = ResourceCast(TextureRHI);
		const EPixelFormat PixelFormat = TextureRHI->GetFormat();

		const FPixelFormatInfo& FormatInfo = GPixelFormats[PixelFormat];
		const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[PixelFormat];

		// Use a texture stage that's not likely to be used for draws, to avoid waiting
		CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Texture->Target, Texture->GetResource(), 0, Texture->GetNumMips());
		CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);

		glPixelStorei(GL_UNPACK_ROW_LENGTH, UpdatePitch / FormatInfo.BlockBytes);
		glPixelStorei(GL_UNPACK_ALIGNMENT, 1);

		if (GLFormat.bCompressed)
		{
			glCompressedTexSubImage2D(
				Texture->Target,
				MipIndex,
				UpdateRegion.DestX, UpdateRegion.DestY,
				UpdateRegion.Width, UpdateRegion.Height,
				GLFormat.Format,
				UpdatePitch * FMath::DivideAndRoundUp<uint32>(UpdateRegion.Height, FormatInfo.BlockSizeY),
				UpdateMemory);
			GL_CHECK();
		}
		else
		{
			glTexSubImage2D(
				Texture->Target,
				MipIndex,
				UpdateRegion.DestX, UpdateRegion.DestY,
				UpdateRegion.Width, UpdateRegion.Height,
				GLFormat.Format,
				GLFormat.Type,
				UpdateMemory);
			GL_CHECK();
		}

		glPixelStorei(GL_UNPACK_ALIGNMENT, 4);
		glPixelStorei(GL_UNPACK_ROW_LENGTH, 0);

		// No need to restore texture stage; leave it like this,
		// and the next draw will take care of cleaning it up; or
		// next operation that needs the stage will switch something else in on it.

		// free source data if we're on RHIT
		if (bNeedStagingMemory)
		{
			FMemory::Free(const_cast<void*>(UpdateMemory));
		}
	});
}

void FOpenGLDynamicRHI::RHIUpdateTexture3D(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, uint32 MipIndex, const FUpdateTextureRegion3D& UpdateRegion, uint32 SourceRowPitch, uint32 SourceDepthPitch, const uint8* SourceData)
{
	uint8* RHITSourceData = nullptr;
	if (RHICmdList.IsTopOfPipe())
	{
		uint32 DataSize = SourceDepthPitch * UpdateRegion.Depth;
		RHITSourceData = (uint8*)FMemory::Malloc(DataSize, 16);
		FMemory::Memcpy(RHITSourceData, SourceData, DataSize);
		SourceData = RHITSourceData;
	}

	RHICmdList.EnqueueLambda([this, TextureRHI, MipIndex, UpdateRegion, SourceRowPitch, SourceDepthPitch, SourceData, RHITSourceData](FRHICommandListBase&)
	{
		VERIFY_GL_SCOPE();
		check(FOpenGL::SupportsTexture3D());
		FOpenGLTexture* Texture = ResourceCast(TextureRHI);

		// Use a texture stage that's not likely to be used for draws, to avoid waiting
		CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Texture->Target, Texture->GetResource(), 0, Texture->GetNumMips());
		CachedBindBuffer(GL_PIXEL_UNPACK_BUFFER, 0);

		EPixelFormat PixelFormat = Texture->GetFormat();
		const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[PixelFormat];
		const FPixelFormatInfo& FormatInfo = GPixelFormats[PixelFormat];
		const uint32 FormatBPP = FormatInfo.BlockBytes;

		check(FOpenGL::SupportsTexture3D());
		// TO DO - add appropriate offsets to source data when necessary
		check(UpdateRegion.SrcX == 0);
		check(UpdateRegion.SrcY == 0);
		check(UpdateRegion.SrcZ == 0);

		glPixelStorei(GL_UNPACK_ALIGNMENT, 1);

		const bool bSRGB = EnumHasAnyFlags(Texture->GetFlags(), TexCreate_SRGB);

		if (GLFormat.bCompressed)
		{
			FOpenGL::CompressedTexSubImage3D(
				Texture->Target,
				MipIndex,
				UpdateRegion.DestX, UpdateRegion.DestY, UpdateRegion.DestZ,
				UpdateRegion.Width, UpdateRegion.Height, UpdateRegion.Depth,
				GLFormat.InternalFormat[bSRGB],
				SourceDepthPitch * UpdateRegion.Depth,
				SourceData);
		}
		else
		{
			glPixelStorei(GL_UNPACK_ROW_LENGTH, UpdateRegion.Width / FormatInfo.BlockSizeX);
			glPixelStorei(GL_UNPACK_IMAGE_HEIGHT, UpdateRegion.Height / FormatInfo.BlockSizeY);

			FOpenGL::TexSubImage3D(Texture->Target, MipIndex, UpdateRegion.DestX, UpdateRegion.DestY, UpdateRegion.DestZ, UpdateRegion.Width, UpdateRegion.Height, UpdateRegion.Depth, GLFormat.Format, GLFormat.Type, SourceData);
		}

		glPixelStorei(GL_UNPACK_ALIGNMENT, 4);
		glPixelStorei(GL_UNPACK_IMAGE_HEIGHT, 0);
		glPixelStorei(GL_UNPACK_ROW_LENGTH, 0);

		// free source data if we're on RHIT
		if (RHITSourceData)
		{
			FMemory::Free(RHITSourceData);
		}

		// No need to restore texture stage; leave it like this,
		// and the next draw will take care of cleaning it up; or
		// next operation that needs the stage will switch something else in on it.
	});
}

void FOpenGLDynamicRHI::InvalidateTextureResourceInCache(GLuint Resource)
{
	VERIFY_GL_SCOPE();
	auto InvalidateContextTextureResources = [&Resource](TArray<FTextureStage>& Textures)
	{
		for (FTextureStage& TextureStage : Textures)
		{
			if (TextureStage.Resource == Resource)
			{
				TextureStage.Target = GL_NONE;
				TextureStage.Resource = 0;
			}
		}
	};
	InvalidateContextTextureResources(ContextState.Textures);
	InvalidateContextTextureResources(PendingState.Textures);
	
	TextureMipLimits.Remove(Resource);
	
	if (PendingState.DepthStencil && PendingState.DepthStencil->GetResource() == Resource)
	{
		PendingState.DepthStencil = nullptr;
	}
}

void FOpenGLDynamicRHI::InvalidateUAVResourceInCache(GLuint Resource)
{
	VERIFY_GL_SCOPE();
	int32 NumUAVs = ContextState.UAVs.Num();
	
	for (int32 UAVIndex = 0; UAVIndex < NumUAVs; ++UAVIndex)
	{
		if (ContextState.UAVs[UAVIndex].Resource == Resource)
		{
			ContextState.UAVs[UAVIndex].Format = GL_NONE;
			ContextState.UAVs[UAVIndex].Resource = 0;
		}

		if (PendingState.UAVs[UAVIndex].Resource == Resource)
		{
			PendingState.UAVs[UAVIndex].Format = GL_NONE;
			PendingState.UAVs[UAVIndex].Resource = 0;
		}
	}
}

/*-----------------------------------------------------------------------------
	Cubemap texture support.
-----------------------------------------------------------------------------*/

void FOpenGLDynamicRHI::RHIBindDebugLabelName(FRHICommandListBase& RHICmdList, FRHITexture* TextureRHI, const TCHAR* Name)
{
	TextureRHI->SetName(Name);

#if GLDEBUG_LABELS_ENABLED
	RHICmdList.EnqueueLambda(
		[TextureRHI, Name = FAnsiString(Name)] (FRHICommandListBase& RHICmdList)
		{
			SetOpenGLResourceName(ResourceCast(TextureRHI), *Name);
		}
	);
#endif
}

void FOpenGLDynamicRHI::RHICopyTexture(FRHITexture* SourceTextureRHI, FRHITexture* DestTextureRHI, const FRHICopyTextureInfo& CopyInfo)
{
	VERIFY_GL_SCOPE();
	FOpenGLTexture* SourceTexture = ResourceCast(SourceTextureRHI);
	FOpenGLTexture* DestTexture = ResourceCast(DestTextureRHI);

	if (DestTexture->IsEvicted()
		&& SourceTexture->EvictionParamsPtr.IsValid()
		&& SourceTexture->GetFormat() == DestTexture->GetFormat()
		&& CopyInfo.SourceSliceIndex == 0
		&& CopyInfo.DestSliceIndex == 0
		&& CopyInfo.NumSlices == 1
		&& CopyInfo.SourcePosition.IsZero()
		&& CopyInfo.DestPosition.IsZero())
	{
		// Both source and destination textures are candidates for eviction.
		// We might be able to achieve the copy entirely offline by copying the CPU-side mip data, rather than involving the GPU, leaving the destination evicted.

		// Check the dimensions of the copy are compatible
		FUintVector3 SrcMipExtent = UE::RHITextureUtils::CalculateMipExtents(SourceTexture->GetDesc(), CopyInfo.SourceMipIndex);
		FUintVector3 DstMipExtent = UE::RHITextureUtils::CalculateMipExtents(DestTexture->GetDesc(), CopyInfo.DestMipIndex);

		if (SrcMipExtent == DstMipExtent && (CopyInfo.Size.IsZero() || CopyInfo.Size == FIntVector3(SrcMipExtent)))
		{
			// Check all the required mips from the source are available to copy from
			bool bRequiredSrcMipsPresent = true;
			const TArray<TArray<uint8>>& SrcMipData = SourceTexture->EvictionParamsPtr->MipImageData;

			for (uint32 Mip = CopyInfo.SourceMipIndex, Index = 0; Index < CopyInfo.NumMips; ++Mip, ++Index)
			{
				if (!SrcMipData.IsValidIndex(Mip) || SrcMipData[Mip].IsEmpty())
				{
					bRequiredSrcMipsPresent = false;
					break;
				}
			}

			if (bRequiredSrcMipsPresent)
			{
				// We're replacing entire mips in the destination with copies from the source.
				// Clone the offline mip data directly from the source.
				DestTexture->EvictionParamsPtr->CloneMipData(*SourceTexture->EvictionParamsPtr, CopyInfo.NumMips, CopyInfo.SourceMipIndex, CopyInfo.DestMipIndex);

				// Leave the destination texture evicted.
				return;
			}
		}
	}

	//since the texture will be modified we cannot use the eviction data.
	DestTexture->RemoveEvictionData();

	GLsizei Width, Height, Depth;

	if (CopyInfo.Size == FIntVector::ZeroValue)
	{
		const FRHITextureDesc& SourceDesc = SourceTextureRHI->GetDesc();

		// Copy whole texture when zero vector is specified for region size.
		FIntVector SrcTexSize = SourceDesc.GetSize();
		Width = FMath::Max(1, SrcTexSize.X >> CopyInfo.SourceMipIndex);
		Height = FMath::Max(1, SrcTexSize.Y >> CopyInfo.SourceMipIndex);
		switch (SourceTexture->Target)
		{
		case GL_TEXTURE_3D:			Depth = FMath::Max(1, SrcTexSize.Z >> CopyInfo.SourceMipIndex); break;
		case GL_TEXTURE_CUBE_MAP:	Depth = 6; break;
		default:					Depth = 1; break;
		}
		ensure(CopyInfo.SourcePosition == FIntVector::ZeroValue);
	}
	else
	{
		Width = CopyInfo.Size.X;
		Height = CopyInfo.Size.Y;
		switch (SourceTexture->Target)
		{
			case GL_TEXTURE_3D:			Depth = CopyInfo.Size.Z; break;
			case GL_TEXTURE_CUBE_MAP:	Depth = CopyInfo.NumSlices; break;
			default:					Depth = 1; break;
		}
	}

	GLint SrcMip = CopyInfo.SourceMipIndex;
	GLint DestMip = CopyInfo.DestMipIndex;

	for (uint32 MipIndex = 0; MipIndex < CopyInfo.NumMips; ++MipIndex)
	{
		// X, Y, Z
		FIntVector Src, Dst;

		auto SetOffsets = [MipIndex, &CopyInfo, &Depth] (const GLenum Target, const FIntVector& Position, const uint32& SliceIndex, FIntVector& OutOffsets)
		{
			switch (Target)
			{
			case GL_TEXTURE_3D:
			case GL_TEXTURE_CUBE_MAP:
				// For cube maps, the Z offsets select the starting faces.
				OutOffsets.Z = Position.Z >> MipIndex;
				break;
			case GL_TEXTURE_1D_ARRAY:
			case GL_TEXTURE_2D_ARRAY:
				// For texture arrays, the Z offsets and depth actually refer to the range of slices to copy.
				OutOffsets.Z = SliceIndex;
				Depth = CopyInfo.NumSlices;
				break;
			default:
				OutOffsets.Z = 0;
				break;
			}

			OutOffsets.X = Position.X >> MipIndex;
			OutOffsets.Y = Position.Y >> MipIndex;
		};

		SetOffsets(SourceTexture->Target, CopyInfo.SourcePosition, CopyInfo.SourceSliceIndex, Src);
		SetOffsets(DestTexture->Target, CopyInfo.DestPosition, CopyInfo.DestSliceIndex, Dst);

		FOpenGL::CopyImageSubData(SourceTexture->GetResource(), SourceTexture->Target, SrcMip, Src.X, Src.Y, Src.Z,
			DestTexture->GetResource(), DestTexture->Target, DestMip, Dst.X, Dst.Y, Dst.Z,
			Width, Height, Depth);

		++SrcMip;
		++DestMip;

		Width = FMath::Max(1, Width >> 1);
		Height = FMath::Max(1, Height >> 1);
		if(DestTexture->Target == GL_TEXTURE_3D)
		{
			Depth = FMath::Max(1, Depth >> 1);
		}
	}
}


FTextureRHIRef FOpenGLDynamicRHI::RHICreateTexture2DFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags TexCreateFlags)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("RHICreateTexture2DFromResource"), SizeX, SizeY, Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(TexCreateFlags)
		.SetNumMips(NumMips)
		.SetNumSamples(NumSamples)
		.DetermineInititialState();

	return new FOpenGLTexture(Desc, Resource);
}

#if PLATFORM_ANDROID
FTextureRHIRef FOpenGLDynamicRHI::RHICreateTexture2DFromAndroidHardwareBuffer(FRHICommandListBase& RHICmdList, AHardwareBuffer* HardwareBuffer)
{
	check(HardwareBuffer);

	AHardwareBuffer_Desc HardwareBufferDesc;
	AHardwareBuffer_describe(HardwareBuffer, &HardwareBufferDesc);
	check((HardwareBufferDesc.usage & AHARDWAREBUFFER_USAGE_GPU_SAMPLED_IMAGE) != 0);
	
	// OpenGL driver will auto convert the color space for YUV (tested 420 and P010).
	// But we should test with different video formats to identify what is not supported.
	// UE_LOGF(LogRHI, Log, "[Media] hardware buffer format: 0x%x", HardwareBufferDesc.format & 0xff);

	// Temp for avoiding a checking failure of PF_Unknown in FRHITextureDesc::Validate()
	EPixelFormat PixelFormat = PF_R8G8B8A8;

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("OpenGLTexture2DFromAndroidHardwareBuffer"), HardwareBufferDesc.width, HardwareBufferDesc.height, PixelFormat)
		.SetFlags(ETextureCreateFlags::External)
		.DetermineInititialState();

	return new FOpenGLTexture(RHICmdList, Desc, HardwareBufferDesc, HardwareBuffer);
}
#endif //PLATFORM_ANDROID

FTextureRHIRef FOpenGLDynamicRHI::RHICreateTexture2DArrayFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags TexCreateFlags)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2DArray(TEXT("RHICreateTexture2DArrayFromResource"), SizeX, SizeY, ArraySize, Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(TexCreateFlags)
		.SetNumMips(NumMips)
		.SetNumSamples(NumSamples)
		.DetermineInititialState();

	return new FOpenGLTexture(Desc, Resource);
}

FTextureRHIRef FOpenGLDynamicRHI::RHICreateTextureCubeFromResource(EPixelFormat Format, uint32 Size, bool bArray, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags TexCreateFlags)
{
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create(TEXT("RHICreateTextureCubeFromResource"), bArray ? ETextureDimension::TextureCube : ETextureDimension::TextureCubeArray)
		.SetExtent(Size)
		.SetArraySize(bArray ? ArraySize : 1)
		.SetFormat(Format)
		.SetClearValue(ClearValueBinding)
		.SetFlags(TexCreateFlags)
		.SetNumMips(NumMips)
		.SetNumSamples(NumSamples)
		.DetermineInititialState();

	return new FOpenGLTexture(Desc, Resource);
}

void FOpenGLDynamicRHI::RHIAliasTextureResources(FTextureRHIRef& DestRHITexture, FTextureRHIRef& SrcRHITexture)
{
	VERIFY_GL_SCOPE();
	FOpenGLTexture* DestTexture = ResourceCast(DestRHITexture);
	FOpenGLTexture* SrcTexture = ResourceCast(SrcRHITexture);

	if (DestTexture && SrcTexture)
	{
		DestTexture->AliasResources(*SrcTexture);
	}
}

FTextureRHIRef FOpenGLDynamicRHI::RHICreateAliasedTexture(FTextureRHIRef& SourceTexture)
{
	const FString Name = SourceTexture->GetName().ToString() + TEXT("Alias");
	return new FOpenGLTexture(*ResourceCast(SourceTexture), *Name, FOpenGLTexture::AliasResource);
}

FRHILockTextureResult FOpenGLDynamicRHI::RHILockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	check(IsInRenderingThread());
	check(Arguments.LockMode != EResourceLockMode::RLM_WriteOnly_NoOverwrite);

	FOpenGLTexture* Texture = ResourceCast(Arguments.Texture);
	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Texture->GetDesc(), Arguments);

	FRHILockTextureResult Result;

	if (Arguments.LockMode == RLM_ReadOnly)
	{
		RHICmdList.EnqueueLambda([this, &Result, Texture, MipIndex = Arguments.MipIndex, ArrayIndex](FRHICommandListImmediate&)
		{
			Result.Data = Texture->Lock(MipIndex, ArrayIndex, RLM_ReadOnly, Result.Stride);
		});

		RHITHREAD_GLTRACE_BLOCKING;
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		Result.ByteCount = Texture->GetLockSize(Arguments.MipIndex, RLM_ReadOnly, Result.Stride);
	}
	else
	{
		Result.ByteCount = Texture->GetLockSize(Arguments.MipIndex, RLM_WriteOnly, Result.Stride);
		Result.Data = FMemory::Malloc(Result.ByteCount, 16);
	}

	check(Result.Data);
	GLLockTracker.Lock(Texture, Result.Data, ArrayIndex, Arguments.MipIndex, Result.Stride, Result.ByteCount, Arguments.LockMode);
	return Result;
}

void FOpenGLDynamicRHI::RHIUnlockTexture(FRHICommandListImmediate& RHICmdList, const FRHILockTextureArgs& Arguments)
{
	check(IsInRenderingThread());

	FOpenGLTexture* Texture = ResourceCast(Arguments.Texture);
	const uint32 ArrayIndex = UE::RHICore::GetLockArrayIndex(Texture->GetDesc(), Arguments);

	FTextureLockTracker::FLockParams Params = GLLockTracker.Unlock(Texture, ArrayIndex, Arguments.MipIndex);
	if (Params.LockMode == RLM_ReadOnly)
	{
		RHICmdList.EnqueueLambda([this, Texture, MipIndex = Arguments.MipIndex, ArrayIndex](FRHICommandListImmediate&)
		{
			Texture->Unlock(MipIndex, ArrayIndex);
		});
	}
	else
	{
		RHICmdList.EnqueueLambda([this, Params, Texture, MipIndex = Arguments.MipIndex, ArrayIndex](FRHICommandListImmediate&)
		{
			uint32 DestStride = 0;
			void* DestinationMemory = Texture->Lock(MipIndex, ArrayIndex, RLM_WriteOnly, DestStride);
			check(DestStride == Params.Stride);

			FMemory::Memcpy(DestinationMemory, Params.Buffer, Params.BufferSize);
			FMemory::Free(Params.Buffer);

			Texture->Unlock(MipIndex, ArrayIndex);
		});
	}
}

void LogTextureEvictionDebugInfo()
{
	static int counter = 0;
	if (GOGLTextureEvictLogging && ++counter == 100)
	{
		UE_LOGF(LogRHI, Warning, "txdbg: Texture mipmem %d. GTotalTexStorageSkipped %d, GTotalCompressedTexStorageSkipped %d, Total noncompressed = %d", GTotalEvictedMipMemStored, GTotalTexStorageSkipped, GTotalCompressedTexStorageSkipped, GTotalTexStorageSkipped - GTotalCompressedTexStorageSkipped);
		UE_LOGF(LogRHI, Warning, "txdbg: Texture GTotalEvictedMipMemDuplicated %d", GTotalEvictedMipMemDuplicated);
		UE_LOGF(LogRHI, Warning, "txdbg: Texture GTotalMipRestores %d, GTotalMipStoredCount %d", GTotalMipRestores, GTotalMipStoredCount);
		UE_LOGF(LogRHI, Warning, "txdbg: Texture GAvgRestoreTime %f (%d), GMaxRestoreTime %f, TotalRestoreTime %f", GAvgRestoreCount ? (float)(GAvgRestoreTime / GAvgRestoreCount) : 0.f, GAvgRestoreCount, (float)GMaxRestoreTime, (float)GAvgRestoreTime);
		UE_LOGF(LogRHI, Warning, "txdbg: Texture LRU %d", FTextureEvictionLRU::Get().Num());

		GAvgRestoreCount = 0;
		GMaxRestoreTime = 0;
		GAvgRestoreTime = 0;

		counter = 0;
	}
}

void FTextureEvictionLRU::TickEviction()
{
#if (UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT)
	LogTextureEvictionDebugInfo();
#endif

	FScopeLock Lock(&TextureLRULock);
	FOpenGLTextureLRUContainer& TextureLRU = GetLRUContainer();

	for (int32 EvictCount = 0; 
		TextureLRU.Num() > FMath::Max(0, GOGLTextureMinLRUCapacity) && (TextureLRU.GetLeastRecent()->EvictionParamsPtr->FrameLastRendered + GOGLTextureEvictFramesToLive) < GFrameNumberRenderThread && EvictCount < GOGLTexturesToEvictPerFrame
		;EvictCount++)
	{
		FOpenGLTexture* RemovedFromLRU = TextureLRU.RemoveLeastRecent();
		RemovedFromLRU->EvictionParamsPtr->LRUNode = FSetElementId();
		RemovedFromLRU->TryEvictGLResource();
	}
}

void FTextureEvictionLRU::Remove(FOpenGLTexture* TextureBase)
{
	if( TextureBase->EvictionParamsPtr.IsValid() )
	{
		FScopeLock Lock(&TextureLRULock);

		check(!TextureBase->EvictionParamsPtr->LRUNode.IsValidId() || GetLRUContainer().Contains(TextureBase));
		check(TextureBase->EvictionParamsPtr->LRUNode.IsValidId() || !GetLRUContainer().Contains(TextureBase));
		if( TextureBase->EvictionParamsPtr->LRUNode.IsValidId())
		{
			GetLRUContainer().Remove(TextureBase);
			TextureBase->EvictionParamsPtr->LRUNode = FSetElementId();
		}
	}
}

bool FTextureEvictionLRU::Add(FOpenGLTexture* TextureBase)
{
	FScopeLock Lock(&TextureLRULock); 
	check(TextureBase->EvictionParamsPtr);
	check(!TextureBase->EvictionParamsPtr->LRUNode.IsValidId())
	FOpenGLTextureLRUContainer& TextureLRU = GetLRUContainer();
	check(!TextureLRU.Contains(TextureBase));

	if(ensure(TextureLRU.Num() != TextureLRU.Max()))
	{
		TextureBase->EvictionParamsPtr->LRUNode = TextureLRU.Add(TextureBase, TextureBase);
		TextureBase->EvictionParamsPtr->FrameLastRendered = GFrameNumberRenderThread;
		return true;
	}
	return false;
}

void FTextureEvictionLRU::Touch(FOpenGLTexture* TextureBase)
{
	FScopeLock Lock(&TextureLRULock);
	check(TextureBase->EvictionParamsPtr);
	check(TextureBase->EvictionParamsPtr->LRUNode.IsValidId())
	check(GetLRUContainer().Contains(TextureBase));
	GetLRUContainer().MarkAsRecent(TextureBase->EvictionParamsPtr->LRUNode);
	TextureBase->EvictionParamsPtr->FrameLastRendered = GFrameNumberRenderThread;
}

FOpenGLTexture* FTextureEvictionLRU::GetLeastRecent()
{
	return GetLRUContainer().GetLeastRecent();
}


FTextureEvictionParams::FTextureEvictionParams(uint32 NumMips) : bHasRestored(0), FrameLastRendered(0)
{
	MipImageData.Reserve(NumMips);
	MipImageData.SetNum(NumMips);
}

FTextureEvictionParams::~FTextureEvictionParams()
{
	VERIFY_GL_SCOPE();

	if (bHasRestored)
	{
		GTotalEvictedMipMemDuplicated -= GetTotalAllocated();
	}

	for (int i = MipImageData.Num() - 1; i >= 0; i--)
	{
		GTotalEvictedMipMemStored -= MipImageData[i].Num();
	}
	GTotalMipStoredCount -= MipImageData.Num();
}

void FTextureEvictionParams::SetMipData(uint32 MipIndex, const void* Data, uint32 Bytes)
{
	checkf(Bytes, TEXT("FTextureEvictionParams::SetMipData: MipIndex %d, Data %p, Bytes %d)"), MipIndex, Data, Bytes);
	ensure(MipIndex < (uint32)MipImageData.Num());

	VERIFY_GL_SCOPE();
	if (MipImageData[MipIndex].Num() == 0)
	{
		GTotalMipStoredCount++;
	}
	MipImageData[MipIndex].Reserve(Bytes);
	MipImageData[MipIndex].SetNumUninitialized(Bytes);
	if (Data)
	{
		FMemory::Memcpy(MipImageData[MipIndex].GetData(), Data, Bytes);
	}
	GTotalEvictedMipMemStored += Bytes;
}

void FTextureEvictionParams::CloneMipData(const FTextureEvictionParams& Src, uint32 InNumMips, int32 SrcOffset, int DstOffset)
{
	VERIFY_GL_SCOPE();

	int32 MaxMip = FMath::Min((int32)InNumMips, (int32)Src.MipImageData.Num() - SrcOffset);
	for (int32 MipIndex = 0; MipIndex < MaxMip; ++MipIndex)
	{
		if (MipImageData[MipIndex + DstOffset].Num())
		{
			checkNoEntry();
		}
		else
		{
			GTotalMipStoredCount++;
		}
		MipImageData[MipIndex + DstOffset] = Src.MipImageData[MipIndex + SrcOffset];
		GTotalEvictedMipMemStored += MipImageData[MipIndex + DstOffset].Num();
	}
}

void FTextureEvictionParams::ReleaseMipData(uint32 RetainMips)
{
	VERIFY_GL_SCOPE();

	for (int i = MipImageData.Num() - 1 - RetainMips; i >= 0; i--)
	{
		GTotalEvictedMipMemStored -= MipImageData[i].Num();
		GTotalMipStoredCount -= MipImageData[i].Num() ? 1 : 0;
		MipImageData[i].Empty();
	}

	// if we're retaining mips then keep entire MipImageData array to ensure there's no MipIndex confusion.
	if (RetainMips == 0)
	{
		MipImageData.Empty();
	}
}
