// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "RenderUtils.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "ClearReplacementShaders.h"

void FOpenGLViewableResource::UpdateLinkedViews()
{
	for (FOpenGLView* View = LinkedViews; View; View = View->Next())
	{
		View->UpdateView();
	}
}

// Binds the specified buffer range to a texture resource and selects glTexBuffer or glTexBufferRange 
static void BindGLTexBufferRange(GLenum Target, GLenum InternalFormat, GLuint Buffer, uint32 StartOffsetBytes, uint32 NumElements, uint32 Stride)
{
	if (StartOffsetBytes == 0 && NumElements == UINT32_MAX)
	{
		FOpenGL::TexBuffer(Target, InternalFormat, Buffer);
	}
	else
	{
		// Validate buffer offset is a multiple of buffer offset alignment
		GLintptr Offset = StartOffsetBytes;
		GLsizeiptr Size = NumElements * Stride;

#if DO_CHECK
		GLint Alignment = FOpenGLBase::GetTextureBufferAlignment();
		check(Stride > 0 && Offset % Alignment == 0);
#endif

		FOpenGL::TexBufferRange(Target, InternalFormat, Buffer, Offset, Size);
	}
}

FOpenGLShaderResourceView::FOpenGLShaderResourceView(FRHICommandListBase& RHICmdList, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	: FRHIShaderResourceView(InResource, InViewDesc)
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		UpdateView();
	});
}

FOpenGLViewableResource* FOpenGLShaderResourceView::GetBaseResource() const
{
	return IsBuffer()
		? static_cast<FOpenGLViewableResource*>(FOpenGLDynamicRHI::ResourceCast(GetBuffer()))
		: static_cast<FOpenGLViewableResource*>(FOpenGLDynamicRHI::ResourceCast(GetTexture()));
}

void FOpenGLShaderResourceView::UpdateView()
{
	VERIFY_GL_SCOPE();

	Invalidate();

	if (ViewDesc.IsBuffer())
	{
		FOpenGLBuffer* Buffer = FOpenGLDynamicRHI::ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.SRV.GetViewInfo(Buffer);

		if (!Info.bNullView)
		{
			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Typed:
				Target = GL_TEXTURE_BUFFER;
				OwnsResource = true;

				FOpenGL::GenTextures(1, &Resource);

				// Use a texture stage that's not likely to be used for draws, to avoid waiting
				FOpenGLDynamicRHI::Get().CachedSetupTextureStage(
					FOpenGL::GetMaxCombinedTextureImageUnits() - 1,
					GL_TEXTURE_BUFFER,
					Resource,
					-1,
					1
				);

				BindGLTexBufferRange(
					GL_TEXTURE_BUFFER,
					GOpenGLTextureFormats[Info.Format].InternalFormat[0],
					Buffer->Resource,
					Info.OffsetInBytes,
					Info.NumElements,
					GPixelFormats[Info.Format].BlockBytes
				);
				break;
			
			case FRHIViewDesc::EBufferType::Raw:
			case FRHIViewDesc::EBufferType::Structured:
				//TODO: add range views for SSBO
				ensure(Info.OffsetInBytes == 0 && Info.SizeInBytes == Buffer->GetDesc().Size);
				Target = GL_SHADER_STORAGE_BUFFER;
				Resource = Buffer->Resource;
				break;
				
			default:
				checkNoEntry(); // not implemented
				break;
			}
		}
	}
	else
	{
		FOpenGLTexture* Texture = FOpenGLDynamicRHI::ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.SRV.GetViewInfo(Texture);

		Target = Texture->Target;
		LimitMip = Info.MipRange.Num == 1 ? Info.MipRange.First : -1;

		ensureMsgf(Info.MipRange.ExclusiveLast() == Texture->GetDesc().NumMips, TEXT("OpenGL RHI does not currently implement creation of SRVs that don't include all mips in the mip tail."));
		ensureMsgf(Info.bAllSlices, TEXT("SRV array slice selection is currently unimplemented in OpenGL RHI."));

		const bool bFormatsMatch = Info.Format == Texture->GetFormat() || Info.Format == PF_X24_G8;

		checkf(bFormatsMatch, TEXT("SRVs cannot modify the pixel format of a texture when texture views are unsupported."));
		Resource = Texture->GetResource();

		// Handle the custom stencil SRV
		if (Info.Format == PF_X24_G8)
		{
			// Use a texture stage that's not likely to be used for draws, to avoid waiting
			FOpenGLDynamicRHI::Get().CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, Target, Resource, LimitMip, Texture->GetNumMips());

			//set the texture to return the stencil index, and then force the components to match D3D
			glTexParameteri(Target, GL_DEPTH_STENCIL_TEXTURE_MODE, GL_STENCIL_INDEX);
			glTexParameteri(Target, GL_TEXTURE_SWIZZLE_R, GL_ZERO);
			glTexParameteri(Target, GL_TEXTURE_SWIZZLE_G, GL_RED);
			glTexParameteri(Target, GL_TEXTURE_SWIZZLE_B, GL_ZERO);
			glTexParameteri(Target, GL_TEXTURE_SWIZZLE_A, GL_ZERO);

			Texture->bInStencilTextureMode = true;
		}
	}
}

void FOpenGLShaderResourceView::Invalidate()
{
	if (Resource && OwnsResource)
	{
		FOpenGLDynamicRHI::Get().InvalidateTextureResourceInCache(Resource);
		FOpenGL::DeleteTextures(1, &Resource);
	}

	Resource = GL_NONE;
	Target = GL_TEXTURE_BUFFER;
	LimitMip = -1;
	OwnsResource = false;
}

FOpenGLShaderResourceView::~FOpenGLShaderResourceView()
{
	VERIFY_GL_SCOPE();

	Unlink();
	Invalidate();
}

FOpenGLUnorderedAccessView::FOpenGLUnorderedAccessView(FRHICommandListBase& RHICmdList, FRHIViewableResource* InResource, FRHIViewDesc const& InViewDesc)
	: FRHIUnorderedAccessView(InResource, InViewDesc)
{
	RHICmdList.EnqueueLambda([this](FRHICommandListBase&)
	{
		LinkHead(GetBaseResource()->LinkedViews);
		UpdateView();
	});
}

FOpenGLViewableResource* FOpenGLUnorderedAccessView::GetBaseResource() const
{
	return IsBuffer()
		? static_cast<FOpenGLViewableResource*>(FOpenGLDynamicRHI::ResourceCast(GetBuffer()))
		: static_cast<FOpenGLViewableResource*>(FOpenGLDynamicRHI::ResourceCast(GetTexture()));
}

void FOpenGLUnorderedAccessView::UpdateView()
{
	VERIFY_GL_SCOPE();

	Invalidate();

	if (IsBuffer())
	{
		FOpenGLBuffer* Buffer = FOpenGLDynamicRHI::ResourceCast(GetBuffer());
		auto const Info = ViewDesc.Buffer.UAV.GetViewInfo(Buffer);

		if (!Info.bNullView)
		{
			const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[Info.Format];
			this->UnrealFormat = Info.Format;
			this->Format = GLFormat.InternalFormat[0];

			ensureMsgf(Info.SizeInBytes == Buffer->GetDesc().Size && Info.OffsetInBytes == 0, TEXT("Only whole-resource UAVs are currently implemented in OpenGL RHI"));
			checkf(!Info.bAppendBuffer && !Info.bAtomicCounter, TEXT("Atomic counter / append buffer not implemented in OpenGL RHI."));
			
			switch (Info.BufferType)
			{
			case FRHIViewDesc::EBufferType::Typed:
				{
					GLuint TextureID = 0;
					FOpenGL::GenTextures(1, &TextureID);

					// Use a texture stage that's not likely to be used for draws, to avoid waiting
					FOpenGLDynamicRHI::Get().CachedSetupTextureStage(FOpenGL::GetMaxCombinedTextureImageUnits() - 1, GL_TEXTURE_BUFFER, TextureID, -1, 1);
					FOpenGL::TexBuffer(GL_TEXTURE_BUFFER, GLFormat.InternalFormat[0], Buffer->Resource);

					// No need to restore texture stage; leave it like this,
					// and the next draw will take care of cleaning it up; or
					// next operation that needs the stage will switch something else in on it.
					this->Resource = TextureID;
					this->OwnsResource = true;
				}
				break;

			case FRHIViewDesc::EBufferType::Structured:
			case FRHIViewDesc::EBufferType::Raw:
				this->BufferResource = Buffer->Resource;
				break;

			default:
				checkNoEntry();
				break;
			}
		}
	}
	else
	{
		FOpenGLTexture* Texture = FOpenGLDynamicRHI::ResourceCast(GetTexture());
		auto const Info = ViewDesc.Texture.UAV.GetViewInfo(Texture);

		ensureMsgf(Info.bAllSlices, TEXT("UAV array slice selection is currently unimplemented in OpenGL RHI."));

		const FOpenGLTextureFormat& GLFormat = GOpenGLTextureFormats[Info.Format];

		check(!Texture->CanBeEvicted() && !Texture->IsEvicted());
		this->Resource = Texture->GetResource();
		this->Format = GLFormat.InternalFormat[0];
		this->UnrealFormat = Info.Format;
		this->Level = Info.MipLevel;
	}
}

void FOpenGLUnorderedAccessView::Invalidate()
{
	if (Resource && OwnsResource)
	{
		FOpenGLDynamicRHI::Get().InvalidateTextureResourceInCache(Resource);
		FOpenGL::DeleteTextures(1, &Resource);
	}

	Resource = 0;
	BufferResource = 0;
	Format = 0;
	UnrealFormat = 0;
	Level = 0;
}

FOpenGLUnorderedAccessView::~FOpenGLUnorderedAccessView()
{
	VERIFY_GL_SCOPE();
	Unlink();
	Invalidate();
}

void FOpenGLDynamicRHI::RHIClearUAVFloat(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FVector4f& Values)
{
	FOpenGLUnorderedAccessView* UAV = ResourceCast(UnorderedAccessViewRHI);

	// Use compute on ES3.1
	TRHICommandList_RecursiveHazardous<FOpenGLDynamicRHI> RHICmdList(this);

	if (UAV->IsTexture())
	{
		FIntVector Size = UAV->GetTexture()->GetSizeXYZ();

		if (UAV->IsLayered())
		{
			ClearUAVShader_T<EClearReplacementResourceType::Texture3D, EClearReplacementValueType::Float, 4, false>(RHICmdList, UnorderedAccessViewRHI, Size.X, Size.Y, Size.Z, *reinterpret_cast<const float(*)[4]>(&Values));
		}
		else
		{
			ClearUAVShader_T<EClearReplacementResourceType::Texture2D, EClearReplacementValueType::Float, 4, false>(RHICmdList, UnorderedAccessViewRHI, Size.X, Size.Y, Size.Z, *reinterpret_cast<const float(*)[4]>(&Values));
		}
	}
	else
	{
		if (UAV->UnrealFormat != 0)
		{
			// Typed buffer
			ClearUAVShader_T<EClearReplacementResourceType::Buffer, EClearReplacementValueType::Float, 4, false>(
				RHICmdList
				, UnorderedAccessViewRHI
				, UAV->GetBufferSize() / GPixelFormats[UAV->UnrealFormat].BlockBytes
				, 1
				, 1
				, *reinterpret_cast<const float(*)[4]>(&Values)
			);
		}
		else
		{
			// Storage buffer
			ClearUAVShader_T<EClearReplacementResourceType::StructuredBuffer, EClearReplacementValueType::Float, 1, false>(
				RHICmdList
				, UnorderedAccessViewRHI
				, UAV->GetBufferSize() / sizeof(float)
				, 1
				, 1
				, *reinterpret_cast<const float(*)[1]>(&Values)
			);
		}
	}
}

void FOpenGLDynamicRHI::RHIClearUAVUint(FRHIUnorderedAccessView* UnorderedAccessViewRHI, const FUintVector4& Values)
{
	FOpenGLUnorderedAccessView* UAV = ResourceCast(UnorderedAccessViewRHI);
	
	TRHICommandList_RecursiveHazardous<FOpenGLDynamicRHI> RHICmdList(this);

	if (UAV->IsTexture())
	{
		FIntVector Size = UAV->GetTexture()->GetSizeXYZ();
		
		if (UAV->IsLayered())
		{
			ClearUAVShader_T<EClearReplacementResourceType::Texture3D, EClearReplacementValueType::Uint32, 4, false>(RHICmdList, UnorderedAccessViewRHI, Size.X, Size.Y, Size.Z, *reinterpret_cast<const uint32(*)[4]>(&Values));
		}
		else
		{
			ClearUAVShader_T<EClearReplacementResourceType::Texture2D, EClearReplacementValueType::Uint32, 4, false>(RHICmdList, UnorderedAccessViewRHI, Size.X, Size.Y, Size.Z, *reinterpret_cast<const uint32(*)[4]>(&Values));
		}
	}
	else
	{
		if (UAV->UnrealFormat != 0)
		{
			// Typed buffer
			ClearUAVShader_T<EClearReplacementResourceType::Buffer, EClearReplacementValueType::Uint32, 4, false>(
				RHICmdList
				, UnorderedAccessViewRHI
				, UAV->GetBufferSize() / GPixelFormats[UAV->UnrealFormat].BlockBytes
				, 1
				, 1
				, *reinterpret_cast<const uint32(*)[4]>(&Values)
			);
		}
		else
		{
			// Storage buffer
			ClearUAVShader_T<EClearReplacementResourceType::StructuredBuffer, EClearReplacementValueType::Uint32, 1, false>(
				RHICmdList
				, UnorderedAccessViewRHI
				, UAV->GetBufferSize() / sizeof(uint32)
				, 1
				, 1
				, *reinterpret_cast<const uint32(*)[1]>(&Values)
			);
		}
	}
}

FShaderResourceViewRHIRef FOpenGLDynamicRHI::RHICreateShaderResourceView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	return new FOpenGLShaderResourceView(RHICmdList, Resource, ViewDesc);
}

FUnorderedAccessViewRHIRef FOpenGLDynamicRHI::RHICreateUnorderedAccessView(class FRHICommandListBase& RHICmdList, FRHIViewableResource* Resource, FRHIViewDesc const& ViewDesc)
{
	return new FOpenGLUnorderedAccessView(RHICmdList, Resource, ViewDesc);
}
