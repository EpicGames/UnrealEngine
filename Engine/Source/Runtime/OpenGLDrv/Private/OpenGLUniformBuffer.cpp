// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLUniformBuffer.cpp: OpenGL Uniform buffer RHI implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "RHI.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "Misc/ScopeLock.h"
#include "ShaderParameterStruct.h"

namespace OpenGLConsoleVariables
{
#if (PLATFORM_WINDOWS)
	int32 RequestedUBOPoolSize = 1024*1024*16;
#else
	int32 RequestedUBOPoolSize = 0;
#endif

	static FAutoConsoleVariableRef CVarUBOPoolSize(
		TEXT("OpenGL.UBOPoolSize"),
		RequestedUBOPoolSize,
		TEXT("Size of the UBO pool, 0 disables UBO Pool"),
		ECVF_ReadOnly
		);

#if PLATFORM_ANDROID
		int32 bUBODirectWrite = 0;
#else
		int32 bUBODirectWrite = 1;
#endif

	static FAutoConsoleVariableRef CVarUBODirectWrite(
		TEXT("OpenGL.UBODirectWrite"),
		bUBODirectWrite,
		TEXT("Enables direct writes to the UBO via Buffer Storage"),
		ECVF_ReadOnly
		);
};

#define NUM_POOL_BUCKETS 45

#define NUM_SAFE_FRAMES 3

static const uint32 RequestedUniformBufferSizeBuckets[NUM_POOL_BUCKETS] = {
	16,32,48,64,80,96,112,128,	// 16-byte increments
	160,192,224,256,			// 32-byte increments
	320,384,448,512,			// 64-byte increments
	640,768,896,1024,			// 128-byte increments
	1280,1536,1792,2048,		// 256-byte increments
	2560,3072,3584,4096,		// 512-byte increments
	5120,6144,7168,8192,		// 1024-byte increments
	10240,12288,14336,16384,	// 2048-byte increments
	20480,24576,28672,32768,	// 4096-byte increments
	40960,49152,57344,65536,	// 8192-byte increments

	// 65536 is current max uniform buffer size for Mac OS X.

	0xFFFF0000 // Not max uint32 to allow rounding
};

// Maps desired size buckets to aligment actually 
static TArray<uint32> UniformBufferSizeBuckets;

static FCriticalSection GGLUniformBufferPoolCS;


static inline bool IsSuballocatingUBOs()
{
#if SUBALLOCATED_CONSTANT_BUFFER
	if (!GUseEmulatedUniformBuffers)
	{
		return OpenGLConsoleVariables::RequestedUBOPoolSize != 0;
	}
#endif
	return false;
}

static inline uint32 GetUBOPoolSize()
{
	static uint32 UBOPoolSize = 0xFFFFFFFF;

	if ( UBOPoolSize == 0xFFFFFFFF )
	{
		GLint Alignment;
		glGetIntegerv( GL_UNIFORM_BUFFER_OFFSET_ALIGNMENT, &Alignment);

		UBOPoolSize = (( OpenGLConsoleVariables::RequestedUBOPoolSize + Alignment - 1) / Alignment ) * Alignment;
	}

	return UBOPoolSize;
}

// Convert bucket sizes to cbe compatible with present device
static void RemapBuckets()
{
	if (!IsSuballocatingUBOs())
	{
		for (int32 Count = 0; Count < NUM_POOL_BUCKETS; Count++)
		{
			UniformBufferSizeBuckets.Push(RequestedUniformBufferSizeBuckets[Count]);
		}
	}
	else
	{
		GLint Alignment;
		glGetIntegerv( GL_UNIFORM_BUFFER_OFFSET_ALIGNMENT, &Alignment);

		for (int32 Count = 0; Count < NUM_POOL_BUCKETS; Count++)
		{
			uint32 AlignedSize = ((RequestedUniformBufferSizeBuckets[Count] + Alignment - 1) / Alignment ) * Alignment;
			if (!UniformBufferSizeBuckets.Contains(AlignedSize))
			{
				UniformBufferSizeBuckets.Push(AlignedSize);
			}
		}
		UE_LOGF(LogRHI,Log,"Configured UBO bucket pool to %d buckets based on alignment of %d bytes", UniformBufferSizeBuckets.Num(), Alignment);
	}
}

static uint32 GetPoolBucketIndex(uint32 NumBytes)
{
	if (UniformBufferSizeBuckets.Num() == 0)
	{
		check(IsInRenderingThread()); // this better be set up before there is any concurrency.
		FScopeLock Lock(&GGLUniformBufferPoolCS);
		RemapBuckets();
	}

	check( UniformBufferSizeBuckets.Num() > 0);

	unsigned long lower = 0;
	unsigned long upper = UniformBufferSizeBuckets.Num();
	unsigned long middle;

	do
	{
		middle = ( upper + lower ) >> 1;
		if( NumBytes <= UniformBufferSizeBuckets[middle-1] )
		{
			upper = middle;
		}
		else
		{
			lower = middle;
		}
	}
	while( upper - lower > 1 );

	check( NumBytes <= UniformBufferSizeBuckets[lower] );
	check( (lower == 0 ) || ( NumBytes > UniformBufferSizeBuckets[lower-1] ) );

	return lower;
}

static inline uint32 GetPoolBucketSize(uint32 NumBytes)
{
	return UniformBufferSizeBuckets[GetPoolBucketIndex(NumBytes)];
}

static FCriticalSection GGLEmulatedUniformBufferDataFactoryCS;
struct FUniformBufferDataFactory
{
	FOpenGLEUniformBufferDataRef Create(uint32 Size, GLuint& OutResource)
	{
		FScopeLock Lock(&GGLEmulatedUniformBufferDataFactoryCS);
		static GLuint TempCounter = 0;
		OutResource = ++TempCounter;

		FOpenGLEUniformBufferDataRef Buffer = new FOpenGLEUniformBufferData(Size);
		Entries.Add(OutResource, Buffer);
		return Buffer;
	}

	FOpenGLEUniformBufferDataRef Get(GLuint Resource)
	{
		FScopeLock Lock(&GGLEmulatedUniformBufferDataFactoryCS);
		FOpenGLEUniformBufferDataRef* Buffer = Entries.Find(Resource);
		check(Buffer);
		return *Buffer;
	}

	void Destroy(GLuint Resource)
	{
		FScopeLock Lock(&GGLEmulatedUniformBufferDataFactoryCS);
		Entries.Remove(Resource);
	}
private:
	TMap<GLuint, FOpenGLEUniformBufferDataRef> Entries;
};

static FUniformBufferDataFactory UniformBufferDataFactory;

// Describes a uniform buffer in the free pool.
struct FPooledGLUniformBuffer
{
	GLuint Buffer;
	uint32 CreatedSize;
	uint32 Offset;
	uint32 FrameFreed;
	uint8* PersistentlyMappedBuffer;
};

// Pool of free uniform buffers, indexed by bucket for constant size search time.
static TArray<FPooledGLUniformBuffer> GLUniformBufferPool[NUM_POOL_BUCKETS][2];
static TArray<FPooledGLUniformBuffer> GLEmulatedUniformBufferPool[NUM_POOL_BUCKETS][2];

// Uniform buffers that have been freed more recently than NumSafeFrames ago.
static TArray<FPooledGLUniformBuffer> SafeGLUniformBufferPools[NUM_SAFE_FRAMES][NUM_POOL_BUCKETS][2];
static TArray<FPooledGLUniformBuffer> SafeGLEmulatedUniformBufferPools[NUM_SAFE_FRAMES][NUM_POOL_BUCKETS][2];

// Delete the uniform buffer's GL resource
static void ReleaseUniformBuffer(bool bEmulatedBufferData, GLuint Resource, uint32 AllocatedSize)
{
	VERIFY_GL_SCOPE();
	if (bEmulatedBufferData)
	{
		UniformBufferDataFactory.Destroy(Resource);
	}
	else
	{
		check(Resource);
		FOpenGL::DeleteBuffers(1, &Resource);
		check(Resource != 0);
	}
	OpenGLBufferStats::UpdateUniformBufferStats(AllocatedSize, false);
}

// Does per-frame global updating for the uniform buffer pool.
void BeginFrame_UniformBufferPoolCleanup()
{
	FScopeLock Lock(&GGLUniformBufferPoolCS);

	int32 NumToCleanThisFrame = 10;

	SCOPE_CYCLE_COUNTER(STAT_OpenGLUniformBufferCleanupTime);

	if (!IsSuballocatingUBOs())
	{
		// Clean a limited number of old entries to reduce hitching when leaving a large level
		for( int32 StreamedIndex = 0; StreamedIndex < 2; ++StreamedIndex)
		{
			for (int32 BucketIndex = 0; BucketIndex < UniformBufferSizeBuckets.Num(); BucketIndex++)
			{
				for (int32 EntryIndex = GLUniformBufferPool[BucketIndex][StreamedIndex].Num() - 1; EntryIndex >= 0; EntryIndex--)
				{
					FPooledGLUniformBuffer& PoolEntry = GLUniformBufferPool[BucketIndex][StreamedIndex][EntryIndex];

					check(PoolEntry.Buffer);

					// Clean entries that are unlikely to be reused
					if (GFrameNumberRenderThread - PoolEntry.FrameFreed > 30)
					{
						DEC_DWORD_STAT(STAT_OpenGLNumFreeUniformBuffers);
						DEC_MEMORY_STAT_BY(STAT_OpenGLFreeUniformBufferMemory, PoolEntry.CreatedSize);
						ReleaseUniformBuffer(false, PoolEntry.Buffer, PoolEntry.CreatedSize);
						GLUniformBufferPool[BucketIndex][StreamedIndex].RemoveAtSwap(EntryIndex);

						--NumToCleanThisFrame;
						if (NumToCleanThisFrame == 0)
						{
							break;
						}
					}
				}

				if (GUseEmulatedUniformBuffers && NumToCleanThisFrame != 0)
				{
					for (int32 EntryIndex = GLEmulatedUniformBufferPool[BucketIndex][StreamedIndex].Num() - 1; EntryIndex >= 0; EntryIndex--)
					{
						FPooledGLUniformBuffer& PoolEntry = GLEmulatedUniformBufferPool[BucketIndex][StreamedIndex][EntryIndex];

						check(PoolEntry.Buffer);

						// Clean entries that are unlikely to be reused
						if (GFrameNumberRenderThread - PoolEntry.FrameFreed > 30)
						{
							DEC_DWORD_STAT(STAT_OpenGLNumFreeUniformBuffers);
							DEC_MEMORY_STAT_BY(STAT_OpenGLFreeUniformBufferMemory, PoolEntry.CreatedSize);
							ReleaseUniformBuffer(true, PoolEntry.Buffer, PoolEntry.CreatedSize);
							GLEmulatedUniformBufferPool[BucketIndex][StreamedIndex].RemoveAtSwap(EntryIndex);

							--NumToCleanThisFrame;
							if (NumToCleanThisFrame == 0)
							{
								break;
							}
						}
					}
				}

				if (NumToCleanThisFrame == 0)
				{
					break;
				}
			}

			if (NumToCleanThisFrame == 0)
			{
				break;
			}
		}
	}

	// Index of the bucket that is now old enough to be reused
	const int32 SafeFrameIndex = GFrameNumberRenderThread % NUM_SAFE_FRAMES;

	// Merge the bucket into the free pool array
	for( int32 StreamedIndex = 0; StreamedIndex < 2; ++StreamedIndex)
	{
		for (int32 BucketIndex = 0; BucketIndex < UniformBufferSizeBuckets.Num(); BucketIndex++)
		{
			GLUniformBufferPool[BucketIndex][StreamedIndex].Append(SafeGLUniformBufferPools[SafeFrameIndex][BucketIndex][StreamedIndex]);
			SafeGLUniformBufferPools[SafeFrameIndex][BucketIndex][StreamedIndex].Reset();

			if (GUseEmulatedUniformBuffers)
			{
				GLEmulatedUniformBufferPool[BucketIndex][StreamedIndex].Append(SafeGLEmulatedUniformBufferPools[SafeFrameIndex][BucketIndex][StreamedIndex]);
				SafeGLEmulatedUniformBufferPools[SafeFrameIndex][BucketIndex][StreamedIndex].Reset();
			}
		}
	}
}

static bool IsPoolingEnabled()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.UniformBufferPooling"));
	int32 CVarValue = IsInParallelRenderingThread() ? CVar->GetValueOnRenderThread() : CVar->GetValueOnGameThread();
	return CVarValue != 0;
};

struct TUBOPoolBuffer
{
	GLuint Resource;
	uint32 ConsumedSpace;
	uint32 AllocatedSpace;
	uint8* Pointer;
};

TArray<TUBOPoolBuffer> UBOPool;

static void SuballocateUBO( uint32 Size, GLuint& Resource, uint32& Offset, uint8*& Pointer)
{
	VERIFY_GL_SCOPE();

	check( Size <= GetUBOPoolSize());
	// Find space in previously allocated pool buffers
	for ( int32 Buffer = 0; Buffer < UBOPool.Num(); Buffer++)
	{
		TUBOPoolBuffer &Pool = UBOPool[Buffer];
		if ( Size < (Pool.AllocatedSpace - Pool.ConsumedSpace))
		{
			Resource = Pool.Resource;
			Offset = Pool.ConsumedSpace;
			Pointer = Pool.Pointer ? Pool.Pointer + Offset : 0;
			Pool.ConsumedSpace += Size;
			return;
		}
	}

	// No space was found to use, create a new Pool buffer
	TUBOPoolBuffer Pool;

	FOpenGL::GenBuffers( 1, &Pool.Resource);
	
	FOpenGLDynamicRHI::Get().CachedBindUniformBuffer(Pool.Resource);

	if (FOpenGL::SupportsBufferStorage() && OpenGLConsoleVariables::bUBODirectWrite)
	{
		FOpenGL::BufferStorage( GL_UNIFORM_BUFFER, GetUBOPoolSize(), NULL, GL_MAP_WRITE_BIT | GL_MAP_PERSISTENT_BIT | GL_MAP_COHERENT_BIT );
		Pool.Pointer = (uint8*)FOpenGL::MapBufferRange(GL_UNIFORM_BUFFER, 0, GetUBOPoolSize(), FOpenGL::EResourceLockMode::RLM_WriteOnlyPersistent);
	}
	else
	{
		glBufferData( GL_UNIFORM_BUFFER, GetUBOPoolSize(), 0, GL_DYNAMIC_DRAW);
		Pool.Pointer = 0;
	}

	Pointer = Pool.Pointer;
	
	INC_MEMORY_STAT_BY(STAT_OpenGLFreeUniformBufferMemory, GetUBOPoolSize());
	
	Pool.ConsumedSpace = Size;
	Pool.AllocatedSpace = GetUBOPoolSize();
	
	Resource = Pool.Resource;
	Offset = 0;
	
	UBOPool.Push(Pool);

	UE_LOGF(LogRHI,Log,"Allocated new buffer for uniform Pool %d buffers with %d bytes",UBOPool.Num(), UBOPool.Num()*GetUBOPoolSize());
}

static uint32 UniqueUniformBufferID()
{
	// make it atomic?
	static uint32 GUniqueUniformBufferID = 0;
	return ++GUniqueUniformBufferID;
}

FOpenGLUniformBuffer::FOpenGLUniformBuffer(const FRHIUniformBufferLayout* InLayout)
	: FRHIUniformBuffer(InLayout)
	, Resource(0)
	, Offset(0)
	, RangeSize(0)
	, PersistentlyMappedBuffer(nullptr)
	, UniqueID(UniqueUniformBufferID())
	, AllocatedSize(0)
	, bStreamDraw(false)
	, bOwnsResource(true)
{
	bIsEmulatedUniformBuffer = GUseEmulatedUniformBuffers && !EnumHasAnyFlags(InLayout->Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer|ERHIUniformBufferFlags::UniformView);
	RangeSize = InLayout->ConstantBufferSize;
}

void FOpenGLUniformBuffer::SetGLUniformBufferParams(GLuint InResource, uint32 InOffset, uint8* InPersistentlyMappedBuffer, uint32 InAllocatedSize, FOpenGLEUniformBufferDataRef InEmulatedBuffer, bool bInStreamDraw)
{
	Resource = InResource;
	Offset = InOffset;
	PersistentlyMappedBuffer = InPersistentlyMappedBuffer;
	EmulatedBufferData = InEmulatedBuffer;
	AllocatedSize = InAllocatedSize;
	bStreamDraw = bInStreamDraw;
	bOwnsResource = true;

	LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelAlloc(ELLMTracker::Default, ((uint8*)this)+1, InAllocatedSize)); //+1 because ptr must be unique for LLM
}

FOpenGLUniformBuffer::~FOpenGLUniformBuffer()
{
	VERIFY_GL_SCOPE();

	if (Resource != 0 && bOwnsResource)
	{
		if (IsPoolingEnabled())
		{
			FPooledGLUniformBuffer NewEntry;
			NewEntry.Buffer = Resource;
			NewEntry.Offset = Offset;
			NewEntry.FrameFreed = GFrameNumberRenderThread;
			NewEntry.CreatedSize = AllocatedSize;
			NewEntry.PersistentlyMappedBuffer = PersistentlyMappedBuffer;

			int StreamedIndex = bStreamDraw ? 1 : 0;

			// Add to this frame's array of free uniform buffers
			const int32 SafeFrameIndex = GFrameNumberRenderThread % NUM_SAFE_FRAMES;
			const uint32 BucketIndex = GetPoolBucketIndex(AllocatedSize);

			check(AllocatedSize == UniformBufferSizeBuckets[BucketIndex]);
			// this might fail with sizes > 65536; handle it then by extending the range? sizes > 65536 are presently unsupported on Mac OS X.

			FScopeLock Lock(&GGLUniformBufferPoolCS);
			
			if (GUseEmulatedUniformBuffers && !EnumHasAnyFlags(GetLayout().Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer))
			{
				SafeGLEmulatedUniformBufferPools[SafeFrameIndex][BucketIndex][StreamedIndex].Add(NewEntry);
			}
			else
			{
				SafeGLUniformBufferPools[SafeFrameIndex][BucketIndex][StreamedIndex].Add(NewEntry);
			}
			INC_DWORD_STAT(STAT_OpenGLNumFreeUniformBuffers);
			INC_MEMORY_STAT_BY(STAT_OpenGLFreeUniformBufferMemory, AllocatedSize);
		}
		else
		{
			ReleaseUniformBuffer(IsValidRef(EmulatedBufferData), Resource, AllocatedSize);
			Resource = 0; 
		}

		LLM_IF_ENABLED(FLowLevelMemTracker::Get().OnLowLevelFree(ELLMTracker::Default, ((uint8*)this)+1)); //+1 because ptr must be unique for LLM
	}
}


void FOpenGLUniformBuffer::SetLayoutTable(const void* Contents, EUniformBufferValidation Validation)
{
	if (GetLayout().Resources.Num())
	{
		int32 NumResources = GetLayout().Resources.Num();
		ResourceTable.Empty(NumResources);
		ResourceTable.AddZeroed(NumResources);

		if (Contents)
		{
			for (int32 Index = 0; Index < NumResources; ++Index)
			{
				ResourceTable[Index] = GetShaderParameterResourceRHI(Contents, GetLayout().Resources[Index].MemberOffset, GetLayout().Resources[Index].MemberType);
			}
		}
	}
}


void CopyDataToUniformBuffer(FRHICommandListImmediate& RHICmdList, FOpenGLUniformBuffer* NewUniformBuffer, const void* Contents, uint32 ContentSize)
{
	FOpenGLEUniformBufferDataRef EmulatedUniformDataRef = NewUniformBuffer->EmulatedBufferData;	
	uint8* PersistentlyMappedBuffer = NewUniformBuffer->PersistentlyMappedBuffer;

	// Copy the contents of the uniform buffer.
	if (IsValidRef(EmulatedUniformDataRef))
	{
		FMemory::Memcpy(EmulatedUniformDataRef->Data.GetData(), Contents, ContentSize);
	}
	else if (PersistentlyMappedBuffer)
	{
		FMemory::Memcpy(PersistentlyMappedBuffer, Contents, ContentSize);
	}
	else
	{
		if (RHICmdList.IsTopOfPipe())
		{
			// Copy the data to the command list since we'll be deferring the buffer init.
			void* ConstantBufferCopy = RHICmdList.Alloc(ContentSize, 16);
			FMemory::Memcpy(ConstantBufferCopy, Contents, ContentSize);

			Contents = ConstantBufferCopy;
		}

		RHICmdList.EnqueueLambda([ContentSize, Contents](FRHICommandListImmediate&)
		{
			VERIFY_GL_SCOPE();
			FOpenGL::BufferSubData(GL_UNIFORM_BUFFER, 0, ContentSize, Contents);
		});
	}
}

static FUniformBufferRHIRef CreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation)
{
	// This should really be synchronized, if there's a chance it'll be used from more than one buffer. Luckily, uniform buffers
	// are only used for drawing/shader usage, not for loading resources or framebuffer blitting, so no synchronization primitives for now.

	// Explicitly check that the size is nonzero before allowing CreateBuffer to opaquely fail.
	check(Layout->Resources.Num() > 0 || Layout->ConstantBufferSize > 0);

	FOpenGLUniformBuffer* NewUniformBuffer = new FOpenGLUniformBuffer(Layout);

	const uint32 BucketIndex = GetPoolBucketIndex(Layout->ConstantBufferSize);
	const uint32 SizeOfBufferToAllocate = UniformBufferSizeBuckets[BucketIndex];
	const uint32 AllocatedSize = (SizeOfBufferToAllocate > 0) ? SizeOfBufferToAllocate : Layout->ConstantBufferSize;
	
	// EmulatedUniformDataRef will not be initialized on RHI thread. safe to use on RT thread.
	FOpenGLEUniformBufferDataRef EmulatedUniformDataRef;

	// PersistentlyMappedBuffer initializes via IsSuballocatingUBOs path which will flush RHI commands. safe to use on RT thread.
	uint8* PersistentlyMappedBuffer = NULL;

	bool bUseEmulatedUBs = GUseEmulatedUniformBuffers && !EnumHasAnyFlags(Layout->Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer);

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	{
		const bool bStreamDraw = (Usage == UniformBuffer_SingleDraw || Usage == UniformBuffer_SingleFrame);
		
		// Nothing usable was found in the free pool, or we're not pooling, so create a new uniform buffer
		if (bUseEmulatedUBs)
		{
			GLuint AllocatedResource = 0;
			uint32 OffsetInBuffer = 0;
			EmulatedUniformDataRef = UniformBufferDataFactory.Create(AllocatedSize, AllocatedResource);
			NewUniformBuffer->SetGLUniformBufferParams(AllocatedResource, OffsetInBuffer, PersistentlyMappedBuffer, AllocatedSize, EmulatedUniformDataRef, bStreamDraw);
		}
		else if (IsSuballocatingUBOs())
		{
			RHICmdList.EnqueueLambda([NewUniformBuffer, AllocatedSize, &PersistentlyMappedBuffer, &EmulatedUniformDataRef, bStreamDraw](FRHICommandListImmediate&)
			{
				GLuint AllocatedResource = 0;
				uint32 OffsetInBuffer = 0;

				SuballocateUBO(AllocatedSize, AllocatedResource, OffsetInBuffer, PersistentlyMappedBuffer);
				NewUniformBuffer->SetGLUniformBufferParams(AllocatedResource, OffsetInBuffer, PersistentlyMappedBuffer, AllocatedSize, EmulatedUniformDataRef, bStreamDraw);
			});
		}
		else
		{
			check(PersistentlyMappedBuffer == nullptr);
			RHICmdList.EnqueueLambda([NewUniformBuffer, AllocatedSize, PersistentlyMappedBuffer, EmulatedUniformDataRef, bStreamDraw](FRHICommandListImmediate&)
			{
				VERIFY_GL_SCOPE();
				GLuint AllocatedResource = 0;
				uint32 OffsetInBuffer = 0;
				FOpenGL::GenBuffers(1, &AllocatedResource);
				FOpenGLDynamicRHI::Get().CachedBindUniformBuffer(AllocatedResource);
				glBufferData(GL_UNIFORM_BUFFER, AllocatedSize, NULL, bStreamDraw ? GL_STREAM_DRAW : GL_STATIC_DRAW);
				NewUniformBuffer->SetGLUniformBufferParams(AllocatedResource, OffsetInBuffer, nullptr, AllocatedSize, EmulatedUniformDataRef, bStreamDraw);
			});
		}
	}

	if(!bUseEmulatedUBs)
	{
		// flush for the UBO case
		// as this path interacts with UBOPool, this hasnt been addressed for the RHI thread case.
		if (RHICmdList.IsTopOfPipe() && IsSuballocatingUBOs())
		{
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			RHITHREAD_GLTRACE_BLOCKING;
		}
	}

	OpenGLBufferStats::UpdateUniformBufferStats(AllocatedSize, true);

	check(!bUseEmulatedUBs || (IsValidRef(EmulatedUniformDataRef) && (EmulatedUniformDataRef->Data.Num() * EmulatedUniformDataRef->Data.GetTypeSize() == AllocatedSize)));

	if (Contents)
	{
		CopyDataToUniformBuffer(RHICmdList, NewUniformBuffer, Contents, Layout->ConstantBufferSize);
	}

	// Initialize the resource table for this uniform buffer.
	NewUniformBuffer->SetLayoutTable(Contents, Validation);

	return NewUniformBuffer;
}

static FOpenGLUniformBuffer* CreateUniformBufferView(FRHICommandListImmediate& RHICmdList, const FRHIUniformBufferLayout* Layout, const void* Contents)
{
	FOpenGLUniformBuffer* UniformBufferView = nullptr;
	
	if (EnumHasAnyFlags(Layout->Flags, ERHIUniformBufferFlags::UniformView))
	{
		UniformBufferView = new FOpenGLUniformBuffer(Layout);
		UniformBufferView->SetLayoutTable(Contents, EUniformBufferValidation::None);

		FRHIShaderResourceView* UniformViewSRV = nullptr;
		for (int32 Index = 0; Index < Layout->Resources.Num() && !UniformViewSRV; ++Index)
		{
			EUniformBufferBaseType ResourceBaseType = Layout->Resources[Index].MemberType;
			if (ResourceBaseType == UBMT_SRV || 
				ResourceBaseType == UBMT_RDG_BUFFER_SRV)
			{
				UniformViewSRV = (FRHIShaderResourceView*)GetShaderParameterResourceRHI(Contents, Layout->Resources[Index].MemberOffset, ResourceBaseType);
			}
		}
		
		check(UniformViewSRV);

		RHICmdList.EnqueueLambda([UniformBufferView, UniformViewSRV](FRHICommandListImmediate&)
		{
			VERIFY_GL_SCOPE();
			
			FOpenGLBuffer* UBO = FOpenGLDynamicRHI::ResourceCast(UniformViewSRV->GetBuffer());
			const FRHIViewDesc::FBufferSRV& SRVInfo = UniformViewSRV->GetDesc().Buffer.SRV;
			
			check(UBO->Resource);
			check(UBO->GetSize() >= PLATFORM_MAX_UNIFORM_BUFFER_RANGE);

			UniformBufferView->Resource = UBO->Resource;
			UniformBufferView->AllocatedSize = UBO->GetSize();
			UniformBufferView->bOwnsResource = false;
			UniformBufferView->Offset = SRVInfo.OffsetInBytes;
			UniformBufferView->RangeSize = PLATFORM_MAX_UNIFORM_BUFFER_RANGE;
			UniformBufferView->PersistentlyMappedBuffer = nullptr;
			UniformBufferView->EmulatedBufferData = nullptr;
			UniformBufferView->bStreamDraw = false;
		});
	}
	
	return UniformBufferView;
}

FUniformBufferRHIRef FOpenGLDynamicRHI::RHICreateUniformBuffer(const void* Contents, const FRHIUniformBufferLayout* Layout, EUniformBufferUsage Usage, EUniformBufferValidation Validation)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	// This should really be synchronized, if there's a chance it'll be used from more than one buffer. Luckily, uniform buffers
	// are only used for drawing/shader usage, not for loading resources or framebuffer blitting, so no synchronization primitives for now.

	// Explicitly check that the size is nonzero before allowing CreateBuffer to opaquely fail.
	check(Layout->Resources.Num() > 0 || Layout->ConstantBufferSize > 0);
	
	if (Contents && Validation == EUniformBufferValidation::ValidateResources)
	{
		ValidateShaderParameterResourcesRHI(Contents, *Layout);
	}

	// 
	FOpenGLUniformBuffer* UniformBufferView = CreateUniformBufferView(RHICmdList, Layout, Contents);
	if (UniformBufferView)
	{
		return UniformBufferView;
	}

	bool bUseEmulatedUBs = GUseEmulatedUniformBuffers && !EnumHasAnyFlags(Layout->Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer);

	bool bStreamDraw = (Usage == UniformBuffer_SingleDraw || Usage == UniformBuffer_SingleFrame);
	GLuint AllocatedResource = 0;
	uint32 OffsetInBuffer = 0;
	uint8* PersistentlyMappedBuffer = NULL;
	uint32 AllocatedSize = 0;
	FOpenGLEUniformBufferDataRef EmulatedUniformDataRef;

	// If the uniform buffer contains constants, allocate a uniform buffer resource from GL.
	if (Layout->ConstantBufferSize > 0)
	{
		uint32 SizeOfBufferToAllocate = 0;
		if (IsPoolingEnabled())
		{
			// Find the appropriate bucket based on size
			const uint32 BucketIndex = GetPoolBucketIndex(Layout->ConstantBufferSize);
			int StreamedIndex = bStreamDraw ? 1 : 0;

			FPooledGLUniformBuffer FreeBufferEntry;
			FreeBufferEntry.Buffer = 0;
			FreeBufferEntry.CreatedSize = 0;
			bool bHasEntry = false;
			{
				FScopeLock Lock(&GGLUniformBufferPoolCS);

				TArray<FPooledGLUniformBuffer>* PoolBucket;

				if (bUseEmulatedUBs)
				{
					PoolBucket = &GLEmulatedUniformBufferPool[BucketIndex][StreamedIndex];
				}
				else
				{
					PoolBucket = &GLUniformBufferPool[BucketIndex][StreamedIndex];
				}

				if (PoolBucket->Num() > 0)
				{
					// Reuse the last entry in this size bucket
					FreeBufferEntry = PoolBucket->Pop();
					bHasEntry = true;
				}
			}
			if (bHasEntry)
			{
				DEC_DWORD_STAT(STAT_OpenGLNumFreeUniformBuffers);
				DEC_MEMORY_STAT_BY(STAT_OpenGLFreeUniformBufferMemory, FreeBufferEntry.CreatedSize);

				AllocatedResource = FreeBufferEntry.Buffer;
				AllocatedSize = FreeBufferEntry.CreatedSize;

				if (bUseEmulatedUBs)
				{
					EmulatedUniformDataRef = UniformBufferDataFactory.Get(AllocatedResource);
				}
				else
				{
					RHICmdList.EnqueueLambda([AllocatedResource](FRHICommandListImmediate&)
					{
						VERIFY_GL_SCOPE();
						FOpenGLDynamicRHI::Get().CachedBindUniformBuffer(AllocatedResource);
					});
				}
			}
			else
			{
				SizeOfBufferToAllocate = UniformBufferSizeBuckets[BucketIndex];
			}
		}
	}

	if (AllocatedSize == 0)
	{
		return CreateUniformBuffer(Contents, Layout, Usage, Validation);
	}

	FOpenGLUniformBuffer* NewUniformBuffer = new FOpenGLUniformBuffer(Layout);
	NewUniformBuffer->SetGLUniformBufferParams(AllocatedResource, OffsetInBuffer, PersistentlyMappedBuffer, AllocatedSize, EmulatedUniformDataRef, bStreamDraw);

	check(!bUseEmulatedUBs || (IsValidRef(EmulatedUniformDataRef) && (EmulatedUniformDataRef->Data.Num() * EmulatedUniformDataRef->Data.GetTypeSize() == AllocatedSize)));
	if (Contents)
	{
		CopyDataToUniformBuffer(RHICmdList, NewUniformBuffer, Contents, Layout->ConstantBufferSize);
	}

	// Initialize the resource table for this uniform buffer.
	NewUniformBuffer->SetLayoutTable(Contents, Validation);

	return NewUniformBuffer;
}

void UpdateUniformBufferContents(FOpenGLUniformBuffer* UniformBuffer, const void* Contents, uint32 ConstantBufferSize)
{
	if (ConstantBufferSize > 0)
	{
		FOpenGLEUniformBufferDataRef EmulatedUniformDataRef = UniformBuffer->EmulatedBufferData;
		uint8* PersistentlyMappedBuffer = UniformBuffer->PersistentlyMappedBuffer;

		if (IsValidRef(EmulatedUniformDataRef))
		{
			FMemory::Memcpy(EmulatedUniformDataRef->Data.GetData(), Contents, ConstantBufferSize);
		}
		else if (PersistentlyMappedBuffer)
		{
			UE_LOGF(LogRHI, Fatal, "RHIUpdateUniformBuffer doesn't support PersistentlyMappedBuffer yet!");
		}
		else
		{
			FOpenGLDynamicRHI::Get().CachedBindUniformBuffer(UniformBuffer->Resource);
			FOpenGL::BufferSubData(GL_UNIFORM_BUFFER, 0, ConstantBufferSize, Contents);
		}
	}
}

void FOpenGLDynamicRHI::RHIUpdateUniformBuffer(FRHICommandListBase& RHICmdList, FRHIUniformBuffer* UniformBufferRHI, const void* Contents)
{
	FOpenGLUniformBuffer* UniformBuffer = ResourceCast(UniformBufferRHI);

	const FRHIUniformBufferLayout& Layout = UniformBufferRHI->GetLayout();
	ValidateShaderParameterResourcesRHI(Contents, Layout);

	const int32 ConstantBufferSize = Layout.ConstantBufferSize;
	const int32 NumResources = Layout.Resources.Num();

	check(UniformBuffer->GetResourceTable().Num() == NumResources);

	uint32 NextUniqueID = UniqueUniformBufferID();

	if (RHICmdList.Bypass())
	{
		UpdateUniformBufferContents(UniformBuffer, Contents, ConstantBufferSize);

		for (int32 Index = 0; Index < NumResources; ++Index)
		{
			UniformBuffer->GetResourceTable()[Index] = GetShaderParameterResourceRHI(Contents, Layout.Resources[Index].MemberOffset, Layout.Resources[Index].MemberType);
		}
		
		UniformBuffer->UniqueID = NextUniqueID;
	}
	else
	{
		FRHIResource** CmdListResources = nullptr;
		void* CmdListConstantBufferData = nullptr;

		if (NumResources > 0)
		{
			CmdListResources = (FRHIResource**)RHICmdList.Alloc(sizeof(FRHIResource*) * NumResources, alignof(FRHIResource*));

			for (int32 Index = 0; Index < NumResources; ++Index)
			{
				const auto Parameter = Layout.Resources[Index];
				CmdListResources[Index] = GetShaderParameterResourceRHI(Contents, Parameter.MemberOffset, Parameter.MemberType);
			}
		}

		if (ConstantBufferSize > 0)
		{
			CmdListConstantBufferData = (void*)RHICmdList.Alloc(ConstantBufferSize, 16);
			FMemory::Memcpy(CmdListConstantBufferData, Contents, ConstantBufferSize);
		}

		RHICmdList.EnqueueLambda([UniformBuffer, CmdListResources, NumResources, CmdListConstantBufferData, ConstantBufferSize, NextUniqueID](FRHICommandListBase&)
		{
			UpdateUniformBufferContents(UniformBuffer, CmdListConstantBufferData, ConstantBufferSize);

			// Update resource table.
			for (int32 ResourceIndex = 0; ResourceIndex < NumResources; ++ResourceIndex)
			{
				UniformBuffer->GetResourceTable()[ResourceIndex] = CmdListResources[ResourceIndex];
			}
			UniformBuffer->UniqueID = NextUniqueID;
		});
		RHICmdList.RHIThreadFence(true);
	}
}
