// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLUtil.h: OpenGL RHI utility implementation.
=============================================================================*/

#include "OpenGLUtil.h"
#include "Stats/Stats.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#include "RHICoreStats.h"

void SetOpenGLResourceName(FOpenGLTexture* Texture, const ANSICHAR* Name)
{
#if GLDEBUG_LABELS_ENABLED
	VERIFY_GL_SCOPE();

	if (Texture->IsEvicted())
	{
		Texture->EvictionParamsPtr->SetDebugLabelName(Name);
	}
	else
	{
		FOpenGL::LabelObject(GL_TEXTURE, Texture->GetResource(), Name);
	}
#endif
}

void SetOpenGLResourceName(FOpenGLTexture* Texture, const TCHAR* Name)
{
#if GLDEBUG_LABELS_ENABLED
	const FAnsiString AnsiName(Name);
	SetOpenGLResourceName(Texture, *AnsiName);
#endif
}

#if ENABLE_DEBUG_OUTPUT

// Enable GL debug output
static int32 GOGLDebugOutputLevel = -1;

bool IsOGLDebugOutputEnabled()
{
	return GetOGLDebugOutputLevel() != 0;
}

int32 GetOGLDebugOutputLevel()
{
	if (GOGLDebugOutputLevel < 0)
	{
		// this can happen super early
		if(FCommandLine::IsInitialized())
		{
			int32 DebugLevel = 0;
			GOGLDebugOutputLevel = FParse::Value(FCommandLine::Get(), TEXT("OpenGLDebugLevel="), DebugLevel) ? DebugLevel : 0;

			if(FParse::Param(FCommandLine::Get(), TEXT("openglDebug")))
			{
				GOGLDebugOutputLevel = 1;
			}
		}
		else
		{
			return 0;
		}
	}

	return GOGLDebugOutputLevel;
}
#endif


void VerifyOpenGLResult(GLenum ErrorCode, const TCHAR* Msg1, const TCHAR* Msg2, const TCHAR* Filename, uint32 Line)
{
	if (ErrorCode != GL_NO_ERROR)
	{
		static const TCHAR* ErrorStrings[] =
		{
			TEXT("GL_INVALID_ENUM"),
			TEXT("GL_INVALID_VALUE"),
			TEXT("GL_INVALID_OPERATION"),
			TEXT("GL_STACK_OVERFLOW"),
			TEXT("GL_STACK_UNDERFLOW"),
			TEXT("GL_OUT_OF_MEMORY"),
			TEXT("GL_INVALID_FRAMEBUFFER_OPERATION_EXT"),
			TEXT("UNKNOWN ERROR")
		};

		uint32 ErrorIndex = FMath::Min<uint32>(ErrorCode - GL_INVALID_ENUM, UE_ARRAY_COUNT(ErrorStrings) - 1);
		UE_LOGF(LogRHI,Warning,"%ls(%u): %ls%ls failed with error %ls (0x%x)",
			Filename,Line,Msg1,Msg2,ErrorStrings[ErrorIndex],ErrorCode);
	}
}

GLenum GetOpenGLCubeFace(ECubeFace Face)
{
	switch (Face)
	{
	case CubeFace_PosX:
	default:
		return GL_TEXTURE_CUBE_MAP_POSITIVE_X;
	case CubeFace_NegX:
		return GL_TEXTURE_CUBE_MAP_NEGATIVE_X;
	case CubeFace_PosY:
		return GL_TEXTURE_CUBE_MAP_POSITIVE_Y;
	case CubeFace_NegY:
		return GL_TEXTURE_CUBE_MAP_NEGATIVE_Y;
	case CubeFace_PosZ:
		return GL_TEXTURE_CUBE_MAP_POSITIVE_Z;
	case CubeFace_NegZ:
		return GL_TEXTURE_CUBE_MAP_NEGATIVE_Z;
	};
}

//
// Stat declarations.
//


DEFINE_STAT(STAT_OpenGLPresentTime);
DEFINE_STAT(STAT_OpenGLCreateTextureTime);
DEFINE_STAT(STAT_OpenGLLockTextureTime);
DEFINE_STAT(STAT_OpenGLUnlockTextureTime);
DEFINE_STAT(STAT_OpenGLCopyTextureTime);
DEFINE_STAT(STAT_OpenGLCopyMipToMipAsyncTime);
DEFINE_STAT(STAT_OpenGLUploadTextureMipTime);
DEFINE_STAT(STAT_OpenGLCreateBoundShaderStateTime);
DEFINE_STAT(STAT_OpenGLConstantBufferUpdateTime);
DEFINE_STAT(STAT_OpenGLUniformCommitTime);
DEFINE_STAT(STAT_OpenGLShaderCompileTime);
DEFINE_STAT(STAT_OpenGLShaderCompileVerifyTime);
DEFINE_STAT(STAT_OpenGLShaderLinkTime);
DEFINE_STAT(STAT_OpenGLShaderLinkVerifyTime);
DEFINE_STAT(STAT_OpenGLShaderBindParameterTime);
DEFINE_STAT(STAT_OpenGLUniformBufferCleanupTime);
DEFINE_STAT(STAT_OpenGLEmulatedUniformBufferTime);
DEFINE_STAT(STAT_OpenGLFreeUniformBufferMemory);
DEFINE_STAT(STAT_OpenGLNumFreeUniformBuffers);
DEFINE_STAT(STAT_OpenGLShaderFirstDrawTime);
DEFINE_STAT(STAT_OpenGLProgramBinaryMemory);
DEFINE_STAT(STAT_OpenGLProgramCount);
DEFINE_STAT(STAT_OpenGLUseCachedProgramTime);
DEFINE_STAT(STAT_OpenGLCreateProgramFromBinaryTime);

DEFINE_STAT(STAT_OpenGLShaderLRUEvictTime);
DEFINE_STAT(STAT_OpenGLShaderLRUMissTime);
DEFINE_STAT(STAT_OpenGLShaderLRUProgramCount);
DEFINE_STAT(STAT_OpenGLShaderLRUEvictedProgramCount);
DEFINE_STAT(STAT_OpenGLShaderLRUMissCount);
DEFINE_STAT(STAT_OpenGLShaderLRUProgramMemory);
DEFINE_STAT(STAT_OpenGLShaderLRUProgramMemoryMapped);

#if OPENGLRHI_DETAILED_STATS
DEFINE_STAT(STAT_OpenGLDrawPrimitiveTime);
DEFINE_STAT(STAT_OpenGLDrawPrimitiveDriverTime);
DEFINE_STAT(STAT_OpenGLDrawPrimitiveUPTime);
DEFINE_STAT(STAT_OpenGLMapBufferTime);
DEFINE_STAT(STAT_OpenGLUnmapBufferTime);
DEFINE_STAT(STAT_OpenGLShaderBindTime);
DEFINE_STAT(STAT_OpenGLTextureBindTime);
DEFINE_STAT(STAT_OpenGLUniformBindTime);
DEFINE_STAT(STAT_OpenGLVBOSetupTime);
#endif


void OpenGLBufferStats::UpdateUniformBufferStats(int64 BufferSize, bool bAllocating)
{
	UE::RHICore::UpdateGlobalUniformBufferStats(BufferSize, bAllocating);
}

void OpenGLBufferStats::UpdateBufferStats(const FRHIBufferDesc& BufferDesc, bool bAllocating)
{
	UE::RHICore::UpdateGlobalBufferStats(BufferDesc, BufferDesc.Size, bAllocating);
}

