// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLVertexDeclaration.cpp: OpenGL vertex declaration RHI implementation.
=============================================================================*/

#include "Algo/StableSort.h"
#include "CoreMinimal.h"
#include "OpenGLDrv.h"

static FORCEINLINE void SetupGLElement(FOpenGLVertexElement& GLElement, GLenum Type, GLuint Size, bool bNormalized, bool bShouldConvertToFloat)
{
	GLElement.Type = Type;
	GLElement.Size = Size;
	GLElement.bNormalized = bNormalized;
	GLElement.bShouldConvertToFloat = bShouldConvertToFloat;
}

/**
 * Key used to look up vertex declarations in the cache.
 */
struct FOpenGLVertexDeclarationKey
{
	/** Vertex elements in the declaration. */
	FOpenGLVertexElements VertexElements;
	/** Hash of the vertex elements. */
	uint32 Hash;

	uint16 StreamStrides[MaxVertexElementCount];

	/** Initialization constructor. */
	explicit FOpenGLVertexDeclarationKey(const FVertexDeclarationElementList& InElements)
	{
		uint16 UsedStreamsMask = 0;
		FMemory::Memzero(StreamStrides);

		for(int32 ElementIndex = 0;ElementIndex < InElements.Num();ElementIndex++)
		{
			const FVertexElement& Element = InElements[ElementIndex];
			FOpenGLVertexElement GLElement;
			GLElement.StreamIndex = Element.StreamIndex;
			GLElement.Offset = Element.Offset;
			GLElement.Divisor = Element.bUseInstanceIndex ? 1 : 0;
			GLElement.AttributeIndex = Element.AttributeIndex;
			GLElement.HashStride = Element.Stride;
			GLElement.Padding = 0;
			switch(Element.Type)
			{
				case VET_Float1:		SetupGLElement(GLElement, GL_FLOAT,			1,			false,	true); break;
				case VET_Float2:		SetupGLElement(GLElement, GL_FLOAT,			2,			false,	true); break;
				case VET_Float3:		SetupGLElement(GLElement, GL_FLOAT,			3,			false,	true); break;
				case VET_Float4:		SetupGLElement(GLElement, GL_FLOAT,			4,			false,	true); break;
				case VET_PackedNormal:	SetupGLElement(GLElement, GL_BYTE,			4,			true,	true); break;
				case VET_UByte4:		SetupGLElement(GLElement, GL_UNSIGNED_BYTE,	4,			false,	false); break;
				case VET_UByte4N:		SetupGLElement(GLElement, GL_UNSIGNED_BYTE,	4,			true,	true); break;
				case VET_Color:	
					if (FOpenGL::SupportsVertexArrayBGRA())
					{
						SetupGLElement(GLElement, GL_UNSIGNED_BYTE,	GL_BGRA,	true,	true);
					}
					else
					{
						SetupGLElement(GLElement, GL_UNSIGNED_BYTE,	4,	true,	true);
					}
					break;
				case VET_Short2:		SetupGLElement(GLElement, GL_SHORT,			2,			false,	false); break;
				case VET_Short4:		SetupGLElement(GLElement, GL_SHORT,			4,			false,	false); break;
				case VET_Short2N:		SetupGLElement(GLElement, GL_SHORT,			2,			true,	true); break;
				case VET_Half2:			SetupGLElement(GLElement, GL_HALF_FLOAT,	2,			false,	true); break;
				case VET_Half4:			SetupGLElement(GLElement, GL_HALF_FLOAT,	4,			false,	true); break;
				case VET_Short4N:		SetupGLElement(GLElement, GL_SHORT,			4,			true,	true); break;
				case VET_UShort2:		SetupGLElement(GLElement, GL_UNSIGNED_SHORT, 2, false, false); break;
				case VET_UShort4:		SetupGLElement(GLElement, GL_UNSIGNED_SHORT, 4, false, false); break;
				case VET_UShort2N:		SetupGLElement(GLElement, GL_UNSIGNED_SHORT, 2, true, true); break;
				case VET_UShort4N:		SetupGLElement(GLElement, GL_UNSIGNED_SHORT, 4, true, true); break;
				case VET_URGB10A2N:		SetupGLElement(GLElement, GL_UNSIGNED_INT_2_10_10_10_REV, 4, true, true); break;
				case VET_UInt:			SetupGLElement(GLElement, GL_UNSIGNED_INT,			1,			false,	false); break;
				default: UE_LOGF(LogRHI, Fatal,"Unknown RHI vertex element type %u",(uint8)InElements[ElementIndex].Type);
			};

			if ((UsedStreamsMask & 1 << Element.StreamIndex) != 0)
			{
				ensure(StreamStrides[Element.StreamIndex] == Element.Stride);
			}
			else
			{
				UsedStreamsMask = UsedStreamsMask | (1 << Element.StreamIndex);
				StreamStrides[Element.StreamIndex] = Element.Stride;
			}

			VertexElements.Add(GLElement);
		}

		struct FCompareFOpenGLVertexElement
		{
			FORCEINLINE bool operator()( const FOpenGLVertexElement& A, const FOpenGLVertexElement& B ) const
			{
				if (A.StreamIndex < B.StreamIndex)
				{
					return true;
				}
				if (A.StreamIndex > B.StreamIndex)
				{
					return false;
				}
				if (A.Offset < B.Offset)
				{
					return true;
				}
				if (A.Offset > B.Offset)
				{
					return false;
				}
				if (A.AttributeIndex < B.AttributeIndex)
				{
					return true;
				}
				if (A.AttributeIndex > B.AttributeIndex)
				{
					return false;
				}
				return false;
			}
		};
		// Sort the FOpenGLVertexElements by stream then offset.
		Algo::StableSort( VertexElements, FCompareFOpenGLVertexElement() );

		Hash = FCrc::MemCrc_DEPRECATED(VertexElements.GetData(),VertexElements.Num()*sizeof(FOpenGLVertexElement));
		Hash = FCrc::MemCrc_DEPRECATED(StreamStrides, sizeof(StreamStrides), Hash);
	}
};

/** Hashes the array of OpenGL vertex element descriptions. */
uint32 GetTypeHash(const FOpenGLVertexDeclarationKey& Key)
{
	return Key.Hash;
}

/** Compare two vertex element descriptions. */
bool operator==(const FOpenGLVertexElement& A, const FOpenGLVertexElement& B)
{
	return A.Type == B.Type && A.StreamIndex == B.StreamIndex && A.Offset == B.Offset && A.Size == B.Size
		&& A.Divisor == B.Divisor && A.bNormalized == B.bNormalized && A.AttributeIndex == B.AttributeIndex
		&& A.bShouldConvertToFloat == B.bShouldConvertToFloat && A.HashStride == B.HashStride;
}

/** Compare two vertex declaration keys. */
bool operator==(const FOpenGLVertexDeclarationKey& A, const FOpenGLVertexDeclarationKey& B)
{
	return A.VertexElements == B.VertexElements;
}

/** Global cache of vertex declarations. */
TMap<FOpenGLVertexDeclarationKey,FVertexDeclarationRHIRef> GOpenGLVertexDeclarationCache;

FVertexDeclarationRHIRef FOpenGLDynamicRHI::RHICreateVertexDeclaration(const FVertexDeclarationElementList& Elements)
{
	// Construct a key from the elements.
	FOpenGLVertexDeclarationKey Key(Elements);

	// Check for a cached vertex declaration.
	FVertexDeclarationRHIRef* VertexDeclarationRefPtr = GOpenGLVertexDeclarationCache.Find(Key);
	if (VertexDeclarationRefPtr == NULL)
	{
		// Create and add to the cache if it doesn't exist.
		VertexDeclarationRefPtr = &GOpenGLVertexDeclarationCache.Add(Key,new FOpenGLVertexDeclaration(Key.VertexElements, Key.StreamStrides));
		
		check(VertexDeclarationRefPtr);
		check(IsValidRef(*VertexDeclarationRefPtr));
	}

	// The cached declaration must match the input declaration!
	check(VertexDeclarationRefPtr);
	check(IsValidRef(*VertexDeclarationRefPtr));
	FOpenGLVertexDeclaration* OpenGLVertexDeclaration = (FOpenGLVertexDeclaration*)VertexDeclarationRefPtr->GetReference();
	checkSlow(OpenGLVertexDeclaration->VertexElements == Key.VertexElements);

	return *VertexDeclarationRefPtr;
}

bool FOpenGLVertexDeclaration::GetInitializer(FVertexDeclarationElementList& Init)
{
	check(!Init.Num());
	for(int32 ElementIndex = 0;ElementIndex < VertexElements.Num();ElementIndex++)
	{
		FOpenGLVertexElement const& GLElement = VertexElements[ElementIndex];
		FVertexElement Element;
		Element.StreamIndex = GLElement.StreamIndex;
		Element.Offset = GLElement.Offset;
		Element.bUseInstanceIndex = GLElement.Divisor == 1;
		Element.AttributeIndex = GLElement.AttributeIndex;
		Element.Stride = GLElement.HashStride;
		
		switch(GLElement.Type)
		{
			case GL_FLOAT:
			{
				switch(GLElement.Size)
				{
					case 1:
						Element.Type = VET_Float1;
						break;
					case 2:
						Element.Type = VET_Float2;
						break;
					case 3:
						Element.Type = VET_Float3;
						break;
					case 4:
						Element.Type = VET_Float4;
						break;
					default:
						check(false);
						break;
				}
				break;
			}
			case GL_UNSIGNED_BYTE:
			{
				if (GLElement.Size == 4)
				{
					// Can't distinguish VET_PackedNormal, VET_Color & VET_UByte4N, but it shouldn't matter
					Element.Type = (GLElement.bNormalized) ? VET_UByte4N : VET_UByte4;
				}
				else if (GLElement.Size == GL_BGRA)
				{
					Element.Type = VET_Color;
				}
				else
				{
					check(false);
				}
				break;
			}
			case GL_BYTE:
			{
				if (GLElement.Size == 4)
				{
					// Can't distinguish VET_PackedNormal, VET_Color & VET_UByte4N, but it shouldn't matter
					ensure(GLElement.bNormalized);
					Element.Type = VET_PackedNormal;
				}
				else
				{
					checkf(false, TEXT("Vertex Declaration GL_BYTE, Size=%d"), GLElement.Size);
				}
				break;
			}
			case GL_SHORT:
			{
				switch(GLElement.Size)
				{
					case 2:
						if (GLElement.bNormalized)
						{
							Element.Type = VET_Short2N;
						}
						else
						{
							Element.Type = (!GLElement.bShouldConvertToFloat) ? VET_Short2 : VET_Half2;
						}
						break;
					case 4:
						if (GLElement.bNormalized)
						{
							Element.Type = VET_Short4N;
						}
						else
						{
							Element.Type = (!GLElement.bShouldConvertToFloat) ? VET_Short4 : VET_Half4;
						}
						break;
					default:
						check(false);
						break;
				}
				break;
			}
#if defined(GL_HALF_FLOAT)
			case GL_HALF_FLOAT:
#endif
#if defined(GL_HALF_FLOAT_OES)
			case GL_HALF_FLOAT_OES:
#endif
			{
				switch(GLElement.Size)
				{
					case 2:
						Element.Type = VET_Half2;
						break;
					case 4:
						Element.Type = VET_Half4;
						break;
					default:
						check(false);
						break;
				}
				break;
			}
			case GL_UNSIGNED_SHORT:
			{
				switch(GLElement.Size)
				{
					case 2:
						if (GLElement.bNormalized)
						{
							Element.Type = VET_UShort2N;
						}
						else
						{
							Element.Type = VET_UShort2;
						}
						break;
					case 4:
						if (GLElement.bNormalized)
						{
							Element.Type = VET_UShort4N;
						}
						else
						{
							Element.Type = VET_UShort4;
						}
						break;
					default:
						check(false);
						break;
				}
				break;
			}
			case GL_UNSIGNED_INT_2_10_10_10_REV:
			{
				Element.Type = VET_URGB10A2N;
				break;
			}
			case GL_UNSIGNED_INT:
			{
				Element.Type = VET_UInt;
				break;
			}
			default:
				checkf(false, TEXT("Unknown GLEnum 0x%x"), (int32)GLElement.Type);
				break;
		}
		Init.Add(Element);
	}

	return true;
}
