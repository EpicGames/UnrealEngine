// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLViewport.cpp: OpenGL viewport RHI implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "OpenGLDrv.h"
#include "OpenGLDrvPrivate.h"
#if PLATFORM_ANDROID
#include "Android/AndroidPlatformMisc.h"
#endif

static int32 GOpenGLPerFrameErrorCheck = 1;
static FAutoConsoleVariableRef CVarPerFrameGLErrorCheck(
	TEXT("r.OpenGL.PerFrameErrorCheck"),
	GOpenGLPerFrameErrorCheck,
	TEXT("When no other GL debugging is in use, check for GL errors once per frame.\nNot active in shipping builds.\n")
	TEXT("0: GL errors not be checked.\n")
	TEXT("1: any GL errors will be logged as errors. (default)\n")
	TEXT("2: any GL errors will be fatal.\n")
	,
	ECVF_RenderThreadSafe
);

static void CheckForGLErrors()
{
#if !UE_BUILD_SHIPPING 
	if (GOpenGLPerFrameErrorCheck && IsOGLDebugOutputEnabled() == false && !ENABLE_VERIFY_GL)
	{
		int32 Error = PlatformGlGetError();
		if (Error != GL_NO_ERROR)
		{
			switch (GOpenGLPerFrameErrorCheck)
			{
			case 1:
				UE_LOGF(LogRHI, Error, "GL Error encountered during frame %d, glerror=0x%x. Set command line arg -OpenGLDebugLevel=1 for detailed debugging.", GFrameNumber, Error);
				break;
			default: checkNoEntry(); [[fallthrough]];
			case 2:
				UE_LOGF(LogRHI, Fatal, "GL Error encountered during frame %d, glerror=0x%x. Set command line arg -OpenGLDebugLevel=1 for detailed debugging.", GFrameNumber, Error);
				break;
			}
		}
	}
#endif
}

void FOpenGLDynamicRHI::RHIGetSupportedResolution(uint32 &Width, uint32 &Height)
{
	PlatformGetSupportedResolution(Width, Height);
}

bool FOpenGLDynamicRHI::RHIGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	const bool Result = PlatformGetAvailableResolutions(Resolutions, bIgnoreRefreshRate);
	if (Result)
	{
		Resolutions.Sort([](const FScreenResolutionRHI& L, const FScreenResolutionRHI& R)
		{
			if (L.Width != R.Width)
			{
				return L.Width < R.Width;
			}
			else if (L.Height != R.Height)
			{
				return L.Height < R.Height;
			}
			else
			{
				return L.RefreshRate < R.RefreshRate;
			}
		});
	}
	return Result;
}

/*=============================================================================
 *	The following RHI functions must be called from the main thread.
 *=============================================================================*/
FViewportRHIRef FOpenGLDynamicRHI::RHICreateViewport(void* WindowHandle, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat PixelFormat)
{
	check(IsInGameThread());
	return new FOpenGLViewport(this, WindowHandle, SizeX, SizeY, bIsFullscreen, PixelFormat);
}

void FOpenGLDynamicRHI::RHIResizeViewport(FRHIViewport* ViewportRHI, uint32 SizeX, uint32 SizeY, bool bIsFullscreen, EPixelFormat /* PixelFormat */)
{
	check(IsInGameThread());
	ResourceCast(ViewportRHI)->Resize(SizeX, SizeY, bIsFullscreen);
}

void FOpenGLDynamicRHI::RHITick(float DeltaTime)
{
}

/*=============================================================================
 *	Viewport functions.
 *=============================================================================*/

void FOpenGLDynamicRHI::RHIEndDrawingViewport(FRHICommandListImmediate& RHICmdList, FRHIViewport* ViewportRHI, const FRHIPresentArgs& PresentArgs)
{
	//
	// Since OpenGL RHI does not implement parallel translation, we can enqueue the present from within an RHI command lambda.
	// This will run after all prior work that has already been recorded by the renderer, on the RHI thread.
	//
	RHICmdList.EnqueueLambda(TEXT("RHIEndDrawingViewport"), [this, ViewportRHI, PresentArgs](FRHICommandListImmediate&)
	{
		VERIFY_GL_SCOPE();

		FOpenGLViewport* Viewport = ResourceCast(ViewportRHI);

		SCOPE_CYCLE_COUNTER(STAT_OpenGLPresentTime);
		{
			FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUPresent);

			FOpenGLTexture* BackBuffer = Viewport->GetBackBuffer();

			if (ContextState.bScissorEnabled)
			{
				ContextState.bScissorEnabled = false;
				glDisable(GL_SCISSOR_TEST);
			}

			bool bNeedFinishFrame = PlatformBlitToViewport(*this, PlatformDevice,
				*Viewport, 
				BackBuffer->GetSizeX(),
				BackBuffer->GetSizeY(),
				PresentArgs.bPresent,
				PresentArgs.bLockToVsync
			);

			// Always consider the Framebuffer in the rendering context dirty after the blit
			ContextState.Framebuffer = -1;

			if (bNeedFinishFrame)
			{
				static const auto CFinishFrameVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FinishCurrentFrame"));
				int32 FinishCurrentFrame = CFinishFrameVar->GetValueOnRenderThread();

				if (FinishCurrentFrame == 0)
				{
					// Wait for the GPU to finish rendering the previous frame before finishing this frame.
					Viewport->WaitForFrameEventCompletion();
					Viewport->IssueFrameEvent();
				}
				else if (FinishCurrentFrame > 0)
				{
					// Finish current frame immediately to reduce latency
					Viewport->IssueFrameEvent();
					Viewport->WaitForFrameEventCompletion();
				}
				else if (FinishCurrentFrame < 0)
				{
					//No frame wait
				}
			}
		}

		EndFrameTick();

		CheckForGLErrors();
	});
}

FTextureRHIRef FOpenGLDynamicRHI::RHIGetViewportBackBuffer(FRHIViewport* ViewportRHI)
{
	FOpenGLViewport* Viewport = ResourceCast(ViewportRHI);
	return Viewport->GetBackBuffer();
}


FOpenGLViewport::FOpenGLViewport(FOpenGLDynamicRHI* InOpenGLRHI,void* InWindowHandle,uint32 InSizeX,uint32 InSizeY,bool bInIsFullscreen,EPixelFormat PreferredPixelFormat)
	: OpenGLRHI(InOpenGLRHI)
	, OpenGLContext(NULL)
	, SizeX(0)
	, SizeY(0)
	, bIsFullscreen(false)
	, PixelFormat(PreferredPixelFormat)
	, bIsValid(true)
	, WindowHandle(InWindowHandle)
{
	check(OpenGLRHI);
#if !PLATFORM_ANDROID
	check(InWindowHandle);
#endif
	check(IsInGameThread());

	// flush out old errors.
	PlatformGlGetError();

	OpenGLContext = PlatformCreateOpenGLContext(OpenGLRHI->PlatformDevice, WindowHandle);

#if PLATFORM_ANDROID
	if (!FAndroidMisc::UseNewWindowBehavior())
	{
		WindowHandle = nullptr; // The original window behaviour received a null handle.
		OpenGLSurfaceContext = AndroidEGL::GetInstance()->FindSurfaceContextFromWindow(nullptr); // there's only one window in this mode.
#if DO_CHECK
		FOpenGLDynamicRHI::FGLViewportContainer Viewports = FOpenGLDynamicRHI::Get().GetViewportContainer();
		// multiple viewports are not supported..
		check(Viewports.Get().IsEmpty());
#endif
	}
	else
#endif
	{
		check(WindowHandle);
		OpenGLSurfaceContext = PlatformCreateOpenGLSurfaceContext(OpenGLContext, WindowHandle);
	}

	Resize(InSizeX, InSizeY, bInIsFullscreen);

	{
		FOpenGLDynamicRHI::FGLViewportContainer Viewports = FOpenGLDynamicRHI::Get().GetViewportContainer();
		Viewports.Get().Add(this);
	}

	ENQUEUE_RENDER_COMMAND(CreateFrameSyncEvent)([this](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.EnqueueLambda([this](FRHICommandListImmediate&)
		{
			FrameSyncEvent = MakeUnique<FOpenGLEventQuery>();
		});
	});
}

FOpenGLViewport::~FOpenGLViewport()
{
	VERIFY_GL_SCOPE();

	{
		FOpenGLDynamicRHI::FGLViewportContainer Viewports = FOpenGLDynamicRHI::Get().GetViewportContainer();
		Viewports.Get().Remove(this);
	}

	if (bIsFullscreen)
	{
		PlatformRestoreDesktopDisplayMode();
	}

	// Release back buffer, before OpenGL context becomes invalid, making it impossible
	BackBuffer.SafeRelease();
	check(!IsValidRef(BackBuffer));

	FrameSyncEvent = nullptr;
	PlatformDestroyOpenGLSurfaceContext(OpenGLRHI->PlatformDevice, OpenGLContext, OpenGLSurfaceContext);
	PlatformDestroyOpenGLContext(OpenGLRHI->PlatformDevice, OpenGLContext);

	OpenGLContext = NULL;
}

void FOpenGLViewport::WaitForFrameEventCompletion()
{
	VERIFY_GL_SCOPE();
	FrameSyncEvent->WaitForCompletion();
}

void FOpenGLViewport::IssueFrameEvent()
{
	VERIFY_GL_SCOPE();
	FrameSyncEvent->IssueEvent();
}

void FOpenGLViewport::Resize(uint32 InSizeX,uint32 InSizeY,bool bInIsFullscreen)
{
	check(IsInGameThread());
	if ((InSizeX == SizeX) && (InSizeY == SizeY) && (bInIsFullscreen == bIsFullscreen))
	{
		return;
	}

	SizeX = InSizeX;
	SizeY = InSizeY;
	bool bWasFullscreen = bIsFullscreen;
	bIsFullscreen = bInIsFullscreen;

	ENQUEUE_RENDER_COMMAND(ResizeViewport)([this, InSizeX, InSizeY, bInIsFullscreen, bWasFullscreen](FRHICommandListImmediate& RHICmdList)
	{
		if (IsValidRef(CustomPresent))
		{
			CustomPresent->OnBackBufferResize();
		}

		BackBuffer.SafeRelease();	// when the rest of the engine releases it, its framebuffers will be released too (those the engine knows about)

		BackBuffer = PlatformCreateBuiltinBackBuffer(OpenGLRHI, OpenGLSurfaceContext, InSizeX, InSizeY);
		if (!BackBuffer)
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FOpenGLViewport"), InSizeX, InSizeY, PixelFormat)
				.SetClearValue(FClearValueBinding::Transparent)
				.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ResolveTargetable)
				.DetermineInititialState();

			BackBuffer = new FOpenGLTexture(Desc);
			BackBuffer->Initialize(RHICmdList);
		}

		RHICmdList.EnqueueLambda([this, InSizeX, InSizeY, bInIsFullscreen, bWasFullscreen](FRHICommandListImmediate&)
		{
			PlatformResizeGLContext(OpenGLRHI->PlatformDevice, OpenGLContext, OpenGLSurfaceContext, InSizeX, InSizeY, bInIsFullscreen, bWasFullscreen, BackBuffer->Target, BackBuffer->GetResource());
		});
	});	
}

void* FOpenGLViewport::GetNativeWindow(void** AddParam) const
{
	return PlatformGetWindow(OpenGLContext, AddParam);
}

