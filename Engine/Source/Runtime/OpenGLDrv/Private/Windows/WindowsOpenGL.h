// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenGLThirdParty.h"
#include "WindowsOpenGLPlatform.h"

// Set to 1 to enable creating an ES 3.1 context and use ES 3.1 shaders on Windows
#define EMULATE_ES31 0

#if !EMULATE_ES31
#include "OpenGL4.h"

// RenderDoc defines
#define GL_DEBUG_TOOL_EXT                 0x6789
#define GL_DEBUG_TOOL_NAME_EXT            0x678A
#define GL_DEBUG_TOOL_PURPOSE_EXT         0x678B

struct FWindowsOpenGL : public FOpenGL4
{
	static FORCEINLINE void InitDebugContext()
	{
		extern bool GRunningUnderRenderDoc;
		bDebugContext = glIsEnabled(GL_DEBUG_OUTPUT) != GL_FALSE || GRunningUnderRenderDoc;
	}

	static FORCEINLINE void LabelObject(GLenum Type, GLuint Object, const ANSICHAR* Name)
	{
		if (glObjectLabel && bDebugContext)
		{
			glObjectLabel(Type, Object, -1, Name);
		}
	}

	static FORCEINLINE void PushGroupMarker(const ANSICHAR* Name)
	{
		if (glPushDebugGroup && bDebugContext)
		{
			glPushDebugGroup(GL_DEBUG_SOURCE_APPLICATION, 1, FCStringAnsi::Strlen(Name), Name);
		}
	}

	static FORCEINLINE void PopGroupMarker()
	{
		if (glPopDebugGroup && bDebugContext)
		{
			glPopDebugGroup();
		}
	}

	static FORCEINLINE bool TexStorage2D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLenum Format, GLenum Type, ETextureCreateFlags Flags)
	{
		if (glTexStorage2D != NULL)
		{
			glTexStorage2D(Target, Levels, InternalFormat, Width, Height);
			return true;
		}
		else
		{
			return false;
		}
	}

	static FORCEINLINE bool TexStorage2DMultisample(GLenum Target, GLsizei Samples, GLint InternalFormat, GLsizei Width, GLsizei Height, GLboolean FixedSampleLocations)
	{
		if (glTexStorage2DMultisample != NULL)
		{
			glTexStorage2DMultisample(Target, Samples, InternalFormat, Width, Height, FixedSampleLocations);
			return true;
		}
		else
		{
			return false;
		}
	}

	static FORCEINLINE void TexStorage3D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type)
	{
		if (glTexStorage3D)
		{
			glTexStorage3D(Target, Levels, InternalFormat, Width, Height, Depth);
		}
		else
		{
			const bool bArrayTexture = Target == GL_TEXTURE_2D_ARRAY || Target == GL_TEXTURE_CUBE_MAP_ARRAY;

			for (uint32 MipIndex = 0; MipIndex < uint32(Levels); MipIndex++)
			{
				glTexImage3D(
					Target,
					MipIndex,
					InternalFormat,
					FMath::Max<uint32>(1, (Width >> MipIndex)),
					FMath::Max<uint32>(1, (Height >> MipIndex)),
					(bArrayTexture) ? Depth : FMath::Max<uint32>(1, (Depth >> MipIndex)),
					0,
					Format,
					Type,
					NULL
				);
			}
		}
	}

	static FORCEINLINE void CopyImageSubData(GLuint SrcName, GLenum SrcTarget, GLint SrcLevel, GLint SrcX, GLint SrcY, GLint SrcZ, GLuint DstName, GLenum DstTarget, GLint DstLevel, GLint DstX, GLint DstY, GLint DstZ, GLsizei Width, GLsizei Height, GLsizei Depth)
	{
		glCopyImageSubData(SrcName, SrcTarget, SrcLevel, SrcX, SrcY, SrcZ, DstName, DstTarget, DstLevel, DstX, DstY, DstZ, Width, Height, Depth);
	}

	static FORCEINLINE bool SupportsBufferStorage()
	{
		return glBufferStorage != NULL;
	}

	static FORCEINLINE bool SupportsDepthBoundsTest()
	{
		return glDepthBoundsEXT != NULL;
	}

	static FORCEINLINE void BufferStorage(GLenum Target, GLsizeiptr Size, const void* Data, GLbitfield Flags)
	{
		glBufferStorage(Target, Size, Data, Flags);
	}

	static FORCEINLINE void DepthBounds(GLfloat Min, GLfloat Max)
	{
		glDepthBoundsEXT(Min, Max);
	}

	static FORCEINLINE GLuint64 GetTextureSamplerHandle(GLuint Texture, GLuint Sampler)
	{
		return glGetTextureSamplerHandleARB(Texture, Sampler);
	}

	static FORCEINLINE GLuint64 GetTextureHandle(GLuint Texture)
	{
		return glGetTextureHandleARB(Texture);
	}

	static FORCEINLINE void MakeTextureHandleResident(GLuint64 TextureHandle)
	{
		glMakeTextureHandleResidentARB(TextureHandle);
	}

	static FORCEINLINE void MakeTextureHandleNonResident(GLuint64 TextureHandle)
	{
		glMakeTextureHandleNonResidentARB(TextureHandle);
	}

	static FORCEINLINE void UniformHandleui64(GLint Location, GLuint64 Value)
	{
		glUniformHandleui64ARB(Location, Value);
	}

	static FORCEINLINE bool SupportsProgramBinary()
	{
		return glProgramBinary != nullptr;
	}

	static FORCEINLINE void GetProgramBinary(GLuint Program, GLsizei BufSize, GLsizei* Length, GLenum* BinaryFormat, void* Binary)
	{
		glGetProgramBinary(Program, BufSize, Length, BinaryFormat, Binary);
	}

	static FORCEINLINE void ProgramBinary(GLuint Program, GLenum BinaryFormat, const void* Binary, GLsizei Length)
	{
		glProgramBinary(Program, BinaryFormat, Binary, Length);
	}
};

#else

//fix-up naming differences between OpenGL and OpenGL ES
#define glMapBufferOES glMapBuffer
#define glUnmapBufferOES glUnmapBuffer
#define GL_CLAMP_TO_BORDER_EXT GL_CLAMP_TO_BORDER
#define GL_WRITE_ONLY_OES GL_WRITE_ONLY
#define GL_ANY_SAMPLES_PASSED_EXT GL_ANY_SAMPLES_PASSED
#define GL_MAX_TESS_CONTROL_UNIFORM_COMPONENTS_EXT		GL_MAX_TESS_CONTROL_UNIFORM_COMPONENTS
#define GL_MAX_TESS_EVALUATION_UNIFORM_COMPONENTS_EXT	GL_MAX_TESS_EVALUATION_UNIFORM_COMPONENTS
#define GL_MAX_TESS_CONTROL_TEXTURE_IMAGE_UNITS_EXT		GL_MAX_TESS_CONTROL_TEXTURE_IMAGE_UNITS
#define GL_MAX_TESS_EVALUATION_TEXTURE_IMAGE_UNITS_EXT	GL_MAX_TESS_EVALUATION_TEXTURE_IMAGE_UNITS
#define GL_DEBUG_SOURCE_API_KHR				GL_DEBUG_SOURCE_API
#define GL_DEBUG_SOURCE_OTHER_KHR			GL_DEBUG_SOURCE_OTHER
#define GL_DEBUG_SOURCE_API_KHR				GL_DEBUG_SOURCE_API
#define GL_DEBUG_TYPE_ERROR_KHR				GL_DEBUG_TYPE_ERROR
#define GL_DEBUG_TYPE_OTHER_KHR				GL_DEBUG_TYPE_OTHER
#define GL_DEBUG_TYPE_ERROR_KHR				GL_DEBUG_TYPE_ERROR
#define GL_DEBUG_TYPE_MARKER_KHR			GL_DEBUG_TYPE_MARKER
#define GL_DEBUG_TYPE_POP_GROUP_KHR			GL_DEBUG_TYPE_POP_GROUP
#define GL_DEBUG_TYPE_MARKER_KHR			GL_DEBUG_TYPE_MARKER
#define GL_DEBUG_SEVERITY_HIGH_KHR			GL_DEBUG_SEVERITY_HIGH
#define GL_DEBUG_SEVERITY_LOW_KHR			GL_DEBUG_SEVERITY_LOW
#define GL_DEBUG_SEVERITY_HIGH_KHR			GL_DEBUG_SEVERITY_HIGH
#define GL_DEBUG_SEVERITY_NOTIFICATION_KHR	GL_DEBUG_SEVERITY_NOTIFICATION
#define GL_DEBUG_TYPE_ERROR_KHR				GL_DEBUG_TYPE_ERROR
#define GL_DEBUG_SEVERITY_HIGH_KHR			GL_DEBUG_SEVERITY_HIGH

#include "OpenGLES31.h"

struct FWindowsOpenGL : public FOpenGLESDeferred
{
	static FORCEINLINE EShaderPlatform GetShaderPlatform()
	{
		return SP_OPENGL_PCES3_1;
	}

	static FORCEINLINE void InitDebugContext()
	{
		bDebugContext = glIsEnabled(GL_DEBUG_OUTPUT) != GL_FALSE;
	}

	static FORCEINLINE void LabelObject(GLenum Type, GLuint Object, const ANSICHAR* Name)
	{
		if (glObjectLabelKHR && bDebugContext)
		{
			glObjectLabelKHR(Type, Object, -1, Name);
		}
	}

	static FORCEINLINE void PushGroupMarker(const ANSICHAR* Name)
	{
		if (glPushDebugGroupKHR && bDebugContext)
		{
			glPushDebugGroupKHR(GL_DEBUG_SOURCE_APPLICATION, 1, -1, Name);
		}
	}

	static FORCEINLINE void PopGroupMarker()
	{
		if (glPopDebugGroupKHR && bDebugContext)
		{
			glPopDebugGroupKHR();
		}
	}

	static FORCEINLINE bool TexStorage2D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLenum Format, GLenum Type, ETextureCreateFlags Flags)
	{
		if (glTexStorage2D != NULL)
		{
			glTexStorage2D(Target, Levels, InternalFormat, Width, Height);
			return true;
		}
		else
		{
			return false;
		}
	}

	static FORCEINLINE void TexStorage3D(GLenum Target, GLint Levels, GLint InternalFormat, GLsizei Width, GLsizei Height, GLsizei Depth, GLenum Format, GLenum Type)
	{
		if (glTexStorage3D)
		{
			glTexStorage3D(Target, Levels, InternalFormat, Width, Height, Depth);
		}
		else
		{
			const bool bArrayTexture = Target == GL_TEXTURE_2D_ARRAY || Target == GL_TEXTURE_CUBE_MAP_ARRAY;

			for (uint32 MipIndex = 0; MipIndex < uint32(Levels); MipIndex++)
			{
				glTexImage3D(
					Target,
					MipIndex,
					InternalFormat,
					FMath::Max<uint32>(1, (Width >> MipIndex)),
					FMath::Max<uint32>(1, (Height >> MipIndex)),
					(bArrayTexture) ? Depth : FMath::Max<uint32>(1, (Depth >> MipIndex)),
					0,
					Format,
					Type,
					NULL
				);
			}
		}
	}

	static FORCEINLINE void CopyImageSubData(GLuint SrcName, GLenum SrcTarget, GLint SrcLevel, GLint SrcX, GLint SrcY, GLint SrcZ, GLuint DstName, GLenum DstTarget, GLint DstLevel, GLint DstX, GLint DstY, GLint DstZ, GLsizei Width, GLsizei Height, GLsizei Depth)
	{
		glCopyImageSubData(SrcName, SrcTarget, SrcLevel, SrcX, SrcY, SrcZ, DstName, DstTarget, DstLevel, DstX, DstY, DstZ, Width, Height, Depth);
	}
};
#endif

using FOpenGL = FWindowsOpenGL;
