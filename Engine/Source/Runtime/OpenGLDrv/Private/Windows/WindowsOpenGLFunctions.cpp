// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsOpenGLFunctions.h"

#define DEFINE_GL_ENTRYPOINTS(Type,Func) Type Func = NULL;
	ENUM_GL_ENTRYPOINTS_ALL(DEFINE_GL_ENTRYPOINTS);
#undef DEFINE_GL_ENTRYPOINTS

PFNWGLCREATECONTEXTATTRIBSARBPROC wglCreateContextAttribsARB = NULL;
