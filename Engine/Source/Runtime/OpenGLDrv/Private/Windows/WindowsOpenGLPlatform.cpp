// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsOpenGLPlatform.h"

#include "HAL/Platform.h"
#include "Misc/ScopeLock.h"
#include "OpenGLDrvPrivate.h"
#include "OpenGLUtil.h"
#include "RHI.h"
#include "RHIUtilities.h"

bool GRunningUnderRenderDoc = false;

/*------------------------------------------------------------------------------
	OpenGL context management.
------------------------------------------------------------------------------*/

static void ContextMakeCurrent(HDC DC, HGLRC RC)
{
	BOOL Result = wglMakeCurrent(DC, RC);
	if (!Result)
	{
		Result = wglMakeCurrent(nullptr, nullptr);
	}
	check(Result);
}

static HGLRC GetCurrentContext()
{
	return wglGetCurrentContext();
}


/** Platform specific OpenGL context. */
struct FPlatformOpenGLContext
{
	HWND WindowHandle;
	HDC DeviceContext;
	HGLRC OpenGLContext;
	bool bReleaseWindowOnDestroy;
	int32 SyncInterval;
	GLuint	ViewportFramebuffer;
	GLuint	VertexArrayObject;	// one has to be generated and set for each context (OpenGL 3.2 Core requirements)
	GLuint	BackBufferResource;
	GLenum	BackBufferTarget;
};

struct FPlatformOpenGLSurfaceContext
{
};

class FScopeContext
{
public:
	FScopeContext(FPlatformOpenGLContext* Context)
	{
		check(Context);
		PrevDC = wglGetCurrentDC();
		PrevContext = GetCurrentContext();
		bSameDC = (PrevDC == Context->DeviceContext);
		bSameDCAndContext = (PrevContext == Context->OpenGLContext) && bSameDC;
		if (!bSameDCAndContext)
		{
			//			if (PrevContext)
			//			{
			//				glFlush();
			//			}
						// no need to glFlush() on Windows, it does flush by itself before switching contexts
			ContextMakeCurrent(Context->DeviceContext, Context->OpenGLContext);
		}
	}

	~FScopeContext(void)
	{
		if (!bSameDCAndContext)
		{
			//			glFlush();	// not needed on Windows, it does flush by itself before switching contexts
			if (PrevContext)
			{
				ContextMakeCurrent(PrevDC, PrevContext);
			}
			else
			{
				ContextMakeCurrent(NULL, NULL);
			}
		}
	}

	bool ContextWasAlreadyActive() const
	{
		return bSameDCAndContext;
	}

	bool ContextsShareSameDC() const
	{
		return bSameDC;
	}

private:
	HDC					PrevDC;
	HGLRC				PrevContext;
	bool				bSameDCAndContext;
	bool				bSameDC;
};

/**
 * A dummy wndproc.
 */
static LRESULT CALLBACK PlatformDummyGLWndproc(HWND hWnd, uint32 Message, WPARAM wParam, LPARAM lParam)
{
	return DefWindowProc(hWnd, Message, wParam, lParam);
}

/**
 * Initialize a pixel format descriptor for the given window handle.
 */
static void PlatformInitPixelFormatForDevice(HDC DeviceContext, bool bTryIsDummyContext)
{
	// Pixel format descriptor for the context.
	PIXELFORMATDESCRIPTOR PixelFormatDesc;
	FMemory::Memzero(PixelFormatDesc);
	PixelFormatDesc.nSize = sizeof(PIXELFORMATDESCRIPTOR);
	PixelFormatDesc.nVersion = 1;
	PixelFormatDesc.dwFlags = PFD_DRAW_TO_WINDOW | PFD_SUPPORT_OPENGL | PFD_DOUBLEBUFFER;
	PixelFormatDesc.iPixelType = PFD_TYPE_RGBA;
	PixelFormatDesc.cColorBits = 32;
	PixelFormatDesc.cDepthBits = 0;
	PixelFormatDesc.cStencilBits = 0;
	PixelFormatDesc.iLayerType = PFD_MAIN_PLANE;

	static bool bRequestedQuadBufferStereo = FParse::Param(FCommandLine::Get(), TEXT("quad_buffer_stereo"));
	if (bRequestedQuadBufferStereo)
	{
		PixelFormatDesc.dwFlags = PixelFormatDesc.dwFlags | PFD_STEREO;
	}

	// Set the pixel format and create the context.
	int32 PixelFormat = ChoosePixelFormat(DeviceContext, &PixelFormatDesc);
	if (!PixelFormat || !SetPixelFormat(DeviceContext, PixelFormat, &PixelFormatDesc))
	{
		UE_LOGF(LogRHI, Fatal, "Failed to set pixel format for device context.");
	}
}

/**
 * Create a dummy window used to construct OpenGL contexts.
 */
static void PlatformCreateDummyGLWindow(FPlatformOpenGLContext* OutContext)
{
	const TCHAR* WindowClassName = TEXT("DummyGLWindow");

	// Register a dummy window class.
	static bool bInitializedWindowClass = false;
	if (!bInitializedWindowClass)
	{
		WNDCLASS wc;

		bInitializedWindowClass = true;
		FMemory::Memzero(wc);
		wc.style = CS_OWNDC;
		wc.lpfnWndProc = PlatformDummyGLWndproc;
		wc.cbClsExtra = 0;
		wc.cbWndExtra = 0;
		wc.hInstance = NULL;
		wc.hIcon = NULL;
		wc.hCursor = NULL;
		wc.hbrBackground = (HBRUSH)(COLOR_MENUTEXT);
		wc.lpszMenuName = NULL;
		wc.lpszClassName = WindowClassName;
		ATOM ClassAtom = ::RegisterClass(&wc);
		check(ClassAtom);
	}

	int32 WinX = 0;
	int32 WinY = 0;
	FParse::Value(FCommandLine::Get(), TEXT("WinX="), WinX);
	FParse::Value(FCommandLine::Get(), TEXT("WinY="), WinY);

	// Create a dummy window.
	OutContext->WindowHandle = CreateWindowEx(
		WS_EX_WINDOWEDGE,
		WindowClassName,
		NULL,
		WS_POPUP,
		WinX, WinY, 1, 1,
		NULL, NULL, NULL, NULL);
	check(OutContext->WindowHandle);
	OutContext->bReleaseWindowOnDestroy = true;

	// Get the device context.
	OutContext->DeviceContext = GetDC(OutContext->WindowHandle);
	check(OutContext->DeviceContext);
	PlatformInitPixelFormatForDevice(OutContext->DeviceContext, true);
}

/**
 * Set OpenGL Context version to fixed version
 */

static void GetOpenGLVersionForCoreProfile(int& OutMajorVersion, int& OutMinorVersion)
{
	// Always initialize GL context with version 4.3, it's the only GL desktop version we support now
	OutMajorVersion = 4;
	OutMinorVersion = 3;
}

/**
 * Enable/Disable debug context from the commandline
 */
static bool PlatformOpenGLDebugCtx()
{
	return IsOGLDebugOutputEnabled();
}


/**
 * Create a core profile OpenGL context.
 */
static void PlatformCreateOpenGLContextCore(FPlatformOpenGLContext* OutContext, int MajorVersion, int MinorVersion, HGLRC InParentContext)
{
	check(wglCreateContextAttribsARB);
	check(OutContext);
	check(OutContext->DeviceContext);

	OutContext->SyncInterval = -1;	// invalid value to enforce setup on first buffer swap
	OutContext->ViewportFramebuffer = 0;

	int DebugFlag = 0;

	if (PlatformOpenGLDebugCtx())
	{
		DebugFlag = WGL_CONTEXT_DEBUG_BIT_ARB;
	}

#if !EMULATE_ES31
	int AttribList[] =
	{
		WGL_CONTEXT_MAJOR_VERSION_ARB, MajorVersion,
		WGL_CONTEXT_MINOR_VERSION_ARB, MinorVersion,
		WGL_CONTEXT_FLAGS_ARB, WGL_CONTEXT_FORWARD_COMPATIBLE_BIT_ARB | DebugFlag,
		WGL_CONTEXT_PROFILE_MASK_ARB, WGL_CONTEXT_CORE_PROFILE_BIT_ARB,
		0
	};
#else
	int AttribList[] =
	{
		WGL_CONTEXT_MAJOR_VERSION_ARB, 3,
		WGL_CONTEXT_MINOR_VERSION_ARB, 1,
		WGL_CONTEXT_FLAGS_ARB, DebugFlag,
		WGL_CONTEXT_PROFILE_MASK_ARB, WGL_CONTEXT_ES2_PROFILE_BIT_EXT,
		0
	};
#endif

	OutContext->OpenGLContext = wglCreateContextAttribsARB(OutContext->DeviceContext, InParentContext, AttribList);
}

void PlatformReleaseOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context);

/** Platform specific OpenGL device. */
struct FPlatformOpenGLDevice
{
	static FPlatformOpenGLDevice* Singleton;

	FPlatformOpenGLContext	RenderingContext;
	TArray<FPlatformOpenGLContext*>	ViewportContexts;
	bool					TargetDirty;

	/** Guards against operating on viewport contexts from more than one thread at the same time. */
	FCriticalSection* ContextUsageGuard;

	FPlatformOpenGLDevice()
		: TargetDirty(true)
	{
		check(!Singleton);
		Singleton = this;

		extern void InitDebugContext();
		ContextUsageGuard = new FCriticalSection;

		int MajorVersion = 0;
		int MinorVersion = 0;
		GetOpenGLVersionForCoreProfile(MajorVersion, MinorVersion);

		// Need to call this before we set the debug callback, otherwise if we're not running under RD, the debug extension will assert (invalid enum)
		GRunningUnderRenderDoc = glIsEnabled(GL_DEBUG_TOOL_EXT) != GL_FALSE;

		PlatformCreateDummyGLWindow(&RenderingContext);
		PlatformCreateOpenGLContextCore(&RenderingContext, MajorVersion, MinorVersion, nullptr);
		ContextMakeCurrent(RenderingContext.DeviceContext, RenderingContext.OpenGLContext);

		check(RenderingContext.OpenGLContext);
		
		InitDebugContext();
		glGenVertexArrays(1, &RenderingContext.VertexArrayObject);
		glBindVertexArray(RenderingContext.VertexArrayObject);
		InitDefaultGLContextState();
		glGenFramebuffers(1, &RenderingContext.ViewportFramebuffer);
	}

	~FPlatformOpenGLDevice()
	{
		check(ViewportContexts.Num() == 0);

		ContextMakeCurrent(NULL, NULL);
		PlatformReleaseOpenGLContext(this, &RenderingContext);

		delete ContextUsageGuard;

		check(Singleton == this);
		Singleton = nullptr;
	}
};

FPlatformOpenGLDevice* FPlatformOpenGLDevice::Singleton = nullptr;

FPlatformOpenGLDevice* PlatformCreateOpenGLDevice()
{
	return new FPlatformOpenGLDevice;
}

bool PlatformCanEnableGPUCapture()
{
	return GRunningUnderRenderDoc;
}

void PlatformDestroyOpenGLDevice(FPlatformOpenGLDevice* Device)
{
	delete Device;
}

/**
 * Create an OpenGL context.
 */
FPlatformOpenGLContext* PlatformCreateOpenGLContext(FPlatformOpenGLDevice* Device, void* InWindowHandle)
{
	check(InWindowHandle);

	Device->TargetDirty = true;

	FPlatformOpenGLContext* Context = new FPlatformOpenGLContext;
	Context->WindowHandle = (HWND)InWindowHandle;
	Context->bReleaseWindowOnDestroy = false;
	Context->DeviceContext = GetDC(Context->WindowHandle);
	check(Context->DeviceContext);
	PlatformInitPixelFormatForDevice(Context->DeviceContext, false);

	int MajorVersion = 0;
	int MinorVersion = 0;
	GetOpenGLVersionForCoreProfile(MajorVersion, MinorVersion);

	PlatformCreateOpenGLContextCore(Context, MajorVersion, MinorVersion, Device->RenderingContext.OpenGLContext);
	check(Context->OpenGLContext);
	{
		FScopeContext Scope(Context);
		InitDefaultGLContextState();
		glGenFramebuffers(1, &Context->ViewportFramebuffer);
	}

	Device->ViewportContexts.Add(Context);
	return Context;
}

FPlatformOpenGLSurfaceContext* PlatformCreateOpenGLSurfaceContext(FPlatformOpenGLContext* SourceContext, void* InWindowHandle)
{
	return nullptr;
}

void PlatformDestroyOpenGLSurfaceContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context, FPlatformOpenGLSurfaceContext* SurfaceContext)
{
}

/**
 * Release an OpenGL context.
 */
void PlatformReleaseOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context)
{
	check(Context && Context->OpenGLContext);

	Device->ViewportContexts.RemoveSingle(Context);
	Device->TargetDirty = true;

	bool bActiveContextWillBeReleased = false;
	bool bSharedDC = false;

	{
		FScopeLock ScopeLock(Device->ContextUsageGuard);
		{
			FScopeContext ScopeContext(Context);

			bActiveContextWillBeReleased = ScopeContext.ContextWasAlreadyActive();
			bSharedDC = ScopeContext.ContextsShareSameDC();

			glBindVertexArray(0);
			glDeleteVertexArrays(1, &Context->VertexArrayObject);

			if (Context->ViewportFramebuffer)
			{
				glDeleteFramebuffers(1, &Context->ViewportFramebuffer);	// this can be done from any context shared with ours, as long as it's not nil.
				Context->ViewportFramebuffer = 0;
			}
		}

		wglDeleteContext(Context->OpenGLContext);
		Context->OpenGLContext = NULL;
	}

	check(Context->DeviceContext);

	if (bActiveContextWillBeReleased)
	{
		wglMakeCurrent(NULL, NULL);
	}
	ReleaseDC(Context->WindowHandle, Context->DeviceContext);
	Context->DeviceContext = NULL;

	if (PlatformOpenGLThreadHasRenderingContext() && bSharedDC)
	{
		// The rendering context has been made current using the DC of the now destroyed context. Since this DC has been released the current context will be invalid.
		// To properly set the rendering context we must make current here with it's own DC.
		ContextMakeCurrent(Device->RenderingContext.DeviceContext, Device->RenderingContext.OpenGLContext);
	}

	check(Context->WindowHandle);

	if (Context->bReleaseWindowOnDestroy)
	{
		DestroyWindow(Context->WindowHandle);
	}
	Context->WindowHandle = NULL;
}

/**
 * Destroy an OpenGL context.
 */
void PlatformDestroyOpenGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context)
{
	PlatformReleaseOpenGLContext(Device, Context);
	delete Context;
}

/**
 * Main function for transferring data to on-screen buffers.
 * On Windows it temporarily switches OpenGL context, on Mac only context's output view.
 */
bool PlatformBlitToViewport(IRHICommandContext& RHICmdContext, FPlatformOpenGLDevice* Device, FOpenGLViewport& Viewport, uint32 BackbufferSizeX, uint32 BackbufferSizeY, bool bPresent, bool bLockToVsync)
{
	int32 SyncInterval = RHIGetSyncInterval();

	FPlatformOpenGLContext* const Context = Viewport.GetGLContext();

	check(Context && Context->DeviceContext);

	if (FOpenGL::IsAndroidGLESCompatibilityModeEnabled())
	{
		glDisable(GL_FRAMEBUFFER_SRGB);

		int32 RealSyncInterval = bLockToVsync ? SyncInterval : 0;
		if (wglSwapIntervalEXT_ProcAddress && Context->SyncInterval != RealSyncInterval)
		{
			wglSwapIntervalEXT_ProcAddress(RealSyncInterval);
			Context->SyncInterval = RealSyncInterval;
		}

		::SwapBuffers(Context->DeviceContext);
		return true;
	}

	FScopeLock ScopeLock(Device->ContextUsageGuard);
	{
		FPlatformOpenGLContext TempContext = *Context;
		if (Device->ViewportContexts.Num() == 1)
		{
			TempContext.OpenGLContext = Device->RenderingContext.OpenGLContext;
			TempContext.ViewportFramebuffer = Device->RenderingContext.ViewportFramebuffer;
		}
		FScopeContext ScopeContext(&TempContext);

		GLuint vfb = TempContext.ViewportFramebuffer;
		if (Viewport.GetCustomPresent())
		{
			Device->TargetDirty = false;
			glDisable(GL_FRAMEBUFFER_SRGB);
			bool bShouldPresent = Viewport.GetCustomPresent()->Present(&Viewport, RHICmdContext, SyncInterval);
			glEnable(GL_FRAMEBUFFER_SRGB);
			if (!bShouldPresent)
			{
				return false;
			}
			else
			{
				Device->TargetDirty = true;
			}
		}

		if (Device->ViewportContexts.Num() == 1 && Device->TargetDirty)
		{
			glBindFramebuffer(GL_FRAMEBUFFER, TempContext.ViewportFramebuffer);
			glFramebufferTexture2D(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, Context->BackBufferTarget, Context->BackBufferResource, 0);

			FOpenGL::CheckFrameBuffer();
			Device->TargetDirty = false;
		}

		glDisable(GL_FRAMEBUFFER_SRGB);
		glBindFramebuffer(GL_DRAW_FRAMEBUFFER, 0);
		FOpenGL::DrawBuffer(GL_BACK);
		glBindFramebuffer(GL_READ_FRAMEBUFFER, TempContext.ViewportFramebuffer);
		glReadBuffer(GL_COLOR_ATTACHMENT0);

		glBlitFramebuffer(
			0, 0, BackbufferSizeX, BackbufferSizeY,
			0, BackbufferSizeY, BackbufferSizeX, 0,
			GL_COLOR_BUFFER_BIT,
			GL_NEAREST
		);
		glEnable(GL_FRAMEBUFFER_SRGB);

		if (bPresent)
		{
			int32 RealSyncInterval = bLockToVsync ? SyncInterval : 0;
			if (wglSwapIntervalEXT_ProcAddress && Context->SyncInterval != RealSyncInterval)
			{
				wglSwapIntervalEXT_ProcAddress(RealSyncInterval);
				Context->SyncInterval = RealSyncInterval;
			}

			::SwapBuffers(Context->DeviceContext);
		}
	}
	return true;
}

void PlatformRenderingContextSetup(FPlatformOpenGLDevice* Device)
{
	check(Device && Device->RenderingContext.DeviceContext && Device->RenderingContext.OpenGLContext);

	if (GetCurrentContext())
	{
		glFlush();
	}

	if (Device->ViewportContexts.Num() == 1)
	{
		// use the HDC of the window, to reduce context swap overhead
		ContextMakeCurrent(Device->ViewportContexts[0]->DeviceContext, Device->RenderingContext.OpenGLContext);
	}
	else
	{
		ContextMakeCurrent(Device->RenderingContext.DeviceContext, Device->RenderingContext.OpenGLContext);
	}
}

void PlatformNULLContextSetup()
{
	if (wglGetCurrentDC())
	{
		// no need to glFlush() on Windows, it does flush by itself before switching contexts
		ContextMakeCurrent(NULL, NULL);
	}
}

/**
 * Resize the GL context.
 */
void PlatformResizeGLContext(FPlatformOpenGLDevice* Device, FPlatformOpenGLContext* Context, FPlatformOpenGLSurfaceContext* SurfaceContext, uint32 SizeX, uint32 SizeY, bool bFullscreen, bool bWasFullscreen, GLenum BackBufferTarget, GLuint BackBufferResource)
{
	FScopeLock ScopeLock(Device->ContextUsageGuard);
	{
		uint32 WindowStyle = WS_CAPTION | WS_SYSMENU;
		uint32 WindowStyleEx = 0;
		HWND InsertAfter = HWND_NOTOPMOST;

		if (bFullscreen)
		{
			// Get the monitor info from the window handle.
			HMONITOR hMonitor = MonitorFromWindow(Context->WindowHandle, MONITOR_DEFAULTTONEAREST);
			MONITORINFOEX MonitorInfo;
			memset(&MonitorInfo, 0, sizeof(MONITORINFOEX));
			MonitorInfo.cbSize = sizeof(MONITORINFOEX);
			GetMonitorInfo(hMonitor, &MonitorInfo);

			DEVMODE Mode;
			Mode.dmSize = sizeof(DEVMODE);
			Mode.dmBitsPerPel = 32;
			Mode.dmPelsWidth = SizeX;
			Mode.dmPelsHeight = SizeY;
			Mode.dmFields = DM_BITSPERPEL | DM_PELSWIDTH | DM_PELSHEIGHT;

			// Turn on fullscreen mode for the current monitor
			ChangeDisplaySettingsEx(MonitorInfo.szDevice, &Mode, NULL, CDS_FULLSCREEN, NULL);

			WindowStyle = WS_POPUP;
			WindowStyleEx = WS_EX_APPWINDOW | WS_EX_TOPMOST;
			InsertAfter = HWND_TOPMOST;
		}
		else if (bWasFullscreen)
		{
			ChangeDisplaySettings(NULL, 0);
		}

		Device->TargetDirty = true;
		Context->BackBufferResource = BackBufferResource;
		Context->BackBufferTarget = BackBufferTarget;

		//SetWindowLong(Context->WindowHandle, GWL_STYLE, WindowStyle);
		//SetWindowLong(Context->WindowHandle, GWL_EXSTYLE, WindowStyleEx);

		if (!FOpenGL::IsAndroidGLESCompatibilityModeEnabled())
		{
			FScopeContext ScopeContext(Context);

			glBindFramebuffer(GL_FRAMEBUFFER, Context->ViewportFramebuffer);
			glFramebufferTexture2D(GL_FRAMEBUFFER, GL_COLOR_ATTACHMENT0, BackBufferTarget, BackBufferResource, 0);
#if UE_BUILD_DEBUG
			glReadBuffer(GL_COLOR_ATTACHMENT0);
			FOpenGL::DrawBuffer(GL_COLOR_ATTACHMENT0);
#endif
			FOpenGL::CheckFrameBuffer();

			glViewport(0, 0, SizeX, SizeY);
			static GLfloat ZeroColor[4] = { 0.0f, 0.0f, 0.0f, 0.0f };
			glClearBufferfv(GL_COLOR, 0, ZeroColor);
		}
	}
}

void PlatformGetSupportedResolution(uint32& Width, uint32& Height)
{
	uint32 InitializedMode = false;
	uint32 BestWidth = 0;
	uint32 BestHeight = 0;
	uint32 ModeIndex = 0;
	DEVMODE DisplayMode;
	FMemory::Memzero(&DisplayMode, sizeof(DEVMODE));

	while (EnumDisplaySettings(NULL, ModeIndex++, &DisplayMode))
	{
		bool IsEqualOrBetterWidth = FMath::Abs((int32)DisplayMode.dmPelsWidth - (int32)Width) <= FMath::Abs((int32)BestWidth - (int32)Width);
		bool IsEqualOrBetterHeight = FMath::Abs((int32)DisplayMode.dmPelsHeight - (int32)Height) <= FMath::Abs((int32)BestHeight - (int32)Height);
		if (!InitializedMode || (IsEqualOrBetterWidth && IsEqualOrBetterHeight))
		{
			BestWidth = DisplayMode.dmPelsWidth;
			BestHeight = DisplayMode.dmPelsHeight;
			InitializedMode = true;
		}
	}
	check(InitializedMode);
	Width = BestWidth;
	Height = BestHeight;
}

bool PlatformGetAvailableResolutions(FScreenResolutionArray& Resolutions, bool bIgnoreRefreshRate)
{
	int32 MinAllowableResolutionX = 0;
	int32 MinAllowableResolutionY = 0;
	int32 MaxAllowableResolutionX = 10480;
	int32 MaxAllowableResolutionY = 10480;
	int32 MinAllowableRefreshRate = 0;
	int32 MaxAllowableRefreshRate = 10480;

	if (MaxAllowableResolutionX == 0) //-V547
	{
		MaxAllowableResolutionX = 10480;
	}
	if (MaxAllowableResolutionY == 0) //-V547
	{
		MaxAllowableResolutionY = 10480;
	}
	if (MaxAllowableRefreshRate == 0) //-V547
	{
		MaxAllowableRefreshRate = 10480;
	}

	uint32 ModeIndex = 0;
	DEVMODE DisplayMode;
	FMemory::Memzero(&DisplayMode, sizeof(DEVMODE));

	while (EnumDisplaySettings(NULL, ModeIndex++, &DisplayMode))
	{
		if (((int32)DisplayMode.dmPelsWidth >= MinAllowableResolutionX) &&
			((int32)DisplayMode.dmPelsWidth <= MaxAllowableResolutionX) &&
			((int32)DisplayMode.dmPelsHeight >= MinAllowableResolutionY) &&
			((int32)DisplayMode.dmPelsHeight <= MaxAllowableResolutionY)
			)
		{
			bool bAddIt = true;
			if (bIgnoreRefreshRate == false)
			{
				if (((int32)DisplayMode.dmDisplayFrequency < MinAllowableRefreshRate) ||
					((int32)DisplayMode.dmDisplayFrequency > MaxAllowableRefreshRate)
					)
				{
					continue;
				}
			}
			else
			{
				// See if it is in the list already
				for (int32 CheckIndex = 0; CheckIndex < Resolutions.Num(); CheckIndex++)
				{
					FScreenResolutionRHI& CheckResolution = Resolutions[CheckIndex];
					if ((CheckResolution.Width == DisplayMode.dmPelsWidth) &&
						(CheckResolution.Height == DisplayMode.dmPelsHeight))
					{
						// Already in the list...
						bAddIt = false;
						break;
					}
				}
			}

			if (bAddIt)
			{
				// Add the mode to the list
				int32 Temp2Index = Resolutions.AddZeroed();
				FScreenResolutionRHI& ScreenResolution = Resolutions[Temp2Index];

				ScreenResolution.Width = DisplayMode.dmPelsWidth;
				ScreenResolution.Height = DisplayMode.dmPelsHeight;
				ScreenResolution.RefreshRate = DisplayMode.dmDisplayFrequency;
			}
		}
	}

	return true;
}

void PlatformRestoreDesktopDisplayMode()
{
	ChangeDisplaySettings(NULL, 0);
}

bool PlatformInitOpenGL()
{
	static bool bInitialized = false;
	static bool bOpenGLSupported = false;
	if (!bInitialized)
	{
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

// Create a dummy context so that wglCreateContextAttribsARB can be initialized.
		FPlatformOpenGLContext DummyContext;
		PlatformCreateDummyGLWindow(&DummyContext);
		DummyContext.OpenGLContext = wglCreateContext(DummyContext.DeviceContext);
		check(DummyContext.OpenGLContext);
		ContextMakeCurrent(DummyContext.DeviceContext, DummyContext.OpenGLContext);
		wglCreateContextAttribsARB = (PFNWGLCREATECONTEXTATTRIBSARBPROC)wglGetProcAddress("wglCreateContextAttribsARB");
		if (wglCreateContextAttribsARB)
		{
			int MajorVersion = 0;
			int MinorVersion = 0;

			ContextMakeCurrent(NULL, NULL);
			wglDeleteContext(DummyContext.OpenGLContext);
			GetOpenGLVersionForCoreProfile(MajorVersion, MinorVersion);
			PlatformCreateOpenGLContextCore(&DummyContext, MajorVersion, MinorVersion, NULL);
			if (DummyContext.OpenGLContext)
			{
				bOpenGLSupported = true;
				ContextMakeCurrent(DummyContext.DeviceContext, DummyContext.OpenGLContext);
			}
			else
			{
				UE_LOGF(LogRHI, Error, "OpenGL %d.%d not supported by driver", MajorVersion, MinorVersion);
			}
		}

		if (bOpenGLSupported)
		{
			// Retrieve the OpenGL DLL.
			void* OpenGLDLL = FPlatformProcess::GetDllHandle(TEXT("opengl32.dll"));
			if (!OpenGLDLL)
			{
				UE_LOGF(LogRHI, Fatal, "Couldn't load opengl32.dll");
			}

			// Initialize entry points required by Unreal from opengl32.dll
#define GET_GL_ENTRYPOINTS_DLL(Type,Func) Func = (Type)FPlatformProcess::GetDllExport(OpenGLDLL,TEXT(#Func));
			ENUM_GL_ENTRYPOINTS_DLL(GET_GL_ENTRYPOINTS_DLL);
#undef GET_GL_ENTRYPOINTS_DLL

			// Release the OpenGL DLL.
			FPlatformProcess::FreeDllHandle(OpenGLDLL);

			// Initialize all entry points required by Unreal.
#define GET_GL_ENTRYPOINTS(Type,Func) Func = (Type)wglGetProcAddress(#Func);
			ENUM_GL_ENTRYPOINTS(GET_GL_ENTRYPOINTS);
			ENUM_GL_ENTRYPOINTS_OPTIONAL(GET_GL_ENTRYPOINTS);
#undef GET_GL_ENTRYPOINTS

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

// Check that all of the entry points have been initialized.
			bool bFoundAllEntryPoints = true;
#define CHECK_GL_ENTRYPOINTS(Type,Func) if (Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogRHI, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }
			ENUM_GL_ENTRYPOINTS_DLL(CHECK_GL_ENTRYPOINTS);
			ENUM_GL_ENTRYPOINTS(CHECK_GL_ENTRYPOINTS);
#undef CHECK_GL_ENTRYPOINTS
			checkf(bFoundAllEntryPoints, TEXT("Failed to find all OpenGL entry points."));
		}

		// The dummy context can now be released.
		if (DummyContext.OpenGLContext)
		{
			ContextMakeCurrent(NULL, NULL);
			wglDeleteContext(DummyContext.OpenGLContext);
		}
		ReleaseDC(DummyContext.WindowHandle, DummyContext.DeviceContext);
		check(DummyContext.bReleaseWindowOnDestroy);
		DestroyWindow(DummyContext.WindowHandle);

	}
	return bOpenGLSupported;
}

int32 PlatformGlGetError()
{
	return glGetError();
}

bool PlatformOpenGLThreadHasRenderingContext()
{
	return GetCurrentContext() == FPlatformOpenGLDevice::Singleton->RenderingContext.OpenGLContext;
}

// =============================================================

FOpenGLTexture* PlatformCreateBuiltinBackBuffer(FOpenGLDynamicRHI* OpenGLRHI, FPlatformOpenGLSurfaceContext* SurfaceContext, uint32 SizeX, uint32 SizeY)
{
	if (FOpenGL::IsAndroidGLESCompatibilityModeEnabled())
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("PlatformCreateBuiltinBackBuffer"), SizeX, SizeY, PF_B8G8R8A8)
			.SetClearValue(FClearValueBinding::Transparent)
			.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::Presentable | ETextureCreateFlags::ResolveTargetable)
			.DetermineInititialState();

		FOpenGLTexture* Texture = new FOpenGLTexture(Desc);
		Texture->Initialize(FRHICommandListImmediate::Get());
		return Texture;
	}

	return nullptr;
}

void* PlatformGetWindow(FPlatformOpenGLContext* Context, void** AddParam)
{
	check(Context && Context->WindowHandle);

	return (void*)&Context->WindowHandle;
}
