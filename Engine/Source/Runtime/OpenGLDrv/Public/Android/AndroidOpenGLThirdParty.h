// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

THIRD_PARTY_INCLUDES_START
	#include <EGL/egl.h>
	#include <EGL/eglext.h>
	#include <GLES2/gl2.h>
	#include <GLES2/gl2ext.h>
	#include <GLES3/gl31.h>
THIRD_PARTY_INCLUDES_END
