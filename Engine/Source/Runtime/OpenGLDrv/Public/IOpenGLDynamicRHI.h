// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RHI.h"

#include "OpenGLThirdParty.h"

struct IOpenGLDynamicRHI : public FDynamicRHIPSOFallback
{
	virtual ERHIInterfaceType GetInterfaceType() const override final { return ERHIInterfaceType::OpenGL; }

	virtual int32 RHIGetGLMajorVersion() const = 0;
	virtual int32 RHIGetGLMinorVersion() const = 0;

	virtual bool RHISupportsFramebufferSRGBEnable() const = 0;

	virtual FTextureRHIRef RHICreateTexture2DFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags Flags) = 0;
	virtual FTextureRHIRef RHICreateTexture2DArrayFromResource(EPixelFormat Format, uint32 SizeX, uint32 SizeY, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags Flags) = 0;
	virtual FTextureRHIRef RHICreateTextureCubeFromResource(EPixelFormat Format, uint32 Size, bool bArray, uint32 ArraySize, uint32 NumMips, uint32 NumSamples, uint32 NumSamplesTileMem, const FClearValueBinding& ClearValueBinding, GLuint Resource, ETextureCreateFlags Flags) = 0;
#if PLATFORM_ANDROID
	virtual FTextureRHIRef RHICreateTexture2DFromAndroidHardwareBuffer(FRHICommandListBase& RHICmdList, AHardwareBuffer* HardwareBuffer) = 0;
#endif //PLATFORM_ANDROID

	virtual GLuint RHIGetResource(FRHITexture* InTexture) const = 0;
	virtual bool RHIIsValidTexture(GLuint InTexture) const = 0;
	virtual void RHISetExternalGPUTime(uint64 InExternalGPUTime) = 0;

	virtual void RHIGenerateMips(FRHITexture* Texture) = 0;

#if PLATFORM_ANDROID
	virtual EGLDisplay RHIGetEGLDisplay() const = 0;
	virtual EGLSurface RHIGetEGLSurface() const = 0;
	virtual EGLConfig  RHIGetEGLConfig() const = 0;
	virtual EGLContext RHIGetEGLContext() const = 0;
	virtual ANativeWindow* RHIGetEGLNativeWindow() const = 0;
	virtual bool RHIEGLSupportsNoErrorContext() const = 0;

	virtual void RHIInitEGLInstanceGLES2() = 0;
	virtual void RHIInitEGLBackBuffer() = 0;
	virtual void RHIEGLSetCurrentRenderingContext() = 0;
	virtual void RHIEGLTerminateContext() = 0;
#endif
};

inline IOpenGLDynamicRHI* GetIOpenGLDynamicRHI()
{
	check(GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::OpenGL);
	return GetDynamicRHI<IOpenGLDynamicRHI>();
}
