// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

THIRD_PARTY_INCLUDES_START
	#include <GL/glcorearb.h>
	#include <GL/glext.h>
THIRD_PARTY_INCLUDES_END
