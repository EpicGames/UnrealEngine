// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <OpenGL/OpenGL.h>
#include <OpenGL/gl3.h>
#include <OpenGL/gl3ext.h>

#ifndef GL_COMPUTE_SHADER
	#define GL_COMPUTE_SHADER 0x91B9
#endif
#ifndef GL_TESS_EVALUATION_SHADER
	#define GL_TESS_EVALUATION_SHADER 0x8E87
#endif
#ifndef GL_TESS_CONTROL_SHADER
	#define GL_TESS_CONTROL_SHADER 0x8E88
#endif
