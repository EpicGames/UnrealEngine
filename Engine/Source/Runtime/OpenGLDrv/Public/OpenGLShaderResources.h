// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLShaderResources.h: OpenGL shader resource RHI definitions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "CrossCompilerCommon.h"
#include "OpenGLThirdParty.h"
#include "Hash/ShaderHash.h"

class FOpenGLLinkedProgram;

/** Set to 1 to enable shader debugging which e.g. keeps the GLSL source as members of TOpenGLShader*/
#define DEBUG_GL_SHADERS (UE_BUILD_DEBUG || UE_EDITOR)

/**
 * Shader related constants.
 */
enum
{
	OGL_MAX_UNIFORM_BUFFER_BINDINGS = 12,	// @todo-mobile: Remove me
	OGL_FIRST_UNIFORM_BUFFER = 0,			// @todo-mobile: Remove me
	OGL_UAV_NOT_SUPPORTED_FOR_GRAPHICS_UNIT = -1, // for now, only CS and PS supports UAVs/ images
};

struct FOpenGLShaderVarying
{
	TArray<ANSICHAR> Varying;
	int32 Location;
	
	friend bool operator==(const FOpenGLShaderVarying& A, const FOpenGLShaderVarying& B)
	{
		if(&A != &B)
		{
			return (A.Location == B.Location) && (A.Varying.Num() == B.Varying.Num()) && (FMemory::Memcmp(A.Varying.GetData(), B.Varying.GetData(), A.Varying.Num() * sizeof(ANSICHAR)) == 0);
		}
		return true;
	}
	
	friend uint32 GetTypeHash(const FOpenGLShaderVarying &Var)
	{
		uint32 Hash = GetTypeHash(Var.Location);
		Hash ^= FCrc::MemCrc32(Var.Varying.GetData(), Var.Varying.Num() * sizeof(ANSICHAR));
		return Hash;
	}
};

inline FArchive& operator<<(FArchive& Ar, FOpenGLShaderVarying& Var)
{
	Ar << Var.Varying;
	Ar << Var.Location;
	return Ar;
}

/**
 * Shader binding information.
 */
struct FOpenGLShaderBindings
{
	TArray<TArray<CrossCompiler::FPackedArrayInfo>>	PackedUniformBuffers;
	TArray<CrossCompiler::FPackedArrayInfo>			PackedGlobalArrays;
	TArray<FOpenGLShaderVarying>					InputVaryings;
	TArray<FOpenGLShaderVarying>					OutputVaryings;
	CrossCompiler::FShaderBindingInOutMask			InOutMask;

	uint8	NumSamplers = 0;
	uint8	NumUniformBuffers = 0;
	uint8	NumUAVs = 0;
	bool	bFlattenUB = false;

	FSHAHash VaryingHash; // Not serialized, built during load to allow us to diff varying info but avoid the memory overhead.

	inline FArchive& Serialize(FArchive& Ar, FShaderResourceTable& ShaderResourceTable);
};

inline FArchive& FOpenGLShaderBindings::Serialize(FArchive& Ar, FShaderResourceTable& ShaderResourceTable)
{
	Ar << PackedUniformBuffers;
	Ar << PackedGlobalArrays;
	Ar << InputVaryings;
	Ar << OutputVaryings;
	Ar << ShaderResourceTable;
	Ar << InOutMask;
	Ar << NumSamplers;
	Ar << NumUniformBuffers;
	Ar << NumUAVs;
	Ar << bFlattenUB;

	if (Ar.IsLoading())
	{
		// hash then strip out the Input/OutputVaryings at load time.
		// The hash ensures varying diffs still affect operator== and GetTypeHash()
		FSHA1 HashState;
		auto HashVarying = [](FSHA1& InHashState, const TArray<FOpenGLShaderVarying>& InInputVaryings)
		{
			for (const FOpenGLShaderVarying& Varying : InInputVaryings)
			{
				InHashState.Update((const uint8*)&Varying.Location, sizeof(Varying.Location));
				InHashState.Update((const uint8*)Varying.Varying.GetData(), Varying.Varying.Num() * sizeof(ANSICHAR));
			}
		};
		HashVarying(HashState, InputVaryings);
		HashVarying(HashState, OutputVaryings);
		HashState.Final();
		HashState.GetHash(&VaryingHash.Hash[0]);

		InputVaryings.Empty();
		OutputVaryings.Empty();
	}

	return Ar;
}

/**
 * Code header information.
 */
struct FOpenGLCodeHeader
{
	uint32 GlslMarker;
	uint16 FrequencyMarker;
	FOpenGLShaderBindings Bindings;
	FString ShaderName;
	TArray<CrossCompiler::FUniformBufferCopyInfo> UniformBuffersCopyInfo;

	inline FArchive& Serialize(FArchive& Ar, FShaderResourceTable& SRT);
};

inline FArchive& FOpenGLCodeHeader::Serialize(FArchive& Ar, FShaderResourceTable& SRT)
{
	Ar << GlslMarker;
	Ar << FrequencyMarker;
	Bindings.Serialize(Ar, SRT);
	Ar << ShaderName;
	int32 NumInfos = UniformBuffersCopyInfo.Num();
	Ar << NumInfos;
	if (Ar.IsSaving())
	{
		for (int32 Index = 0; Index < NumInfos; ++Index)
		{
			Ar << UniformBuffersCopyInfo[Index];
		}
	}
	else if (Ar.IsLoading())
	{
		UniformBuffersCopyInfo.Empty(NumInfos);
		for (int32 Index = 0; Index < NumInfos; ++Index)
		{
			CrossCompiler::FUniformBufferCopyInfo Info;
			Ar << Info;
			UniformBuffersCopyInfo.Add(Info);
		}
	}
    return Ar;
}

class FOpenGLLinkedProgram;

class FOpenGLCompiledShaderKey
{
public:
	FOpenGLCompiledShaderKey() = default;
	FOpenGLCompiledShaderKey(
		GLenum InTypeEnum,
		uint32 InCodeSize,
		uint32 InCodeCRC
	)
		: TypeEnum(InTypeEnum)
		, CodeSize(InCodeSize)
		, CodeCRC(InCodeCRC)
	{
	}

	friend bool operator == (const FOpenGLCompiledShaderKey& A, const FOpenGLCompiledShaderKey& B)
	{
		return A.TypeEnum == B.TypeEnum && A.CodeSize == B.CodeSize && A.CodeCRC == B.CodeCRC;
	}

	friend uint32 GetTypeHash(const FOpenGLCompiledShaderKey& Key)
	{
		return GetTypeHash(Key.TypeEnum) ^ GetTypeHash(Key.CodeSize) ^ GetTypeHash(Key.CodeCRC);
	}

	uint32 GetCodeCRC() const { return CodeCRC; }

private:
	GLenum TypeEnum = 0;
	uint32 CodeSize = 0;
	uint32 CodeCRC  = 0;
};

/**
 * OpenGL shader resource.
 */
class FOpenGLShader
{
public:
	/** The OpenGL resource ID. */
	GLuint Resource = 0;

	/** External bindings for this shader. */
	FOpenGLShaderBindings Bindings;

	// List of memory copies from RHIUniformBuffer to packed uniforms
	TArray<CrossCompiler::FUniformBufferCopyInfo> UniformBuffersCopyInfo;

	FOpenGLCompiledShaderKey ShaderCodeKey;

	bool bUsesProgrammableBlending = false;

#if DEBUG_GL_SHADERS
	TArray<ANSICHAR> GlslCode;
	const ANSICHAR*  GlslCodeString; // make it easier in VS to see shader code in debug mode; points to begin of GlslCode
#endif

	FOpenGLShader(TArrayView<const uint8> Code, GLenum TypeEnum, FShaderResourceTable& SRT, FRHIShader* RHIShader);

	~FOpenGLShader() = default;

protected:
	void Compile(GLenum TypeEnum);
};

class FOpenGLVertexShader : public FRHIVertexShader, public FOpenGLShader
{
public:
	static constexpr EShaderFrequency Frequency = SF_Vertex;

	FOpenGLVertexShader(TArrayView<const uint8> Code);

	void ConditionalyCompile();
};

class FOpenGLPixelShader : public FRHIPixelShader, public FOpenGLShader
{
public:
	static constexpr EShaderFrequency Frequency = SF_Pixel;

	FOpenGLPixelShader(TArrayView<const uint8> Code);

	void ConditionalyCompile();
};

class FOpenGLGeometryShader : public FRHIGeometryShader, public FOpenGLShader
{
public:
	static constexpr EShaderFrequency Frequency = SF_Geometry;

	FOpenGLGeometryShader(TArrayView<const uint8> Code);

	void ConditionalyCompile();
};

class FOpenGLComputeShader : public FRHIComputeShader, public FOpenGLShader
{
public:
	static constexpr EShaderFrequency Frequency = SF_Compute;

	FOpenGLComputeShader(TArrayView<const uint8> Code);

	void ConditionalyCompile();

	bool NeedsTextureStage(int32 TextureStageIndex);
	int32 MaxTextureStageUsed();
	const TBitArray<>& GetTextureNeeds(int32& OutMaxTextureStageUsed);
	const TBitArray<>& GetUAVNeeds(int32& OutMaxUAVUnitUsed) const;
	bool NeedsUAVStage(int32 UAVStageIndex) const;

	FOpenGLLinkedProgram* LinkedProgram = nullptr;
};

/**
 * Caching of OpenGL uniform parameters.
 */
class FOpenGLShaderParameterCache
{
public:
	/** Constructor. */
	FOpenGLShaderParameterCache();

	/** Destructor. */
	~FOpenGLShaderParameterCache();

	void InitializeResources(int32 UniformArraySize);

	/**
	 * Marks all uniform arrays as dirty.
	 */
	void MarkAllDirty();

	/**
	 * Sets values directly into the packed uniform array
	 */
	void Set(uint32 BufferIndex, uint32 ByteOffset, uint32 NumBytes, const void* NewValues);

	/**
	 * Commit shader parameters to the currently bound program.
	 * @param ParameterTable - Information on the bound uniform arrays for the program.
	 */
	void CommitPackedGlobals(const FOpenGLLinkedProgram* LinkedProgram, CrossCompiler::EShaderStage Stage);

	void CommitPackedUniformBuffers(FOpenGLLinkedProgram* LinkedProgram, CrossCompiler::EShaderStage Stage, FRHIUniformBuffer** UniformBuffers, const TArray<CrossCompiler::FUniformBufferCopyInfo>& UniformBuffersCopyInfo);

private:

	/** CPU memory block for storing uniform values. */
	uint8* PackedGlobalUniforms[CrossCompiler::PACKED_TYPEINDEX_MAX];
	
	struct FRange
	{
		uint32	StartVector;
		uint32	NumVectors;

		void MarkDirtyRange(uint32 NewStartVector, uint32 NewNumVectors);
	};
	/** Dirty ranges for each uniform array. */
	FRange	PackedGlobalUniformDirty[CrossCompiler::PACKED_TYPEINDEX_MAX];

	/** Scratch CPU memory block for uploading packed uniforms. */
	uint8* PackedUniformsScratch[CrossCompiler::PACKED_TYPEINDEX_MAX];

	/** in bytes */
	int32 GlobalUniformArraySize;
};

// unique identifier for a program. (composite of shader keys)
class FOpenGLProgramKey
{
public:
	FOpenGLProgramKey() = default;
	FOpenGLProgramKey(FRHIComputeShader* ComputeShaderRHI);
	FOpenGLProgramKey(FRHIVertexShader* VertexShaderRHI, FRHIPixelShader* PixelShaderRHI, FRHIGeometryShader* GeometryShaderRHI);

	friend bool operator == (const FOpenGLProgramKey& A, const FOpenGLProgramKey& B)
	{
		bool bHashMatch = true;
		for (uint32 i = 0; i < CrossCompiler::NUM_SHADER_STAGES && bHashMatch; ++i)
		{
			bHashMatch = A.ShaderHashes[i] == B.ShaderHashes[i];
		}
		return bHashMatch;
	}

	friend bool operator != (const FOpenGLProgramKey& A, const FOpenGLProgramKey& B)
	{
		return !(A==B);
	}

	friend uint32 GetTypeHash(const FOpenGLProgramKey& Key)
	{
		return FCrc::MemCrc32(Key.ShaderHashes, sizeof(Key.ShaderHashes));
	}

	friend FArchive& operator<<(FArchive& Ar, FOpenGLProgramKey& HashSet)
	{
		for (int32 Stage = 0; Stage < CrossCompiler::NUM_SHADER_STAGES; Stage++)
		{
			Ar << HashSet.ShaderHashes[Stage];
		}
		return Ar;
	}

	FString ToString() const
	{
		FString retme;
		if(ShaderHashes[CrossCompiler::SHADER_STAGE_VERTEX] != FShaderHash())
		{
			retme = TEXT("Program V_") + ShaderHashes[CrossCompiler::SHADER_STAGE_VERTEX].ToString();
			retme += TEXT("_P_") + ShaderHashes[CrossCompiler::SHADER_STAGE_PIXEL].ToString();
			return retme;
		}
		else if(ShaderHashes[CrossCompiler::SHADER_STAGE_COMPUTE] != FShaderHash())
		{
			retme = TEXT("Program C_") + ShaderHashes[CrossCompiler::SHADER_STAGE_COMPUTE].ToString();
			return retme;
		}
		else
		{
			retme = TEXT("Program with unset key");
			return retme;
		}
	}

	FShaderHash ShaderHashes[CrossCompiler::NUM_SHADER_STAGES];
};

