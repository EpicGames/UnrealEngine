// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OpenGLUtil.h: OpenGL RHI utility definitions.
=============================================================================*/

#pragma once

#include "RHICommandList.h"
#include "OpenGLThirdParty.h"
#include "OpenGLShaderResources.h"

class FOpenGLTexture;

/** Set to 1 to enable the VERIFY_GL macros which call glGetError */
#define ENABLE_VERIFY_GL (0 & DO_CHECK)
#define ENABLE_VERIFY_GL_TRACE 0

// Include GL debug output functionality on everything but shipping configs.
// to enable the debug output specify '-OpenGLDebugLevel=[1-5]' via the command line.
#define ENABLE_DEBUG_OUTPUT	(!UE_BUILD_SHIPPING)
#if !ENABLE_DEBUG_OUTPUT
inline bool IsOGLDebugOutputEnabled() { return false; }
inline int32 GetOGLDebugOutputLevel() { return 0; }
#else
bool IsOGLDebugOutputEnabled();
int32 GetOGLDebugOutputLevel();
#endif

// Additional check that our GL calls are occurring on the expected thread
#define ENABLE_VERIFY_GL_THREAD (!(UE_BUILD_TEST || UE_BUILD_SHIPPING))

/** Set to 1 to verify that the the engine side uniform buffer layout matches the driver side of the GLSL shader*/
#define ENABLE_UNIFORM_BUFFER_LAYOUT_VERIFICATION ( 0 & UE_BUILD_DEBUG & (!PLATFORM_ANDROID))

/** Set to 1 to additinally dump uniform buffer layout at shader link time, this assumes ENABLE_UNIFORM_BUFFER_LAYOUT_VERIFICATION == 1 */
#define ENABLE_UNIFORM_BUFFER_LAYOUT_DUMP 0

/** Set to 1 to enable calls to place event markers into the OpenGL stream
    this is purposefully not considered for OPENGL_PERFORMANCE_DATA_INVALID, 
	since there is an additional cvar OpenGLConsoleVariables::bEnableARBDebug*/

#define ENABLE_OPENGL_DEBUG_GROUPS 1

#define OPENGL_PERFORMANCE_DATA_INVALID (ENABLE_VERIFY_GL | ENABLE_UNIFORM_BUFFER_LAYOUT_VERIFICATION | DEBUG_GL_SHADERS)

void SetOpenGLResourceName(FOpenGLTexture* Texture, const ANSICHAR* Name);
void SetOpenGLResourceName(FOpenGLTexture* Texture, const TCHAR* Name);

/**
* Convert from ECubeFace to GLenum type
* @param Face - ECubeFace type to convert
* @return OpenGL cube face enum value
*/
GLenum GetOpenGLCubeFace(ECubeFace Face);

extern bool PlatformOpenGLThreadHasRenderingContext();

#if ENABLE_VERIFY_GL_THREAD
	#define CHECK_EXPECTED_GL_THREAD() \
		if (!PlatformOpenGLThreadHasRenderingContext()) \
		{ \
			UE_LOGF(LogRHI, Fatal, "Potential use of GL context from incorrect thread. [IsInGameThread() = %d, IsInRenderingThread() = %d, IsInRHIThread() = %d, IsRunningRHIInSeparateThread() = %d]", IsInGameThread(), IsInRenderingThread(), IsInRHIThread(), IsRunningRHIInSeparateThread()); \
		}
#else
	#define CHECK_EXPECTED_GL_THREAD() 
#endif

#if ENABLE_VERIFY_GL
	extern int32 PlatformGlGetError();

	void VerifyOpenGLResult(GLenum ErrorCode, const TCHAR* Msg1, const TCHAR* Msg2, const TCHAR* Filename, uint32 Line);
	#define VERIFY_GL(msg) { CHECK_EXPECTED_GL_THREAD(); GLenum ErrorCode = PlatformGlGetError(); if (ErrorCode != GL_NO_ERROR) { VerifyOpenGLResult(ErrorCode,TEXT(#msg),TEXT(""),TEXT(__FILE__),__LINE__); } }

	struct FOpenGLErrorScope
	{
		const char* FunctionName;
		const TCHAR* Filename;
		const uint32 Line;

		FOpenGLErrorScope(
			const char* InFunctionName,
			const TCHAR* InFilename,
			const uint32 InLine)
			: FunctionName(InFunctionName)
			, Filename(InFilename)
			, Line(InLine)
		{
#if ENABLE_VERIFY_GL_TRACE
			UE_LOGF(LogRHI, Log, "log before %ls(%d): %ls", InFilename, InLine, ANSI_TO_TCHAR(InFunctionName));
#endif
			CheckForErrors(0);
		}

		~FOpenGLErrorScope()
		{
#if ENABLE_VERIFY_GL_TRACE
			UE_LOGF(LogRHI, Log, "log after  %ls(%d): %ls", Filename, Line, ANSI_TO_TCHAR(FunctionName));

#endif

			CheckForErrors(1);
		}

		void CheckForErrors(int32 BeforeOrAfter)
		{
			check(PlatformOpenGLThreadHasRenderingContext());

			GLenum ErrorCode = PlatformGlGetError();
			if (ErrorCode != GL_NO_ERROR)
			{
				const TCHAR* PrefixStrings[] = { TEXT("Before "), TEXT("During ") };
				VerifyOpenGLResult(ErrorCode,PrefixStrings[BeforeOrAfter], ANSI_TO_TCHAR(FunctionName),Filename,Line);
			}
		}
	};
	#define MACRO_TOKENIZER(IdentifierName, Msg, FileName, LineNumber) FOpenGLErrorScope IdentifierName_ ## LineNumber (Msg, FileName, LineNumber)
	#define MACRO_TOKENIZER2(IdentifierName, Msg, FileName, LineNumber) MACRO_TOKENIZER(IdentiferName, Msg, FileName, LineNumber)
	#define VERIFY_GL_SCOPE_WITH_MSG_STR(MsgStr) CHECK_EXPECTED_GL_THREAD(); MACRO_TOKENIZER2(ErrorScope_, MsgStr, TEXT(__FILE__), __LINE__)
	#define VERIFY_GL_SCOPE() VERIFY_GL_SCOPE_WITH_MSG_STR(__FUNCTION__)
	#define VERIFY_GL_FUNC(Func, ...) { VERIFY_GL_SCOPE_WITH_MSG_STR((#Func)); Func(__VA_ARGS__); }

	/**
	 * Some important GL calls are trapped individually.
	 */
	#define glBlitFramebuffer(...) VERIFY_GL_FUNC(glBlitFramebuffer, __VA_ARGS__)
	#define glTexImage2D(...) VERIFY_GL_FUNC(glTexImage2D, __VA_ARGS__)
	#define glTexSubImage2D(...) VERIFY_GL_FUNC(glTexSubImage2D, __VA_ARGS__)
	#define glCompressedTexImage2D(...) VERIFY_GL_FUNC(glCompressedTexImage2D, __VA_ARGS__)

#else
	#define VERIFY_GL(...) CHECK_EXPECTED_GL_THREAD()
	#define VERIFY_GL_SCOPE(...) CHECK_EXPECTED_GL_THREAD()
#endif

struct FRHICommandGLCommandString
{
	static const TCHAR* TStr() { return TEXT("FRHICommandGLCommand"); }
};

#define GL_CAPTURE_CALLSTACK 0 // Capture the callstack at the point of enqueuing the command. 

